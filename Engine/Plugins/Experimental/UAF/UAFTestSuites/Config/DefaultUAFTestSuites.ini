[CoreRedirects]
+PropertyRedirects=(OldName="/Script/UAFTestSuite.UAFVariableReferenceTestStruct.BoolVariable",NewName="/Script/UAFTestSuite.UAFVariableReferenceTestStruct.Renamed_BoolVariable")