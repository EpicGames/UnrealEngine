// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextAnimGraphTraitEventTest.h"

#include "AnimNextRuntimeTest.h"
#include "AnimNextTest.h"
#include "AnimNextAnimGraphTraitInterfacesTest.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitBinding.h"
#include "TraitCore/TraitReader.h"
#include "TraitCore/TraitWriter.h"
#include "TraitCore/ExecutionContext.h"
#include "TraitCore/NodeTemplateBuilder.h"
#include "TraitCore/NodeTemplateRegistry.h"
#include "TraitInterfaces/IHierarchy.h"
#include "TraitInterfaces/IUpdate.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextAnimationGraphFactory.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

//****************************************************************************
// AnimNext Runtime Trait Graph Event Tests
//****************************************************************************

namespace UE::UAF
{
	struct FTraitGraphTest_EventAB_NoChildren : FBaseTrait
	{
		DECLARE_ANIM_TRAIT(FTraitGraphTest_EventAB_NoChildren, FBaseTrait)

		ETraitStackPropagation OnEventA(const FExecutionContext& Context, FTraitBinding& Binding, FTraitAnimGraphTest_EventA& Event) const
		{
			Event.VisitedTraits.Add(FTraitGraphTest_EventAB_NoChildren::TraitUID);
			return ETraitStackPropagation::Continue;
		}

		ETraitStackPropagation OnEventB(const FExecutionContext& Context, FTraitBinding& Binding, FTraitAnimGraphTest_EventB& Event) const
		{
			Event.VisitedTraits.Add(FTraitGraphTest_EventAB_NoChildren::TraitUID);
			return ETraitStackPropagation::Continue;
		}
	};

	// Trait implementation boilerplate
	#define TRAIT_EVENT_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(FTraitGraphTest_EventAB_NoChildren::OnEventA) \
		GeneratorMacro(FTraitGraphTest_EventAB_NoChildren::OnEventB) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FTraitGraphTest_EventAB_NoChildren, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_EVENT_ENUMERATOR

	struct FTraitGraphTest_EventAB_OneChild : FBaseTrait, IHierarchy
	{
		DECLARE_ANIM_TRAIT(FTraitGraphTest_EventAB_OneChild, FBaseTrait)

		using FSharedData = FTraitWithOneChildSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			FTraitPtr Child;

			void Construct(const FExecutionContext& Context, const FTraitBinding& Binding)
			{
				Child = Context.AllocateNodeInstance(Binding.GetTraitPtr(), Binding.GetSharedData<FSharedData>()->Child);
			}
		};

		ETraitStackPropagation OnEventA(const FExecutionContext& Context, FTraitBinding& Binding, FTraitAnimGraphTest_EventA& Event) const
		{
			if (Event.bTestFlag)
			{
				Event.MarkConsumed();
			}

			Event.VisitedTraits.Add(FTraitGraphTest_EventAB_OneChild::TraitUID);
			return ETraitStackPropagation::Continue;
		}

		ETraitStackPropagation OnEventB(FExecutionContext& Context, FTraitBinding& Binding, FTraitAnimGraphTest_EventB& Event) const
		{
			if (Event.bTestFlag0)
			{
				auto EventA = MakeTraitEvent<FTraitAnimGraphTest_EventA>();
				Event.ChildEvent = EventA;

				Context.RaiseInputTraitEvent(EventA);
			}

			if (Event.bTestFlag1)
			{
				auto EventA = MakeTraitEvent<FTraitAnimGraphTest_EventA>();
				Event.ChildEvent = EventA;

				Context.RaiseOutputTraitEvent(EventA);
			}

			Event.VisitedTraits.Add(FTraitGraphTest_EventAB_OneChild::TraitUID);
			return ETraitStackPropagation::Continue;
		}

		// IHierarchy impl
		virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override
		{
			return 1;
		}

		virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override
		{
			const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

			Children.Add(InstanceData->Child);
		}
	};

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IHierarchy) \

	#define TRAIT_EVENT_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(FTraitGraphTest_EventAB_OneChild::OnEventA) \
		GeneratorMacro(FTraitGraphTest_EventAB_OneChild::OnEventB) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FTraitGraphTest_EventAB_OneChild, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR
	#undef TRAIT_EVENT_ENUMERATOR

	struct FTraitGraphTest_EventAB_TwoChildren : FBaseTrait, IHierarchy
	{
		DECLARE_ANIM_TRAIT(FTraitGraphTest_EventAB_TwoChildren, FBaseTrait)

		using FSharedData = FTraitWithChildrenSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			FTraitPtr Children[2];

			void Construct(const FExecutionContext& Context, const FTraitBinding& Binding)
			{
				Children[0] = Context.AllocateNodeInstance(Binding.GetTraitPtr(), Binding.GetSharedData<FSharedData>()->Children[0]);
				Children[1] = Context.AllocateNodeInstance(Binding.GetTraitPtr(), Binding.GetSharedData<FSharedData>()->Children[1]);
			}
		};

		ETraitStackPropagation OnEventA(const FExecutionContext& Context, FTraitBinding& Binding, FTraitAnimGraphTest_EventA& Event) const
		{
			Event.VisitedTraits.Add(FTraitGraphTest_EventAB_TwoChildren::TraitUID);
			return ETraitStackPropagation::Continue;
		}

		ETraitStackPropagation OnEventB(const FExecutionContext& Context, FTraitBinding& Binding, FTraitAnimGraphTest_EventB& Event) const
		{
			Event.VisitedTraits.Add(FTraitGraphTest_EventAB_TwoChildren::TraitUID);
			return ETraitStackPropagation::Continue;
		}

		// IHierarchy impl
		virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override
		{
			return 2;
		}

		virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override
		{
			const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

			Children.Add(InstanceData->Children[0]);
			Children.Add(InstanceData->Children[1]);
		}
	};

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IHierarchy) \

	#define TRAIT_EVENT_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(FTraitGraphTest_EventAB_TwoChildren::OnEventA) \
		GeneratorMacro(FTraitGraphTest_EventAB_TwoChildren::OnEventB) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FTraitGraphTest_EventAB_TwoChildren, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR
	#undef TRAIT_EVENT_ENUMERATOR
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnimationAnimNextRuntimeTest_GraphTraitEvent, "Animation.AnimNext.Runtime.Graph.TraitEvent", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAnimationAnimNextRuntimeTest_GraphTraitEvent::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	{
		AUTO_REGISTER_ANIM_TRAIT(FTraitGraphTest_EventAB_NoChildren)
		AUTO_REGISTER_ANIM_TRAIT(FTraitGraphTest_EventAB_OneChild)
		AUTO_REGISTER_ANIM_TRAIT(FTraitGraphTest_EventAB_TwoChildren)

		UFactory* GraphFactory = NewObject<UUAFAnimGraphFactory>();
		UUAFAnimGraph* AnimationGraph = CastChecked<UUAFAnimGraph>(GraphFactory->FactoryCreateNew(UUAFAnimGraph::StaticClass(), GetTransientPackage(), TEXT("TestAnimNextGraph"), RF_Transient, nullptr, nullptr, NAME_None));
		UE_RETURN_ON_ERROR(AnimationGraph != nullptr, "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Failed to create animation graph");

		FScopedClearNodeTemplateRegistry ScopedClearNodeTemplateRegistry;
		FNodeTemplateRegistry& Registry = FNodeTemplateRegistry::Get();

		// Graph: Root -> NodeA -> NodeB
		//                NodeC

		// We create a few node templates
		// Template A has a single child node (NodeA)
		TArray<FTraitUID> NodeTemplateTraitListA;
		NodeTemplateTraitListA.Add(FTraitGraphTest_EventAB_OneChild::TraitUID);

		// Template B has two children (Root)
		TArray<FTraitUID> NodeTemplateTraitListB;
		NodeTemplateTraitListB.Add(FTraitGraphTest_EventAB_TwoChildren::TraitUID);

		// Template C has no children (NodeB and NodeC)
		TArray<FTraitUID> NodeTemplateTraitListC;
		NodeTemplateTraitListC.Add(FTraitGraphTest_EventAB_NoChildren::TraitUID);

		// Populate our node template registry
		TArray<uint8> NodeTemplateBufferA, NodeTemplateBufferB, NodeTemplateBufferC;
		const FNodeTemplate* NodeTemplateA = FNodeTemplateBuilder::BuildNodeTemplate(NodeTemplateTraitListA, NodeTemplateBufferA);
		const FNodeTemplate* NodeTemplateB = FNodeTemplateBuilder::BuildNodeTemplate(NodeTemplateTraitListB, NodeTemplateBufferB);
		const FNodeTemplate* NodeTemplateC = FNodeTemplateBuilder::BuildNodeTemplate(NodeTemplateTraitListC, NodeTemplateBufferC);

		TArray<FNodeHandle> NodeHandles;

		// Write our graph
		TArray<uint8> GraphSharedDataArchiveBuffer;
		TArray<TObjectPtr<UObject>> GraphReferencedObjects;
		TArray<FSoftObjectPath> GraphReferencedSoftObjects;
		{
			FTraitWriter TraitWriter;

			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplateB));	// Root
			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplateA));	// NodeA
			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplateC));	// NodeB
			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplateC));	// NodeC

			TArray<TMap<FName, FString>> TraitPropertiesRoot;
			TraitPropertiesRoot.AddDefaulted(NodeTemplateTraitListB.Num());
			FAnimNextTraitHandle ChildrenHandlesRoot[2] = { FAnimNextTraitHandle(NodeHandles[1]), FAnimNextTraitHandle(NodeHandles[3]) };
			TraitPropertiesRoot[0].Add(TEXT("Children"), ToString<FTraitGraphTest_EventAB_TwoChildren::FSharedData>(TEXT("Children"), ChildrenHandlesRoot));

			TArray<TMap<FName, FString>> TraitPropertiesNodeA;
			TraitPropertiesNodeA.AddDefaulted(NodeTemplateTraitListA.Num());
			TraitPropertiesNodeA[0].Add(TEXT("Child"), ToString<FTraitGraphTest_EventAB_OneChild::FSharedData>(TEXT("Child"), FAnimNextTraitHandle(NodeHandles[2])));

			TArray<TMap<FName, FString>> TraitPropertiesNodeB;
			TraitPropertiesNodeB.AddDefaulted(NodeTemplateTraitListC.Num());

			TArray<TMap<FName, FString>> TraitPropertiesNodeC;
			TraitPropertiesNodeC.AddDefaulted(NodeTemplateTraitListC.Num());

			TraitWriter.BeginNodeWriting();
			TraitWriter.WriteNode(NodeHandles[0],
				[&TraitPropertiesRoot](uint32 TraitIndex, FName PropertyName)
				{
					return TraitPropertiesRoot[TraitIndex][PropertyName];
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.WriteNode(NodeHandles[1],
				[&TraitPropertiesNodeA](uint32 TraitIndex, FName PropertyName)
				{
					return TraitPropertiesNodeA[TraitIndex][PropertyName];
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.WriteNode(NodeHandles[2],
				[&TraitPropertiesNodeB](uint32 TraitIndex, FName PropertyName)
				{
					return TraitPropertiesNodeB[TraitIndex][PropertyName];
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.WriteNode(NodeHandles[3],
				[&TraitPropertiesNodeC](uint32 TraitIndex, FName PropertyName)
				{
					return TraitPropertiesNodeC[TraitIndex][PropertyName];
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.EndNodeWriting();

			AddErrorIfFalse(TraitWriter.GetErrorState() == FTraitWriter::EErrorState::None, "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Failed to write traits");
			GraphSharedDataArchiveBuffer = TraitWriter.GetGraphSharedData();
			GraphReferencedObjects = TraitWriter.GetGraphReferencedObjects();
			GraphReferencedSoftObjects = TraitWriter.GetGraphReferencedSoftObjects();
		}

		// Read our graph
		FTestUtils::LoadFromArchiveBuffer(*AnimationGraph, NodeHandles, GraphSharedDataArchiveBuffer);

		TSharedPtr<FAnimNextGraphInstance> GraphInstance = AnimationGraph->AllocateInstance();

		{
			// Raise EventA and EventB on graph, every node sees them
			{
				auto EventA = MakeTraitEvent<FTraitAnimGraphTest_EventA>();
				auto EventB = MakeTraitEvent<FTraitAnimGraphTest_EventB>();

				FUpdateGraphContext UpdateGraphContext(*GraphInstance.Get(), 0.0333f);
				UpdateGraphContext.PushInputEvent(EventA);
				UpdateGraphContext.PushInputEvent(EventB);

				UpdateGraph(UpdateGraphContext);

				UE_RETURN_ON_ERROR(EventA->VisitedTraits.Num() == 4, "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected number of traits visited");
				AddErrorIfFalse(EventA->VisitedTraits[0] == NodeTemplateTraitListB[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");
				AddErrorIfFalse(EventA->VisitedTraits[1] == NodeTemplateTraitListA[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");
				AddErrorIfFalse(EventA->VisitedTraits[2] == NodeTemplateTraitListC[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");
				AddErrorIfFalse(EventA->VisitedTraits[3] == NodeTemplateTraitListC[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");

				UE_RETURN_ON_ERROR(EventB->VisitedTraits.Num() == 4, "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected number of traits visited");
				AddErrorIfFalse(EventB->VisitedTraits[0] == NodeTemplateTraitListB[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");
				AddErrorIfFalse(EventB->VisitedTraits[1] == NodeTemplateTraitListA[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");
				AddErrorIfFalse(EventB->VisitedTraits[2] == NodeTemplateTraitListC[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");
				AddErrorIfFalse(EventB->VisitedTraits[3] == NodeTemplateTraitListC[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");
			}

			// Raise EventA on graph, NodeA consumes it, only Root and NodeA see it
			{
				auto EventA = MakeTraitEvent<FTraitAnimGraphTest_EventA>();
				EventA->bTestFlag = true;

				FUpdateGraphContext UpdateGraphContext((*(GraphInstance.Get())), 0.0333f);
				UpdateGraphContext.PushInputEvent(EventA);

				UpdateGraph(UpdateGraphContext);

				UE_RETURN_ON_ERROR(EventA->VisitedTraits.Num() == 2, "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected number of traits visited");
				AddErrorIfFalse(EventA->VisitedTraits[0] == NodeTemplateTraitListB[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");
				AddErrorIfFalse(EventA->VisitedTraits[1] == NodeTemplateTraitListA[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");
			}

			// Raise EventA and EventB on graph (in list), NodeA consumes EventA, only Root and NodeA see it, every node sees EventB
			{
				auto EventA = MakeTraitEvent<FTraitAnimGraphTest_EventA>();
				EventA->bTestFlag = true;
				auto EventB = MakeTraitEvent<FTraitAnimGraphTest_EventB>();

				FUpdateGraphContext UpdateGraphContext(*GraphInstance.Get(), 0.0333f);
				UpdateGraphContext.PushInputEvent(EventA);
				UpdateGraphContext.PushInputEvent(EventB);

				UpdateGraph(UpdateGraphContext);

				UE_RETURN_ON_ERROR(EventA->VisitedTraits.Num() == 2, "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected number of traits visited");
				AddErrorIfFalse(EventA->VisitedTraits[0] == NodeTemplateTraitListB[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");
				AddErrorIfFalse(EventA->VisitedTraits[1] == NodeTemplateTraitListA[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");

				UE_RETURN_ON_ERROR(EventB->VisitedTraits.Num() == 4, "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected number of traits visited");
				AddErrorIfFalse(EventB->VisitedTraits[0] == NodeTemplateTraitListB[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");
				AddErrorIfFalse(EventB->VisitedTraits[1] == NodeTemplateTraitListA[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");
				AddErrorIfFalse(EventB->VisitedTraits[2] == NodeTemplateTraitListC[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");
				AddErrorIfFalse(EventB->VisitedTraits[3] == NodeTemplateTraitListC[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");
			}

			// NodeA raises input EventA, only visible to NodeB
			{
				auto EventB = MakeTraitEvent<FTraitAnimGraphTest_EventB>();
				EventB->bTestFlag0 = true;

				FUpdateGraphContext UpdateGraphContext(*GraphInstance.Get(), 0.0333f);
				UpdateGraphContext.PushInputEvent(EventB);

				UpdateGraph(UpdateGraphContext);

				UE_RETURN_ON_ERROR(EventB->VisitedTraits.Num() == 4, "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected number of traits visited");
				AddErrorIfFalse(EventB->VisitedTraits[0] == NodeTemplateTraitListB[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");
				AddErrorIfFalse(EventB->VisitedTraits[1] == NodeTemplateTraitListA[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");
				AddErrorIfFalse(EventB->VisitedTraits[2] == NodeTemplateTraitListC[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");
				AddErrorIfFalse(EventB->VisitedTraits[3] == NodeTemplateTraitListC[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");

				UE_RETURN_ON_ERROR(EventB->ChildEvent.IsValid(), "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Expected child event");
				auto EventA = EventB->ChildEvent->AsType<FTraitAnimGraphTest_EventA>();
				UE_RETURN_ON_ERROR(EventA != nullptr, "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Expected child event of correct type");

				UE_RETURN_ON_ERROR(EventA->VisitedTraits.Num() == 1, "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected number of traits visited");
				AddErrorIfFalse(EventA->VisitedTraits[0] == NodeTemplateTraitListC[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");
			}

			// NodeA raises output event, only visible to Root
			{
				auto EventB = MakeTraitEvent<FTraitAnimGraphTest_EventB>();
				EventB->bTestFlag1 = true;

				FUpdateGraphContext UpdateGraphContext(*GraphInstance.Get(), 0.0333f);
				UpdateGraphContext.PushInputEvent(EventB);

				UpdateGraph(UpdateGraphContext);

				UE_RETURN_ON_ERROR(EventB->VisitedTraits.Num() == 4, "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected number of traits visited");
				AddErrorIfFalse(EventB->VisitedTraits[0] == NodeTemplateTraitListB[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");
				AddErrorIfFalse(EventB->VisitedTraits[1] == NodeTemplateTraitListA[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");
				AddErrorIfFalse(EventB->VisitedTraits[2] == NodeTemplateTraitListC[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");
				AddErrorIfFalse(EventB->VisitedTraits[3] == NodeTemplateTraitListC[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");

				UE_RETURN_ON_ERROR(EventB->ChildEvent.IsValid(), "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Expected child event");
				auto EventA = EventB->ChildEvent->AsType<FTraitAnimGraphTest_EventA>();
				UE_RETURN_ON_ERROR(EventA != nullptr, "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Expected child event of correct type");

				UE_RETURN_ON_ERROR(EventA->VisitedTraits.Num() == 1, "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected number of traits visited");
				AddErrorIfFalse(EventA->VisitedTraits[0] == NodeTemplateTraitListB[0], "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Unexpected trait visited");
			}
		}
	}

	Tests::FUtils::CleanupAfterTests();

	return true;
}

#endif
