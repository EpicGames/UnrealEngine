// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/TraitUID.h"
#include "TraitCore/TraitEvent.h"

struct FTraitAnimGraphTest_EventA : public FAnimNextTraitEvent
{
	DECLARE_ANIM_TRAIT_EVENT(FTraitAnimGraphTest_EventA, FAnimNextTraitEvent)

	bool bTestFlag = false;

	TArray<UE::UAF::FTraitUID> VisitedTraits;
};

struct FTraitAnimGraphTest_EventB : public FAnimNextTraitEvent
{
	DECLARE_ANIM_TRAIT_EVENT(FTraitAnimGraphTest_EventB, FAnimNextTraitEvent)

	bool bTestFlag0 = false;
	bool bTestFlag1 = false;

	TArray<UE::UAF::FTraitUID> VisitedTraits;

	FAnimNextTraitEventPtr ChildEvent;
};
