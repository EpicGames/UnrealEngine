// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextAnimGraphTraitGraphTest.h"

#include "AnimNextRuntimeTest.h"
#include "AnimNextTest.h"
#include "AssetToolsModule.h"
#include "IAnimNextRigVMExportInterface.h"
#include "UncookedOnlyUtils.h"
#include "Animation/AnimSequence.h"
#include "TraitCore/TraitRegistry.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IUpdate.h"
#include "Editor/Transactor.h"
#include "Entries/AnimNextVariableEntry.h"
#include "Graph/AnimNextAnimationGraph_EditorData.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextAnimationGraphFactory.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Graph/RigDecorator_AnimNextCppTrait.h"
#include "Graph/RigUnit_AnimNextGraphRoot.h"
#include "Graph/RigUnit_AnimNextTraitStack.h"
#include "Misc/AutomationTest.h"
#include "RigVMFunctions/Math/RigVMFunction_MathInt.h"
#include "RigVMFunctions/Math/RigVMFunction_MathFloat.h"
#include "AnimNextTraitStackUnitNode.h"
#include "AnimGraphUncookedOnlyUtils.h"
#include "Script/UAFRigVMComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextAnimGraphTraitGraphTest)

#if WITH_DEV_AUTOMATION_TESTS

//****************************************************************************
// AnimNext Runtime Trait Graph Tests
//****************************************************************************

namespace UE::UAF
{
	struct FTestTrait final : FBaseTrait, IEvaluate, IUpdate
	{
		DECLARE_ANIM_TRAIT(FTestTrait, FBaseTrait)

		using FSharedData = FTestTraitSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			int32 UpdateCount = 0;
			int32 EvaluateCount = 0;
		};
		
		// IUpdate impl
		virtual void PostUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override
		{
			IUpdate::PostUpdate(Context, Binding, TraitState);

			const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
			FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

			FRigVMExecuteContext& ExecuteContext = Context.GetRootGraphInstance().GetOrAddComponent<FUAFRigVMComponent>().GetExtendedExecuteContext().GetPublicData();
			ExecuteContext.Logf(EMessageSeverity::Info, TEXT("UpdateCount == %d"), ++InstanceData->UpdateCount);
			ExecuteContext.Logf(EMessageSeverity::Info, TEXT("SomeInt32 == %d"), SharedData->SomeInt32);
			ExecuteContext.Logf(EMessageSeverity::Info, TEXT("SomeFloat == %.02f"), SharedData->SomeFloat);
			ExecuteContext.Logf(EMessageSeverity::Info, TEXT("SomeLatentInt32 == %d"), SharedData->GetSomeLatentInt32(Binding));
			ExecuteContext.Logf(EMessageSeverity::Info, TEXT("SomeOtherLatentInt32 == %d"), SharedData->GetSomeOtherLatentInt32(Binding));
			ExecuteContext.Logf(EMessageSeverity::Info, TEXT("SomeOutOfOrderLatentBool == %d"), (int32)SharedData->GetSomeOutOfOrderLatentBool(Binding));
			const FVector& LatentVector = SharedData->GetSomeLatentVector(Binding);
			ExecuteContext.Logf(EMessageSeverity::Info, TEXT("SomeLatentVector == [%.02f, %.02f, %.02f]"), LatentVector.X, LatentVector.Y, LatentVector.Z);
			ExecuteContext.Logf(EMessageSeverity::Info, TEXT("SomeLatentFloat == %.02f"), SharedData->GetSomeLatentFloat(Binding));
		}

		// IEvaluate impl
		virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override
		{
			IEvaluate::PostEvaluate(Context, Binding);

			FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
			FRigVMExecuteContext& ExecuteContext = Context.GetRootGraphInstance().GetOrAddComponent<FUAFRigVMComponent>().GetExtendedExecuteContext().GetPublicData();
			ExecuteContext.Logf(EMessageSeverity::Info, TEXT("EvaluateCount == %d"), ++InstanceData->EvaluateCount);
		}
	};

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IEvaluate) \
		GeneratorMacro(IUpdate) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FTestTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	// --- FTestBasicTrait ---

	struct FTestBasicTrait final : FBaseTrait
	{
		DECLARE_ANIM_TRAIT(FTestBasicTrait, FBaseTrait)

		using FSharedData = FTestTraitSharedData;
	};

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FTestBasicTrait, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnimationAnimNextEditorTest_GraphAddTrait, "Animation.AnimNext.Editor.Graph.AddTrait", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAnimationAnimNextEditorTest_GraphAddTrait::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	{
		AUTO_REGISTER_ANIM_TRAIT(FTestTrait)

		FScopedClearNodeTemplateRegistry ScopedClearNodeTemplateRegistry;

		UFactory* GraphFactory = NewObject<UUAFAnimGraphFactory>();
		UUAFAnimGraph* AnimationGraph = CastChecked<UUAFAnimGraph>(GraphFactory->FactoryCreateNew(UUAFAnimGraph::StaticClass(), GetTransientPackage(), TEXT("TestAnimNextGraph"), RF_Transient, nullptr, nullptr, NAME_None));
		UE_RETURN_ON_ERROR(AnimationGraph != nullptr, "FAnimationAnimNextEditorTest_GraphAddTrait -> Failed to create animation graph");

		UUAFAnimGraph_EditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFAnimGraph_EditorData>(AnimationGraph);
		UE_RETURN_ON_ERROR(EditorData != nullptr, "FAnimationAnimNextEditorTest_GraphAddTrait -> Failed to find module editor data");

		URigVMController* Controller = EditorData->GetRigVMClient()->GetController(EditorData->GetRigVMClient()->GetDefaultModel());
		UE_RETURN_ON_ERROR(Controller != nullptr, "FAnimationAnimNextEditorTest_GraphAddTrait -> Failed to get RigVM controller");

		// Create an empty trait stack node
		URigVMUnitNode* TraitStackNode = Controller->AddUnitNode(FRigUnit_AnimNextTraitStack::StaticStruct(), FRigVMStruct::ExecuteName, FVector2D(0.0f, 0.0f), FString(), false);
		UE_RETURN_ON_ERROR(TraitStackNode != nullptr, "FAnimationAnimNextRuntimeTest_GraphAddTrait -> Failed to create trait stack node");

		// Add a trait
		const UScriptStruct* CppTraitStruct = FRigDecorator_AnimNextCppDecorator::StaticStruct();
		UE_RETURN_ON_ERROR(CppTraitStruct != nullptr, "FAnimationAnimNextRuntimeTest_GraphAddDecorator -> Failed to get find Cpp trait static struct");

		const FTrait* Trait = FTraitRegistry::Get().Find(FTestTrait::TraitUID);
		UE_RETURN_ON_ERROR(Trait != nullptr, "FAnimationAnimNextEditorTest_GraphAddTrait -> Failed to find test trait");

		UScriptStruct* ScriptStruct = Trait->GetTraitSharedDataStruct();
		UE_RETURN_ON_ERROR(ScriptStruct != nullptr, "FAnimationAnimNextEditorTest_GraphAddTrait -> Failed to find trait shared data struct");

		FString DefaultValue;
		{
			const FRigDecorator_AnimNextCppDecorator DefaultCppDecoratorStructInstance;
			FRigDecorator_AnimNextCppDecorator CppDecoratorStructInstance;
			CppDecoratorStructInstance.DecoratorSharedDataStruct = ScriptStruct;

			UE_RETURN_ON_ERROR(CppDecoratorStructInstance.CanBeAddedToNode(TraitStackNode, nullptr), "FAnimationAnimNextEditorTest_GraphAddTrait -> Trait cannot be added to trait stack node");

			FRigDecorator_AnimNextCppDecorator::StaticStruct()->ExportText(DefaultValue, &CppDecoratorStructInstance, &DefaultCppDecoratorStructInstance, nullptr, PPF_None, nullptr);
		}

		FString DisplayNameMetadata;
		ScriptStruct->GetStringMetaDataHierarchical(FRigVMStruct::DisplayNameMetaName, &DisplayNameMetadata);
		const FString DisplayName = DisplayNameMetadata.IsEmpty() ? Trait->GetTraitName() : DisplayNameMetadata;

		const FName TraitName = Controller->AddTrait(
			TraitStackNode->GetFName(),
			*CppTraitStruct->GetPathName(),
			*DisplayName,
			DefaultValue, INDEX_NONE, true, true);
		UE_RETURN_ON_ERROR(TraitName == DisplayName, TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait name"));

		URigVMPin* TraitPin = TraitStackNode->FindPin(*DisplayName);
		UE_RETURN_ON_ERROR(TraitPin != nullptr, TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Failed to find trait pin"));

		// Our first pin is the hard coded output result, trait pins follow
		UE_RETURN_ON_ERROR(TraitStackNode->GetPins().Num() == 2, TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected number of pins"));
		UE_RETURN_ON_ERROR(TraitPin->IsTraitPin() == true, TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected pin type"));
		UE_RETURN_ON_ERROR(TraitPin->GetFName() == TraitName, TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected pin name"));
		UE_RETURN_ON_ERROR(TraitPin->GetCPPTypeObject() == FRigDecorator_AnimNextCppDecorator::StaticStruct(), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected pin type"));

		// Our first sub-pin is the hard coded script struct member that parametrizes the trait, dynamic trait sub-pins follow
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins().Num() == 10, TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait sub pins"));

		// UpdateCount
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[1]->GetCPPType() == TEXT("int32"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin type"));
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[1]->GetDefaultValue() == TEXT("0"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin value"));
		UE_RETURN_ON_ERROR(!TraitPin->GetSubPins()[1]->IsLazy(), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Expected non-lazy trait pin"));

		// EvaluateCount
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[2]->GetCPPType() == TEXT("int32"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin type"));
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[2]->GetDefaultValue() == TEXT("0"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin value"));
		UE_RETURN_ON_ERROR(!TraitPin->GetSubPins()[2]->IsLazy(), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Expected non-lazy trait pin"));

		// SomeInt32
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[3]->GetCPPType() == TEXT("int32"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin type"));
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[3]->GetDefaultValue() == TEXT("3"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin value"));
		UE_RETURN_ON_ERROR(!TraitPin->GetSubPins()[3]->IsLazy(), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Expected non-lazy trait pin"));

		// SomeFloat
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[4]->GetCPPType() == TEXT("float"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin type"));
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[4]->GetDefaultValue() == TEXT("34.000000"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin value"));
		UE_RETURN_ON_ERROR(!TraitPin->GetSubPins()[4]->IsLazy(), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Expected non-lazy trait pin"));

		// SomeLatentInt32
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[5]->GetCPPType() == TEXT("int32"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin type"));
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[5]->GetDefaultValue() == TEXT("5"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin value"));
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[5]->IsLazy(), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Expected lazy trait pin"));

		// SomeOtherLatentInt32
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[6]->GetCPPType() == TEXT("int32"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin type"));
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[6]->GetDefaultValue() == TEXT("7"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin value"));
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[6]->IsLazy(), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Expected lazy trait pin"));

		// SomeOutOfOrderLatentBool
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[7]->GetCPPType() == TEXT("bool"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin type"));
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[7]->GetDefaultValue() == TEXT("False"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin value"));
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[7]->IsLazy(), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Expected lazy trait pin"));

		// SomeLatentVector
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[8]->GetCPPType() == TEXT("FVector"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin type"));
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[8]->GetDefaultValue() == TEXT("(X=1.000000,Y=1.000000,Z=1.000000)"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin value"));
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[8]->IsLazy(), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Expected lazy trait pin"));

		// SomeLatentFloat
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[9]->GetCPPType() == TEXT("float"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin type"));
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[9]->GetDefaultValue() == TEXT("34.000000"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin value"));
		UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[9]->IsLazy(), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Expected lazy trait pin"));
	}

	Tests::FUtils::CleanupAfterTests();

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnimationAnimNextEditorTest_GraphTraitOperations, "Animation.AnimNext.Editor.Graph.TraitOperations", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAnimationAnimNextEditorTest_GraphTraitOperations::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	{
		AUTO_REGISTER_ANIM_TRAIT(FTestTrait)
		AUTO_REGISTER_ANIM_TRAIT(FTestBasicTrait)

		FScopedClearNodeTemplateRegistry ScopedClearNodeTemplateRegistry;

		UFactory* GraphFactory = NewObject<UUAFAnimGraphFactory>();
		UUAFAnimGraph* AnimationGraph = CastChecked<UUAFAnimGraph>(GraphFactory->FactoryCreateNew(UUAFAnimGraph::StaticClass(), GetTransientPackage(), TEXT("TestAnimNextGraph"), RF_Transient, nullptr, nullptr, NAME_None));
		UE_RETURN_ON_ERROR(AnimationGraph != nullptr, "FAnimationAnimNextEditorTest_GraphTraitOperations -> Failed to create animation graph");

		UUAFAnimGraph_EditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFAnimGraph_EditorData>(AnimationGraph);
		UE_RETURN_ON_ERROR(EditorData != nullptr, "FAnimationAnimNextEditorTest_GraphTraitOperations -> Failed to find module editor data");

		UAnimNextController* Controller = Cast<UAnimNextController>(EditorData->GetRigVMClient()->GetController(EditorData->GetRigVMClient()->GetDefaultModel()));
		UE_RETURN_ON_ERROR(Controller != nullptr, "FAnimationAnimNextEditorTest_GraphTraitOperations -> Failed to get RigVM controller");

		// Create an empty trait stack node
		URigVMUnitNode* TraitStackNode = Controller->AddUnitNode(FRigUnit_AnimNextTraitStack::StaticStruct(), FRigVMStruct::ExecuteName, FVector2D(0.0f, 0.0f), FString(), false);
		UE_RETURN_ON_ERROR(TraitStackNode != nullptr, "FAnimationAnimNextEditorTest_GraphTraitOperations -> Failed to create trait stack node");

		const TArray<URigVMPin*>& NodePins = TraitStackNode->GetPins();

		FName TraitInstanceName = NAME_None;

		// --- Add a trait ---
		{
			const FTrait* Trait = FTraitRegistry::Get().Find(FTestTrait::TraitUID);
			UE_RETURN_ON_ERROR(Trait != nullptr, "FAnimationAnimNextEditorTest_GraphTraitOperations -> Failed to find test trait");

			const FName TraitTypeName = *Trait->GetTraitName();
			const FName TraitSharedDataStructName = Trait->GetTraitSharedDataStruct()->GetFName();
			
			TraitInstanceName = Controller->AddTraitByName(TraitStackNode->GetFName(), TraitTypeName, INDEX_NONE);
			UE_RETURN_ON_ERROR(TraitInstanceName == TraitSharedDataStructName, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected Trait name"));

			URigVMPin* TraitPin = TraitStackNode->FindPin(TraitInstanceName.ToString());
			UE_RETURN_ON_ERROR(TraitPin != nullptr, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Failed to find Trait pin"));

			// Our first pin is the hard coded output result, trait pins follow
			UE_RETURN_ON_ERROR(NodePins.Num() == 2, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected number of pins"));
			UE_RETURN_ON_ERROR(TraitPin->IsTraitPin() == true, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected pin type"));
			UE_RETURN_ON_ERROR(TraitPin->GetFName() == TraitInstanceName, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected pin name"));
			UE_RETURN_ON_ERROR(TraitPin->GetCPPTypeObject() == FRigDecorator_AnimNextCppDecorator::StaticStruct(), TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected pin type"));

			// Our first sub-pin is the hard coded script struct member that parametrizes the trait, dynamic trait sub-pins follow
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins().Num() == 10, TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait sub pins"));

			// UpdateCount
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[1]->GetCPPType() == TEXT("int32"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin type"));
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[1]->GetDefaultValue() == TEXT("0"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin value"));
			UE_RETURN_ON_ERROR(!TraitPin->GetSubPins()[1]->IsLazy(), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Expected non-lazy trait pin"));

			// EvaluateCount
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[2]->GetCPPType() == TEXT("int32"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin type"));
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[2]->GetDefaultValue() == TEXT("0"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin value"));
			UE_RETURN_ON_ERROR(!TraitPin->GetSubPins()[2]->IsLazy(), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Expected non-lazy trait pin"));

			// SomeInt32
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[3]->GetCPPType() == TEXT("int32"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin type"));
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[3]->GetDefaultValue() == TEXT("3"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin value"));
			UE_RETURN_ON_ERROR(!TraitPin->GetSubPins()[3]->IsLazy(), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Expected non-lazy trait pin"));

			// SomeFloat
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[4]->GetCPPType() == TEXT("float"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin type"));
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[4]->GetDefaultValue() == TEXT("34.000000"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin value"));
			UE_RETURN_ON_ERROR(!TraitPin->GetSubPins()[4]->IsLazy(), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Expected non-lazy trait pin"));

			// SomeLatentInt32
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[5]->GetCPPType() == TEXT("int32"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin type"));
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[5]->GetDefaultValue() == TEXT("5"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin value"));
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[5]->IsLazy(), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Expected lazy trait pin"));

			// SomeOtherLatentInt32
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[6]->GetCPPType() == TEXT("int32"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin type"));
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[6]->GetDefaultValue() == TEXT("7"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin value"));
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[6]->IsLazy(), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Expected lazy trait pin"));

			// SomeOutOfOrderLatentBool
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[7]->GetCPPType() == TEXT("bool"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin type"));
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[7]->GetDefaultValue() == TEXT("False"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin value"));
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[7]->IsLazy(), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Expected lazy trait pin"));

			// SomeLatentVector
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[8]->GetCPPType() == TEXT("FVector"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin type"));
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[8]->GetDefaultValue() == TEXT("(X=1.000000,Y=1.000000,Z=1.000000)"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin value"));
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[8]->IsLazy(), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Expected lazy trait pin"));

			// SomeLatentFloat
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[9]->GetCPPType() == TEXT("float"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin type"));
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[9]->GetDefaultValue() == TEXT("34.000000"), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Unexpected trait pin value"));
			UE_RETURN_ON_ERROR(TraitPin->GetSubPins()[9]->IsLazy(), TEXT("FAnimationAnimNextEditorTest_GraphAddTrait -> Expected lazy trait pin"));
		}

		// --- Undo Add Trait ---
		{
			Controller->Undo();

			URigVMPin* TraitPin = TraitStackNode->FindPin(TraitInstanceName.ToString());
			UE_RETURN_ON_ERROR(TraitPin == nullptr, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Undo AddTrait failed, Trait pin is still present"));

			// Our first pin is the hard coded output result, trait pins follow
			const URigVMPin* FirstPin = NodePins[0];
			UE_RETURN_ON_ERROR(NodePins.Num() == 1, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected number of pins"));
			UE_RETURN_ON_ERROR(FirstPin->IsTraitPin() == false, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected pin type"));
		}

		// --- Redo Add Trait ---
		{
			Controller->Redo();

			URigVMPin* TraitPin = TraitStackNode->FindPin(TraitInstanceName.ToString());
			UE_RETURN_ON_ERROR(TraitPin != nullptr, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Redo AddTrait failed, can not find Trait pin"));

			// Our first pin is the hard coded output result, trait pins follow
			UE_RETURN_ON_ERROR(NodePins.Num() == 2, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected number of pins"));
			UE_RETURN_ON_ERROR(TraitPin->IsTraitPin() == true, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected pin type"));
			UE_RETURN_ON_ERROR(TraitPin->GetFName() == TraitInstanceName, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected pin name"));
			UE_RETURN_ON_ERROR(TraitPin->GetCPPTypeObject() == FRigDecorator_AnimNextCppDecorator::StaticStruct(), TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected pin type"));
		}

		// --- Remove the created trait ---
		{
			Controller->RemoveTraitByName(TraitStackNode->GetFName(), TraitInstanceName);

			// Our first pin is the hard coded output result, trait pins follow
			UE_RETURN_ON_ERROR(NodePins.Num() == 1, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected number of pins"));

			URigVMPin* DeletedTraitPin = TraitStackNode->FindPin(TraitInstanceName.ToString());
			UE_RETURN_ON_ERROR(DeletedTraitPin == nullptr, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Failed to remove Trait pin"));

			// Only the output result pin should be in the pin array
			const URigVMPin* FirstPin = NodePins[0];
			UE_RETURN_ON_ERROR(FirstPin->IsTraitPin() == false, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected pin type"));
			UE_RETURN_ON_ERROR(FirstPin->GetFName() != TraitInstanceName, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected pin name"));
		}

		// --- Undo Remove Trait ---
		{
			Controller->Undo();

			URigVMPin* TraitPin = TraitStackNode->FindPin(TraitInstanceName.ToString());
			UE_RETURN_ON_ERROR(TraitPin != nullptr, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Undo failed, unable to find Trait pin"));

			// Our first pin is the hard coded output result, trait pins follow
			UE_RETURN_ON_ERROR(NodePins.Num() == 2, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected number of pins"));
			UE_RETURN_ON_ERROR(TraitPin->IsTraitPin() == true, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected pin type"));
			UE_RETURN_ON_ERROR(TraitPin->GetFName() == TraitInstanceName, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected pin name"));
			UE_RETURN_ON_ERROR(TraitPin->GetCPPTypeObject() == FRigDecorator_AnimNextCppDecorator::StaticStruct(), TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected pin type"));
		}

		// --- Swap the FTestTrait with FTestBasicTrait ---
		{
			const FTrait* BasicTrait = FTraitRegistry::Get().Find(FTestBasicTrait::TraitUID);
			UE_RETURN_ON_ERROR(BasicTrait != nullptr, "FAnimationAnimNextEditorTest_GraphTraitOperations -> Failed to find test basic trait");

			const FName BasicTraitTypeName = *BasicTrait->GetTraitName();
			const FName BasicTraitSharedDataStructName = BasicTrait->GetTraitSharedDataStruct()->GetFName();

			TraitInstanceName = Controller->SwapTraitByName(TraitStackNode->GetFName(), TraitInstanceName, 1, BasicTraitTypeName);
			UE_RETURN_ON_ERROR(TraitInstanceName == BasicTraitSharedDataStructName, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected Trait name"));

			URigVMPin* TraitPin = TraitStackNode->FindPin(TraitInstanceName.ToString());
			UE_RETURN_ON_ERROR(TraitPin != nullptr, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Failed to find FTestBasicTrait pin"));

			// Our first pin is the hard coded output result, trait pins follow
			UE_RETURN_ON_ERROR(NodePins.Num() == 2, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected number of pins"));
			UE_RETURN_ON_ERROR(TraitPin->IsTraitPin() == true, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected pin type"));
			UE_RETURN_ON_ERROR(TraitPin->GetFName() == TraitInstanceName, TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected pin name"));
			UE_RETURN_ON_ERROR(TraitPin->GetCPPTypeObject() == FRigDecorator_AnimNextCppDecorator::StaticStruct(), TEXT("FAnimationAnimNextEditorTest_GraphTraitOperations -> Unexpected pin type"));
		}
	}

	Tests::FUtils::CleanupAfterTests();

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnimationAnimNextRuntimeTest_GraphExecute, "Animation.AnimNext.Runtime.Graph.Execute", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAnimationAnimNextRuntimeTest_GraphExecute::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	{
		AUTO_REGISTER_ANIM_TRAIT(FTestTrait)

		FScopedClearNodeTemplateRegistry ScopedClearNodeTemplateRegistry;

		UFactory* GraphFactory = NewObject<UUAFAnimGraphFactory>();
		UUAFAnimGraph* AnimationGraph = CastChecked<UUAFAnimGraph>(GraphFactory->FactoryCreateNew(UUAFAnimGraph::StaticClass(), GetTransientPackage(), TEXT("TestAnimNextGraph"), RF_Transient, nullptr, nullptr, NAME_None));
		UE_RETURN_ON_ERROR(AnimationGraph != nullptr, "FAnimationAnimNextEditorTest_GraphTraitOperations -> Failed to create animation graph");

		UUAFAnimGraph_EditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFAnimGraph_EditorData>(AnimationGraph);
		UE_RETURN_ON_ERROR(EditorData != nullptr, "FAnimationAnimNextRuntimeTest_GraphExecute -> Failed to find module editor data");

		URigVMController* Controller = EditorData->GetRigVMClient()->GetController(EditorData->GetRigVMClient()->GetDefaultModel());
		UE_RETURN_ON_ERROR(Controller != nullptr, "FAnimationAnimNextRuntimeTest_GraphExecute -> Failed to get RigVM controller");

		// Find graph entry point
		URigVMNode* MainEntryPointNode = Controller->GetGraph()->FindNodeByName(FRigUnit_AnimNextGraphRoot::StaticStruct()->GetFName());
		UE_RETURN_ON_ERROR(MainEntryPointNode != nullptr, "FAnimationAnimNextRuntimeTest_GraphExecute -> Failed to find main entry point node");

		URigVMPin* BeginExecutePin = MainEntryPointNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AnimNextGraphRoot, Result));
		UE_RETURN_ON_ERROR(BeginExecutePin != nullptr && BeginExecutePin->GetDirection() == ERigVMPinDirection::Input, "FAnimationAnimNextRuntimeTest_GraphExecute -> Failed to create entry point");

		URigVMUnitNode* DecoratorStackNode = nullptr;
		FString DisplayName;
		{
			// Suspend auto compilation until we have constructed a valid trait stack
			TGuardValue<bool> SuspendCompile(EditorData->bAutoRecompileVM, false);

			// Create an empty trait stack node
			DecoratorStackNode = Controller->AddUnitNode(FRigUnit_AnimNextTraitStack::StaticStruct(), FRigVMStruct::ExecuteName, FVector2D(0.0f, 0.0f), FString(), false);
			UE_RETURN_ON_ERROR(DecoratorStackNode != nullptr, "FAnimationAnimNextRuntimeTest_GraphExecute -> Failed to create trait stack node");

			// Link our stack result to our entry point
			Controller->AddLink(DecoratorStackNode->GetPins()[0], MainEntryPointNode->GetPins()[0]);

			// Add a trait
			const UScriptStruct* CppDecoratorStruct = FRigDecorator_AnimNextCppDecorator::StaticStruct();
			UE_RETURN_ON_ERROR(CppDecoratorStruct != nullptr, "FAnimationAnimNextRuntimeTest_GraphExecute -> Failed to get find Cpp trait static struct");

			const FTrait* Trait = FTraitRegistry::Get().Find(FTestTrait::TraitUID);
			UE_RETURN_ON_ERROR(Trait != nullptr, "FAnimationAnimNextRuntimeTest_GraphExecute -> Failed to find test trait");

			UScriptStruct* ScriptStruct = Trait->GetTraitSharedDataStruct();
			UE_RETURN_ON_ERROR(ScriptStruct != nullptr, "FAnimationAnimNextRuntimeTest_GraphExecute -> Failed to find trait shared data struct");

			FString DefaultValue;
			{
				const FRigDecorator_AnimNextCppDecorator DefaultCppDecoratorStructInstance;
				FRigDecorator_AnimNextCppDecorator CppDecoratorStructInstance;
				CppDecoratorStructInstance.DecoratorSharedDataStruct = ScriptStruct;

				UE_RETURN_ON_ERROR(CppDecoratorStructInstance.CanBeAddedToNode(DecoratorStackNode, nullptr), "FAnimationAnimNextRuntimeTest_GraphExecute -> Trait cannot be added to trait stack node");

				FRigDecorator_AnimNextCppDecorator::StaticStruct()->ExportText(DefaultValue, &CppDecoratorStructInstance, &DefaultCppDecoratorStructInstance, nullptr, PPF_None, nullptr);
			}

			FString DisplayNameMetadata;
			ScriptStruct->GetStringMetaDataHierarchical(FRigVMStruct::DisplayNameMetaName, &DisplayNameMetadata);
			DisplayName = DisplayNameMetadata.IsEmpty() ? Trait->GetTraitName() : DisplayNameMetadata;

			const FName DecoratorName = Controller->AddTrait(
				DecoratorStackNode->GetFName(),
				*CppDecoratorStruct->GetPathName(),
				*DisplayName,
				DefaultValue, INDEX_NONE, true, true);
			UE_RETURN_ON_ERROR(DecoratorName == DisplayName, TEXT("FAnimationAnimNextRuntimeTest_GraphExecute -> Unexpected trait name"));
		}

		URigVMPin* DecoratorPin = DecoratorStackNode->FindPin(*DisplayName);
		UE_RETURN_ON_ERROR(DecoratorPin != nullptr, TEXT("FAnimationAnimNextRuntimeTest_GraphExecute -> Failed to find trait pin"));

		// Set some values on our trait
		Controller->SetPinDefaultValue(DecoratorPin->GetSubPins()[3]->GetPinPath(), TEXT("78"));
		Controller->SetPinDefaultValue(DecoratorPin->GetSubPins()[4]->GetPinPath(), TEXT("142.33"));

		TSharedPtr<FAnimNextGraphInstance> GraphInstance = AnimationGraph->AllocateInstance();

		TArray<FString> Messages;
		FRigVMRuntimeSettings RuntimeSettings;
		RuntimeSettings.SetLogFunction([&Messages](const FRigVMLogSettings& InLogSettings, const FRigVMExecuteContext* InContext, const FString& Message)
		{
			Messages.Add(Message);
		});
		GraphInstance->GetOrAddComponent<FUAFRigVMComponent>().GetExtendedExecuteContext().SetRuntimeSettings(RuntimeSettings);

		{
			FUpdateGraphContext UpdateGraphContext(*GraphInstance.Get(), 1.0f / 30.0f);
			UE::UAF::UpdateGraph(UpdateGraphContext);
			const UE::UAF::FEvaluateGraphContext EvaluateGraphContext(*GraphInstance.Get(), UE::UAF::FReferencePose(), 0);
			(void)UE::UAF::EvaluateGraph(EvaluateGraphContext);
		}

		UE_RETURN_ON_ERROR(Messages.Num() == 9, "FAnimationAnimNextRuntimeTest_GraphExecute - > Unexpected message count");
		AddErrorIfFalse(Messages[0] == "UpdateCount == 1", "FAnimationAnimNextRuntimeTest_GraphExecute -> Unexpected update count");
		AddErrorIfFalse(Messages[1] == "SomeInt32 == 78", "FAnimationAnimNextRuntimeTest_GraphExecute -> Unexpected SomeInt32 value");
		AddErrorIfFalse(Messages[2] == "SomeFloat == 142.33", "FAnimationAnimNextRuntimeTest_GraphExecute -> Unexpected SomeFloat value");
		AddErrorIfFalse(Messages[3] == "SomeLatentInt32 == 5", "FAnimationAnimNextRuntimeTest_GraphExecute -> Unexpected SomeLatentInt32 value");
		AddErrorIfFalse(Messages[4] == "SomeOtherLatentInt32 == 7", "FAnimationAnimNextRuntimeTest_GraphExecute -> Unexpected SomeOtherLatentInt32 value");
		AddErrorIfFalse(Messages[5] == "SomeOutOfOrderLatentBool == 0", "FAnimationAnimNextRuntimeTest_GraphExecute -> Unexpected SomeOutOfOrderLatentBool value");
		AddErrorIfFalse(Messages[6] == "SomeLatentVector == [1.00, 1.00, 1.00]", "FAnimationAnimNextRuntimeTest_GraphExecute -> Unexpected SomeLatentVector value");
		AddErrorIfFalse(Messages[7] == "SomeLatentFloat == 34.00", "FAnimationAnimNextRuntimeTest_GraphExecute -> Unexpected SomeLatentFloat value");
		AddErrorIfFalse(Messages[8] == "EvaluateCount == 1", "FAnimationAnimNextRuntimeTest_GraphExecute -> Unexpected evaluate count");
	}

	Tests::FUtils::CleanupAfterTests();

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnimationAnimNextRuntimeTest_GraphExecuteLatent, "Animation.AnimNext.Runtime.Graph.ExecuteLatent", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAnimationAnimNextRuntimeTest_GraphExecuteLatent::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	{
		AUTO_REGISTER_ANIM_TRAIT(FTestTrait)
		FScopedClearNodeTemplateRegistry ScopedClearNodeTemplateRegistry;

		UFactory* GraphFactory = NewObject<UUAFAnimGraphFactory>();
		UUAFAnimGraph* AnimationGraph = CastChecked<UUAFAnimGraph>(GraphFactory->FactoryCreateNew(UUAFAnimGraph::StaticClass(), GetTransientPackage(), TEXT("TestAnimNextGraph"), RF_Transient, nullptr, nullptr, NAME_None));
		UE_RETURN_ON_ERROR(AnimationGraph != nullptr, "FAnimationAnimNextEditorTest_GraphTraitOperations -> Failed to create animation graph");

		UUAFAnimGraph_EditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFAnimGraph_EditorData>(AnimationGraph);
		UE_RETURN_ON_ERROR(EditorData != nullptr, "FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Failed to find module editor data");

		EditorData->AddVariable(TEXT("TestIntVar"), FAnimNextParamType::GetType<int32>(), TEXT("34"));
		EditorData->AddVariable(TEXT("TestVec3Var"), FAnimNextParamType::GetType<FVector>(), TEXT("(X=2.5,Y=1.25,Z=-7.75)"));
		
		UAnimNextController* Controller = Cast<UAnimNextController>(EditorData->GetRigVMClient()->GetController(EditorData->GetRigVMClient()->GetDefaultModel()));
		UE_RETURN_ON_ERROR(Controller != nullptr, "FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Failed to get RigVM controller");

		// Find graph entry point
		URigVMNode* MainEntryPointNode = Controller->GetGraph()->FindNodeByName(FRigUnit_AnimNextGraphRoot::StaticStruct()->GetFName());
		UE_RETURN_ON_ERROR(MainEntryPointNode != nullptr, "FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Failed to find main entry point node");

		URigVMPin* BeginExecutePin = MainEntryPointNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AnimNextGraphRoot, Result));
		UE_RETURN_ON_ERROR(BeginExecutePin != nullptr && BeginExecutePin->GetDirection() == ERigVMPinDirection::Input, "FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Failed to create entry point");

		URigVMUnitNode* DecoratorStackNode = nullptr;
		FString DisplayName;
		{
			// Suspend auto compilation until we have constructed a valid trait stack
			TGuardValue<bool> SuspendCompile(EditorData->bAutoRecompileVM, false);

			// Create an empty trait stack node
			DecoratorStackNode = Controller->AddUnitNode(FRigUnit_AnimNextTraitStack::StaticStruct(), FRigVMStruct::ExecuteName, FVector2D(0.0f, 0.0f), FString(), false);
			UE_RETURN_ON_ERROR(DecoratorStackNode != nullptr, "FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Failed to create trait stack node");

			// Link our stack result to our entry point
			Controller->AddLink(DecoratorStackNode->GetPins()[0], MainEntryPointNode->GetPins()[0]);

			// Add a trait
			const UScriptStruct* CppDecoratorStruct = FRigDecorator_AnimNextCppDecorator::StaticStruct();
			UE_RETURN_ON_ERROR(CppDecoratorStruct != nullptr, "FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Failed to get find Cpp trait static struct");

			const FTrait* Trait = FTraitRegistry::Get().Find(FTestTrait::TraitUID);
			UE_RETURN_ON_ERROR(Trait != nullptr, "FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Failed to find test trait");

			UScriptStruct* ScriptStruct = Trait->GetTraitSharedDataStruct();
			UE_RETURN_ON_ERROR(ScriptStruct != nullptr, "FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Failed to find trait shared data struct");

			FString DefaultValue;
			{
				const FRigDecorator_AnimNextCppDecorator DefaultCppDecoratorStructInstance;
				FRigDecorator_AnimNextCppDecorator CppDecoratorStructInstance;
				CppDecoratorStructInstance.DecoratorSharedDataStruct = ScriptStruct;

				UE_RETURN_ON_ERROR(CppDecoratorStructInstance.CanBeAddedToNode(DecoratorStackNode, nullptr), "FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Trait cannot be added to trait stack node");

				FRigDecorator_AnimNextCppDecorator::StaticStruct()->ExportText(DefaultValue, &CppDecoratorStructInstance, &DefaultCppDecoratorStructInstance, nullptr, PPF_None, nullptr);
			}

			FString DisplayNameMetadata;
			ScriptStruct->GetStringMetaDataHierarchical(FRigVMStruct::DisplayNameMetaName, &DisplayNameMetadata);
			DisplayName = DisplayNameMetadata.IsEmpty() ? Trait->GetTraitName() : DisplayNameMetadata;

			const FName DecoratorName = Controller->AddTrait(
				DecoratorStackNode->GetFName(),
				*CppDecoratorStruct->GetPathName(),
				*DisplayName,
				DefaultValue, INDEX_NONE, true, true);
			UE_RETURN_ON_ERROR(DecoratorName == DisplayName, TEXT("FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Unexpected trait name"));
		}

		// Set some values on our trait
		URigVMPin* DecoratorPin = DecoratorStackNode->FindPin(*DisplayName);
		UE_RETURN_ON_ERROR(DecoratorPin != nullptr, TEXT("FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Failed to find trait pin"));

		Controller->SetPinDefaultValue(DecoratorPin->GetSubPins()[3]->GetPinPath(), TEXT("78"));		// SomeInt32
		Controller->SetPinDefaultValue(DecoratorPin->GetSubPins()[4]->GetPinPath(), TEXT("142.33"));	// SomeFloat
		Controller->SetPinDefaultValue(DecoratorPin->GetSubPins()[9]->GetPinPath(), TEXT("1123.31"));	// SomeLatentFloat, inline value on latent pin

		// Set some latent values on our trait
		{
			FRigVMFunction_MathIntAdd IntAdd;
			IntAdd.A = 10;
			IntAdd.B = 23;

			URigVMUnitNode* IntAddNode = Controller->AddUnitNodeWithDefaults(FRigVMFunction_MathIntAdd::StaticStruct(), FRigStructScope(IntAdd), FRigVMStruct::ExecuteName, FVector2D::ZeroVector, FString(), false);
			UE_RETURN_ON_ERROR(IntAddNode != nullptr, TEXT("FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Failed to create Int add node"));

			Controller->AddLink(
				IntAddNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigVMFunction_MathIntAdd, Result)),
				DecoratorPin->GetSubPins()[5]);	// SomeLatentInt32
		}

		{
			URigVMVariableNode* GetVariableNode = Controller->AddVariableNode(TEXT("TestIntVar"), RigVMTypeUtils::Int32Type, nullptr, true, TEXT(""));
			UE_RETURN_ON_ERROR(GetVariableNode != nullptr, TEXT("FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Failed to create variable node"));

			Controller->AddLink(
				GetVariableNode->FindPin(TEXT("Value")),
				DecoratorPin->GetSubPins()[6]);	// SomeOtherLatentInt32

			URigVMVariableNode* GetVariableNode1 = Controller->AddVariableNode(TEXT("TestVec3Var"), TEXT("FVector"), nullptr, true, TEXT(""));
			UE_RETURN_ON_ERROR(GetVariableNode1 != nullptr, TEXT("FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Failed to create variable node"));

			Controller->AddLink(
				GetVariableNode1->FindPin(TEXT("Value")),
				DecoratorPin->GetSubPins()[8]);	// SomeOtherLatentInt32
		}

		TSharedPtr<FAnimNextGraphInstance> GraphInstance = AnimationGraph->AllocateInstance();

		TArray<FString> Messages;
		FRigVMRuntimeSettings RuntimeSettings;
		RuntimeSettings.SetLogFunction([&Messages](const FRigVMLogSettings& InLogSettings, const FRigVMExecuteContext* InContext, const FString& Message)
		{
			Messages.Add(Message);
		});
		GraphInstance->GetOrAddComponent<FUAFRigVMComponent>().GetExtendedExecuteContext().SetRuntimeSettings(RuntimeSettings);
		
		{
			FUpdateGraphContext UpdateGraphContext(*GraphInstance.Get(), 1.0f / 30.0f);
			UE::UAF::UpdateGraph(UpdateGraphContext);
			const UE::UAF::FEvaluateGraphContext EvaluateGraphContext(*GraphInstance.Get(), UE::UAF::FReferencePose(), 0);
			(void)UE::UAF::EvaluateGraph(EvaluateGraphContext);
		}

		UE_RETURN_ON_ERROR(Messages.Num() == 9, "FAnimationAnimNextRuntimeTest_GraphExecuteLatent - > Unexpected message count");
		AddErrorIfFalse(Messages[0] == "UpdateCount == 1", "FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Unexpected update count");
		AddErrorIfFalse(Messages[1] == "SomeInt32 == 78", "FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Unexpected SomeInt32 value");
		AddErrorIfFalse(Messages[2] == "SomeFloat == 142.33", "FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Unexpected SomeFloat value");
		AddErrorIfFalse(Messages[3] == "SomeLatentInt32 == 33", "FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Unexpected SomeLatentInt32 value");
		AddErrorIfFalse(Messages[4] == "SomeOtherLatentInt32 == 34", "FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Unexpected SomeOtherLatentInt32 value");
		AddErrorIfFalse(Messages[5] == "SomeOutOfOrderLatentBool == 0", "FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Unexpected SomeOutOfOrderLatentBool value");
		AddErrorIfFalse(Messages[6] == "SomeLatentVector == [2.50, 1.25, -7.75]", "FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Unexpected SomeLatentVector value");
		AddErrorIfFalse(Messages[7] == "SomeLatentFloat == 1123.31", "FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Unexpected SomeLatentFloat value");
		AddErrorIfFalse(Messages[8] == "EvaluateCount == 1", "FAnimationAnimNextRuntimeTest_GraphExecuteLatent -> Unexpected evaluate count");
	}

	Tests::FUtils::CleanupAfterTests();

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnimationAnimNextRuntimeTest_Variables, "Animation.AnimNext.Runtime.Graph.Variables", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAnimationAnimNextRuntimeTest_Variables::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	{
		UFactory* GraphFactory = NewObject<UUAFAnimGraphFactory>();
		UUAFAnimGraph* AnimationGraph = CastChecked<UUAFAnimGraph>(GraphFactory->FactoryCreateNew(UUAFAnimGraph::StaticClass(), GetTransientPackage(), TEXT("TestAnimNextGraph"), RF_Transient, nullptr, nullptr, NAME_None));
		UE_RETURN_ON_ERROR(AnimationGraph != nullptr, "FAnimationAnimNextRuntimeTest_Variables -> Failed to create animation graph");

		auto AddPublicVariable = [this](UUAFAnimGraph* InAnimationGraph, FName InName, const auto& InValue)
		{
			using ValueType = std::remove_reference_t<decltype(InValue)>;
			FAnimNextParamType Type = FAnimNextParamType::GetType<ValueType>();
			UAnimNextVariableEntry* VariableEntry = UUAFRigVMAssetLibrary::AddVariable(InAnimationGraph, InName, Type.GetValueType(), Type.GetContainerType(), Type.GetValueTypeObject(), TEXT(""), false, false);
			UE_RETURN_ON_ERROR(VariableEntry != nullptr, "FAnimationAnimNextRuntimeTest_Variables::AddPublicVariable -> Failed to create variable");
			VariableEntry->SetExportAccessSpecifier(EAnimNextExportAccessSpecifier::Public, false);
			UE_RETURN_ON_ERROR(VariableEntry->SetDefaultValue(InValue, false), "FAnimationAnimNextRuntimeTest_Variables::AddPublicVariable -> Failed to set variable default value");
			return true;
		};

		AddPublicVariable(AnimationGraph, "Bool", true);
		AddPublicVariable(AnimationGraph, "Byte", (uint8)42);
		AddPublicVariable(AnimationGraph, "Int32", (int32)-4679222);
		AddPublicVariable(AnimationGraph, "Int64", (int64)-3415919105);
		AddPublicVariable(AnimationGraph, "Float", 1.0f);
		AddPublicVariable(AnimationGraph, "Double", 1.0);
		AddPublicVariable(AnimationGraph, "Name", FName("Test"));
		AddPublicVariable(AnimationGraph, "String", FString("Test"));
		AddPublicVariable(AnimationGraph, "Enum", EPropertyBagPropertyType::Double);
		AddPublicVariable(AnimationGraph, "Struct", FVector::OneVector);
		AddPublicVariable(AnimationGraph, "DerivedStruct", FTestDerivedVector());
		AddPublicVariable(AnimationGraph, "Object", UUAFSharedVariables::StaticClass()->GetDefaultObject<UUAFSharedVariables>());
		AddPublicVariable(AnimationGraph, "SoftObject", FSoftObjectPath(UUAFSharedVariables::StaticClass()->GetDefaultObject<UUAFSharedVariables>()));
		AddPublicVariable(AnimationGraph, "Class", UUAFSharedVariables::StaticClass());
		AddPublicVariable(AnimationGraph, "SoftClass", FSoftClassPath(UUAFSharedVariables::StaticClass()));

		TSharedPtr<FAnimNextGraphInstance> GraphInstance = AnimationGraph->AllocateInstance();
		UE_RETURN_ON_ERROR(GraphInstance.IsValid(), "Failed to allocate animation graph instance");

		// Bool/Integers + conversions
		{
			// Gets
			bool TestBool = false;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Bool", AnimationGraph), TestBool) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestBool == true, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			uint8 TestByte = 0;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Byte", AnimationGraph), TestByte) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestByte == 42, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			int32 TestInt32 = 0;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Int32", AnimationGraph), TestInt32) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestInt32 == -4679222, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			int64 TestInt64 = 0;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Int64", AnimationGraph), TestInt64) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestInt64 == -3415919105, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			// Conversions

			// Bool
			TestByte = 0;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Bool", AnimationGraph), TestByte) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestByte == 1, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			TestInt32 = 0;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Bool", AnimationGraph), TestInt32) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestInt32 == 1, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			TestInt64 = 0;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Bool", AnimationGraph), TestInt64) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestInt64 == 1, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			// Byte
			TestBool = false;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Byte", AnimationGraph), TestBool) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestBool == true, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			TestInt32 = 0;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Byte", AnimationGraph), TestInt32) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestInt32 == 42, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			TestInt64 = 0;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Byte", AnimationGraph), TestInt64) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestInt64 == 42, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			// Int32
			TestBool = false;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Int32", AnimationGraph), TestBool) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestBool == true, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			TestByte = 0;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Int32", AnimationGraph), TestByte) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestByte == 202/*(uint8)-4679222*/, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			TestInt64 = 0;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Int32", AnimationGraph), TestInt64) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestInt64 == -4679222, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			// Int64
			TestBool = false;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Int64", AnimationGraph), TestBool) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestBool == true, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			TestByte = 0;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Int64", AnimationGraph), TestByte) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestByte == 255/*(uint8)-3415919105*/, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			TestInt32 = 0;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Int64", AnimationGraph), TestInt32) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestInt32 == 879048191/*(int32)-3415919105*/, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");
		}

		// Float/double + conversions
		{
			// Gets
			float TestFloat = 0.0f;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Float", AnimationGraph), TestFloat) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestFloat == 1.0f, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			double TestDouble = 0.0;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Double", AnimationGraph), TestDouble) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestDouble == 1.0, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			// Conversions
			TestDouble = 0.0f;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Float", AnimationGraph), TestDouble) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestDouble == 1.0f, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			TestFloat = 0.0;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Double", AnimationGraph), TestFloat) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestFloat == 1.0, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");
		}
		
		FName TestFName = NAME_None;
		UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Name", AnimationGraph), TestFName) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
		UE_RETURN_ON_ERROR(TestFName == "Test", "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

		FString TestString;
		UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("String", AnimationGraph), TestString) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
		UE_RETURN_ON_ERROR(TestString == "Test", "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

		// Enums
		{
			EPropertyBagPropertyType TestEnum = EPropertyBagPropertyType::None;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Enum", AnimationGraph), TestEnum) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestEnum == EPropertyBagPropertyType::Double, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			// Test mismatched enum fails
			EPropertyBagContainerType MismatchedEnum = EPropertyBagContainerType::None;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Enum", AnimationGraph), MismatchedEnum) != EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable succeeded");
		}

		// Structs
		{
			FVector TestStruct = FVector::ZeroVector;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Struct", AnimationGraph), TestStruct) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestStruct == FVector::OneVector, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			// Test conversion from base -> derived fails 
			FTestDerivedVector TestDerived;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Struct", AnimationGraph), TestDerived) != EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable succeeded");

			// Test conversion from derived -> base
			TestStruct = FVector::ZeroVector;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("DerivedStruct", AnimationGraph), TestStruct) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestStruct == FVector::OneVector, "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");
		}

		// Objects
		{
			UUAFSharedVariables* TestObject = nullptr;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Object", AnimationGraph), TestObject) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestObject == UUAFSharedVariables::StaticClass()->GetDefaultObject<UUAFSharedVariables>(), "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			// Test unrelated object fails
			UAnimSequence* TestAnimSequence = nullptr;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Object", AnimationGraph), TestAnimSequence) != EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable succeeded");

			// Test Derived -> Base succeeds
			UObject* BaseObject = nullptr;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Object", AnimationGraph), BaseObject) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(BaseObject == UUAFSharedVariables::StaticClass()->GetDefaultObject<UUAFSharedVariables>(), "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			// Test Base -> Derived fails
			UUAFAnimGraph* DerivedObject = nullptr;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Object", AnimationGraph), DerivedObject) != EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable succeeded");
		}

		FSoftObjectPath TestSoftObjectPath;
		UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("SoftObject", AnimationGraph), TestSoftObjectPath) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
		UE_RETURN_ON_ERROR(TestSoftObjectPath == FSoftObjectPath(UUAFSharedVariables::StaticClass()->GetDefaultObject<UUAFSharedVariables>()), "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

		// Classes
		{
			UClass* TestClass = nullptr;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Class", AnimationGraph), TestClass) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(TestClass == UUAFSharedVariables::StaticClass(), "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");


			// Test unrelated class fails
			TSubclassOf<UAnimSequence> TestUnrelatedSubclassOf;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Class", AnimationGraph), TestUnrelatedSubclassOf) != EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable succeeded");

			// Test Derived -> Base succeeds
			TSubclassOf<UObject> BaseSubclassOf;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Class", AnimationGraph), BaseSubclassOf) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
			UE_RETURN_ON_ERROR(BaseSubclassOf == UUAFSharedVariables::StaticClass(), "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");

			// Test Base -> Derived fails
			TSubclassOf<UUAFAnimGraph> DerivedSubclassOf;
			UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("Class", AnimationGraph), DerivedSubclassOf) != EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable succeeded");
		}

		FSoftClassPath TestSoftClassPath;
		UE_RETURN_ON_ERROR(GraphInstance->GetVariable(FAnimNextVariableReference::FromName("SoftClass", AnimationGraph), TestSoftClassPath) == EPropertyBagResult::Success, "FAnimationAnimNextRuntimeTest_Variables -> GetVariable failed");
		UE_RETURN_ON_ERROR(TestSoftClassPath == FSoftClassPath(UUAFSharedVariables::StaticClass()), "FAnimationAnimNextRuntimeTest_Variables -> Variable value did not match");
	}

	Tests::FUtils::CleanupAfterTests();

	return true;
}


#endif
