// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/TraitBinding.h"
#include "TraitCore/TraitSharedData.h"
#include "Module/AnimNextModule.h"

#include "AnimNextAnimGraphTraitGraphTest.generated.h"

USTRUCT()
struct FTestTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY(meta = (Input, Inline))
	int32 UpdateCount = 0;

	UPROPERTY(meta = (Input, Inline))
	int32 EvaluateCount = 0;

	UPROPERTY(meta = (Input, Inline))
	int32 SomeInt32 = 3;

	UPROPERTY(meta = (Input, Inline))
	float SomeFloat = 34.0f;

	UPROPERTY(meta = (Input))
	int32 SomeLatentInt32 = 5;			// MathAdd with constants, latent

	UPROPERTY(meta = (Input))
	int32 SomeOtherLatentInt32 = 7;		// GetParameter, latent

	UPROPERTY(meta = (Input))
	bool SomeOutOfOrderLatentBool = false;	// GetParameter, latent

	UPROPERTY(meta = (Input))
	FVector SomeLatentVector = FVector::OneVector;	// GetParameter, latent

	UPROPERTY(meta = (Input))
	float SomeLatentFloat = 34.0f;		// Inline value, not latent

	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(SomeLatentInt32) \
		/* Out of order with property definition above */ \
		GeneratorMacro(SomeOutOfOrderLatentBool) \
		GeneratorMacro(SomeOtherLatentInt32) \
		GeneratorMacro(SomeLatentVector) \
		GeneratorMacro(SomeLatentFloat) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FTestTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

USTRUCT()
struct FTestDerivedVector : public FVector
{
	GENERATED_BODY()
	
	FTestDerivedVector()
		: FVector(FVector::OneVector)
	{}

	UPROPERTY()
	int32 W = 1;
};
