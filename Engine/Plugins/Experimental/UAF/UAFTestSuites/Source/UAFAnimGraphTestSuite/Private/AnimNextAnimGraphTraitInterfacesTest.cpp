// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextAnimGraphTraitInterfacesTest.h"
#include "AnimNextRuntimeTest.h"
#include "AnimNextTest.h"

#include "Misc/AutomationTest.h"
#include "TraitInterfaces/IScopedTag.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextAnimGraphTraitInterfacesTest)

#if WITH_DEV_AUTOMATION_TESTS

#include "Serialization/MemoryReader.h"

#include "TraitCore/Trait.h"
#include "TraitCore/TraitReader.h"
#include "TraitCore/TraitRegistry.h"
#include "TraitCore/TraitInterfaceRegistry.h"
#include "TraitCore/TraitWriter.h"
#include "TraitCore/ExecutionContext.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/IScopedTraitInterface.h"
#include "TraitCore/NodeInstance.h"
#include "TraitCore/NodeTemplateBuilder.h"
#include "TraitCore/NodeTemplateRegistry.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IHierarchy.h"
#include "TraitInterfaces/IUpdate.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextAnimationGraphFactory.h"

//****************************************************************************
// AnimNext Runtime TraitInterfaces Tests
//****************************************************************************

namespace UE::UAF
{
	namespace Private
	{
		static TArray<FTraitUID>* UpdatedTraits = nullptr;
		static TArray<FTraitUID>* EvaluatedTraits = nullptr;

		static FName TestTag(TEXT("MyTag"));
		static TArray<bool>* IsTagInScope = nullptr;
		static bool AutoPopTag = false;
	}

	struct FTraitWithNoChildren : FBaseTrait, IUpdate, IEvaluate
	{
		DECLARE_ANIM_TRAIT(FTraitWithNoChildren, FBaseTrait)

		// IUpdate impl
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override
		{
			if (Private::UpdatedTraits != nullptr)
			{
				Private::UpdatedTraits->Add(FTraitWithNoChildren::TraitUID);
			}

			IUpdate::PreUpdate(Context, Binding, TraitState);
		}

		virtual void PostUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override
		{
			if (Private::UpdatedTraits != nullptr)
			{
				Private::UpdatedTraits->Add(FTraitWithNoChildren::TraitUID);
			}

			IUpdate::PostUpdate(Context, Binding, TraitState);
		}

		// IEvaluate impl
		virtual void PreEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override
		{
			if (Private::EvaluatedTraits != nullptr)
			{
				Private::EvaluatedTraits->Add(FTraitWithNoChildren::TraitUID);
			}

			IEvaluate::PreEvaluate(Context, Binding);
		}

		virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override
		{
			if (Private::EvaluatedTraits != nullptr)
			{
				Private::EvaluatedTraits->Add(FTraitWithNoChildren::TraitUID);
			}

			IEvaluate::PostEvaluate(Context, Binding);
		}
	};

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IEvaluate) \
		GeneratorMacro(IUpdate) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FTraitWithNoChildren, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	// This trait does not update or evaluate
	struct FTraitWithOneChild : FBaseTrait, IHierarchy
	{
		DECLARE_ANIM_TRAIT(FTraitWithOneChild, FBaseTrait)

		using FSharedData = FTraitWithOneChildSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			FTraitPtr Child;

			void Construct(const FExecutionContext& Context, const FTraitBinding& Binding)
			{
				Child = Context.AllocateNodeInstance(Binding.GetTraitPtr(), Binding.GetSharedData<FSharedData>()->Child);
			}
		};

		// IHierarchy impl
		virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override
		{
			return 1;
		}

		virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override
		{
			const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

			Children.Add(InstanceData->Child);
		}
	};

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IHierarchy) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FTraitWithOneChild, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	struct FTraitWithChildren : FBaseTrait, IHierarchy, IUpdate, IUpdateTraversal, IEvaluate
	{
		DECLARE_ANIM_TRAIT(FTraitWithChildren, FBaseTrait)

		using FSharedData = FTraitWithChildrenSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			FTraitPtr Children[2];

			void Construct(const FExecutionContext& Context, const FTraitBinding& Binding)
			{
				Children[0] = Context.AllocateNodeInstance(Binding.GetTraitPtr(), Binding.GetSharedData<FSharedData>()->Children[0]);
				Children[1] = Context.AllocateNodeInstance(Binding.GetTraitPtr(), Binding.GetSharedData<FSharedData>()->Children[1]);
			}
		};

		// IHierarchy impl
		virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override
		{
			return 2;
		}

		virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override
		{
			const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

			Children.Add(InstanceData->Children[0]);
			Children.Add(InstanceData->Children[1]);
		}

		// IUpdate impl
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override
		{
			if (Private::UpdatedTraits != nullptr)
			{
				Private::UpdatedTraits->Add(FTraitWithChildren::TraitUID);
			}

			IUpdate::PreUpdate(Context, Binding, TraitState);
		}

		virtual void PostUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override
		{
			if (Private::UpdatedTraits != nullptr)
			{
				Private::UpdatedTraits->Add(FTraitWithChildren::TraitUID);
			}

			IUpdate::PostUpdate(Context, Binding, TraitState);
		}

		// IUpdateTraversal impl
		virtual void QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const override
		{
			const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

			TraversalQueue.Push(InstanceData->Children[0], TraitState);
			TraversalQueue.Push(InstanceData->Children[1], TraitState);
		}

		// IEvaluate impl
		virtual void PreEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override
		{
			if (Private::EvaluatedTraits != nullptr)
			{
				Private::EvaluatedTraits->Add(FTraitWithChildren::TraitUID);
			}

			IEvaluate::PreEvaluate(Context, Binding);
		}

		virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override
		{
			if (Private::EvaluatedTraits != nullptr)
			{
				Private::EvaluatedTraits->Add(FTraitWithChildren::TraitUID);
			}

			IEvaluate::PostEvaluate(Context, Binding);
		}
	};

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IEvaluate) \
		GeneratorMacro(IHierarchy) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IUpdateTraversal) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FTraitWithChildren, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	// Adds a scoped tag
	struct FScopedTagTrait : FAdditiveTrait, IScopedTagInterface, IUpdate
	{
		DECLARE_ANIM_TRAIT(FScopedTagTrait, FAdditiveTrait)

		// IScopedTagInterface impl
		virtual FName GetTag(const FExecutionContext& Context, const TTraitBinding<IScopedTagInterface>& Binding) const override
		{
			return Private::TestTag;
		}

		// IUpdate impl
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override
		{
			Context.PushScopedInterface<IScopedTagInterface>(Binding);

			IUpdate::PreUpdate(Context, Binding, TraitState);
		}

		virtual void PostUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override
		{
			if (!Private::AutoPopTag)
			{
				ensure(Context.PopScopedInterface<IScopedTagInterface>(Binding));
			}

			IUpdate::PostUpdate(Context, Binding, TraitState);
		}
	};

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IScopedTagInterface) \
		GeneratorMacro(IUpdate) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FScopedTagTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	// Tests if we have a scoped tag
	struct FTestScopedTagTrait : FAdditiveTrait, IUpdate
	{
		DECLARE_ANIM_TRAIT(FTestScopedTagTrait, FAdditiveTrait)

		// IUpdate impl
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override
		{
			if (Private::IsTagInScope != nullptr)
			{
				Private::IsTagInScope->Add(IScopedTagInterface::IsTagInScope(Context, Private::TestTag));
			}

			IUpdate::PreUpdate(Context, Binding, TraitState);
		}

		virtual void PostUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override
		{
			if (Private::IsTagInScope != nullptr)
			{
				Private::IsTagInScope->Add(IScopedTagInterface::IsTagInScope(Context, Private::TestTag));
			}

			IUpdate::PostUpdate(Context, Binding, TraitState);
		}
	};

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IUpdate) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FTestScopedTagTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR


	static const FText GInterfaceTestAName = FText::FromString(TEXT("Interface Test A"));
	static const FText GInterfaceTestAShortName = FText::FromString(TEXT("ITA"));

	struct ITraitInterfaceTestA : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(ITraitInterfaceTestA)

	#if WITH_EDITOR
		virtual const FText& GetDisplayName() const override
		{
			return GInterfaceTestAName;
		}

		virtual const FText& GetDisplayShortName() const override
		{
			return GInterfaceTestAShortName;
		}
	#endif // WITH_EDITOR
	};

	static const FText GInterfaceTestBName = FText::FromString(TEXT("Interface Test B"));
	static const FText GInterfaceTestBShortName = FText::FromString(TEXT("ITB"));

	struct ITraitInterfaceTestB : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(ITraitInterfaceTestB)

	#if WITH_EDITOR
		virtual const FText& GetDisplayName() const override
		{
			return GInterfaceTestBName;
		}

		virtual const FText& GetDisplayShortName() const override
		{
			return GInterfaceTestBShortName;
		}

		virtual bool IsInternal() const override
		{
			return true;
		}
	#endif // WITH_EDITOR
	};

} // end namespace UE::UAF

// --- Runtime Test Trait Interface Registry ---

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnimationAnimNextRuntimeTest_TraitInterfaceRegistry, "Animation.AnimNext.Runtime.TraitInterfaceRegistry", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAnimationAnimNextRuntimeTest_TraitInterfaceRegistry::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	{
		FTraitInterfaceRegistry& Registry = FTraitInterfaceRegistry::Get();

		// Some traits already exist in the engine, keep track of them
		const uint32 NumAutoRegisteredTraitInterfaces = Registry.GetNum();

		AddErrorIfFalse(Registry.Find(ITraitInterfaceTestA::InterfaceUID) == nullptr, "FAnimationAnimNextRuntimeTest_TraitInterfaceRegistry -> Registry should not contain the Test Interface A");
		AddErrorIfFalse(Registry.Find(ITraitInterfaceTestB::InterfaceUID) == nullptr, "FAnimationAnimNextRuntimeTest_TraitInterfaceRegistry -> Registry should not contain the Test Interface B");

		{
			AUTO_REGISTER_ANIM_TRAIT_INTERFACE(ITraitInterfaceTestA)

			AddErrorIfFalse(Registry.GetNum() == NumAutoRegisteredTraitInterfaces + 1, "FAnimationAnimNextRuntimeTest_TraitInterfaceRegistry -> Registry should contain 1 new trait interface");

			const ITraitInterface* TraitInterfaceA = Registry.Find(ITraitInterfaceTestA::InterfaceUID);
			AddErrorIfFalse(TraitInterfaceA->GetInterfaceUID() == ITraitInterfaceTestA::InterfaceUID, "FAnimationAnimNextRuntimeTest_TraitInterfaceRegistry -> Incorrect InterfaceUID for ITraitInterfaceTestA");

#if WITH_EDITOR
			AddErrorIfFalse(TraitInterfaceA->GetDisplayName().EqualTo(GInterfaceTestAName), "FAnimationAnimNextRuntimeTest_TraitInterfaceRegistry -> Incorrect Interface Display Name for ITraitInterfaceTestA");
			AddErrorIfFalse(TraitInterfaceA->GetDisplayShortName().EqualTo(GInterfaceTestAShortName), "FAnimationAnimNextRuntimeTest_TraitInterfaceRegistry -> Incorrect Interface Display Short Name for ITraitInterfaceTestA");

			AddErrorIfFalse(TraitInterfaceA->IsInternal() == false, "FAnimationAnimNextRuntimeTest_TraitInterfaceRegistry -> Incorrect Interface Internal flag for ITraitInterfaceTestA");
#endif // WITH_EDITOR

			{
				AUTO_REGISTER_ANIM_TRAIT_INTERFACE(ITraitInterfaceTestB)

				AddErrorIfFalse(Registry.GetNum() == NumAutoRegisteredTraitInterfaces + 2, "FAnimationAnimNextRuntimeTest_TraitInterfaceRegistry -> Registry should contain 2 new trait interfaces");

				const ITraitInterface* TraitInterfaceB = Registry.Find(ITraitInterfaceTestB::InterfaceUID);
				AddErrorIfFalse(TraitInterfaceB->GetInterfaceUID() == ITraitInterfaceTestB::InterfaceUID, "FAnimationAnimNextRuntimeTest_TraitInterfaceRegistry -> Incorrect InterfaceUID for ITraitInterfaceTestB");

#if WITH_EDITOR
				AddErrorIfFalse(TraitInterfaceB->GetDisplayName().EqualTo(GInterfaceTestBName), "FAnimationAnimNextRuntimeTest_TraitInterfaceRegistry -> Incorrect Interface Display Name for ITraitInterfaceTestB");
				AddErrorIfFalse(TraitInterfaceB->GetDisplayShortName().EqualTo(GInterfaceTestBShortName), "FAnimationAnimNextRuntimeTest_TraitInterfaceRegistry -> Incorrect Interface Display Short Name for ITraitInterfaceTestB");

				AddErrorIfFalse(TraitInterfaceB->IsInternal() == true, "FAnimationAnimNextRuntimeTest_TraitInterfaceRegistry -> Incorrect Interface Internal flag for ITraitInterfaceTestB");
#endif // WITH_EDITOR
			}

			AddErrorIfFalse(Registry.Find(ITraitInterfaceTestB::InterfaceUID) == nullptr, "FAnimationAnimNextRuntimeTest_TraitInterfaceRegistry -> Registry should not contain the Test Interface B");

			AddErrorIfFalse(Registry.GetNum() == NumAutoRegisteredTraitInterfaces + 1, "FAnimationAnimNextRuntimeTest_TraitInterfaceRegistry -> Registry should contain 1 new trait interface");
		}

		AddErrorIfFalse(Registry.Find(ITraitInterfaceTestA::InterfaceUID) == nullptr, "FAnimationAnimNextRuntimeTest_TraitInterfaceRegistry -> Registry should not contain the Test Interface A");

		AddErrorIfFalse(Registry.GetNum() == NumAutoRegisteredTraitInterfaces, "FAnimationAnimNextRuntimeTest_TraitInterfaceRegistry -> Registry should contain 0 new trait interfaces");
	}

	Tests::FUtils::CleanupAfterTests();

	return true;
}

// --- Trait Interfaces IHierarchy Test ---

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnimationAnimNextRuntimeTest_IHierarchy, "Animation.AnimNext.Runtime.IHierarchy", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAnimationAnimNextRuntimeTest_IHierarchy::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	{
		AUTO_REGISTER_ANIM_TRAIT(FTraitWithNoChildren)
		AUTO_REGISTER_ANIM_TRAIT(FTraitWithOneChild)
		AUTO_REGISTER_ANIM_TRAIT(FTraitWithChildren)

		UFactory* GraphFactory = NewObject<UUAFAnimGraphFactory>();
		UUAFAnimGraph* AnimationGraph = CastChecked<UUAFAnimGraph>(GraphFactory->FactoryCreateNew(UUAFAnimGraph::StaticClass(), GetTransientPackage(), TEXT("TestAnimNextGraph"), RF_Transient, nullptr, nullptr, NAME_None));
		UE_RETURN_ON_ERROR(AnimationGraph != nullptr, "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Failed to create animation graph");

		FScopedClearNodeTemplateRegistry ScopedClearNodeTemplateRegistry;
		FNodeTemplateRegistry& Registry = FNodeTemplateRegistry::Get();

		// We create a few node templates
		// Template A has a single trait with no children
		TArray<FTraitUID> NodeTemplateTraitListA;
		NodeTemplateTraitListA.Add(FTraitWithNoChildren::TraitUID);

		// Template B has a single trait with one child
		TArray<FTraitUID> NodeTemplateTraitListB;
		NodeTemplateTraitListB.Add(FTraitWithOneChild::TraitUID);

		// Template C has two traits, each with one child
		TArray<FTraitUID> NodeTemplateTraitListC;
		NodeTemplateTraitListC.Add(FTraitWithOneChild::TraitUID);
		NodeTemplateTraitListC.Add(FTraitWithOneChild::TraitUID);

		// Template D has a single trait with children
		TArray<FTraitUID> NodeTemplateTraitListD;
		NodeTemplateTraitListD.Add(FTraitWithChildren::TraitUID);

		// Populate our node template registry
		TArray<uint8> NodeTemplateBufferA, NodeTemplateBufferB, NodeTemplateBufferC, NodeTemplateBufferD;
		const FNodeTemplate* NodeTemplateA = FNodeTemplateBuilder::BuildNodeTemplate(NodeTemplateTraitListA, NodeTemplateBufferA);
		const FNodeTemplate* NodeTemplateB = FNodeTemplateBuilder::BuildNodeTemplate(NodeTemplateTraitListB, NodeTemplateBufferB);
		const FNodeTemplate* NodeTemplateC = FNodeTemplateBuilder::BuildNodeTemplate(NodeTemplateTraitListC, NodeTemplateBufferC);
		const FNodeTemplate* NodeTemplateD = FNodeTemplateBuilder::BuildNodeTemplate(NodeTemplateTraitListD, NodeTemplateBufferD);

		// Build our graph, it as follow (each node template has a single node instance):
		// NodeA has no children
		// NodeB has one child: NodeA
		// NodeC has two children: NodeA and NodeB (but both traits are base, only NodeB will be referenced)
		// NodeD has two children: NodeA and NodeC

		TArray<FNodeHandle> NodeHandles;

		// Write our graph
		TArray<uint8> GraphSharedDataArchiveBuffer;
		TArray<TObjectPtr<UObject>> GraphReferencedObjects;
		TArray<FSoftObjectPath> GraphReferencedSoftObjects;
		{
			FTraitWriter TraitWriter;

			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplateA));
			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplateB));
			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplateC));
			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplateD));

			// We don't have trait properties
			TArray<TMap<FName, FString>> TraitPropertiesA;
			TraitPropertiesA.AddDefaulted(NodeTemplateTraitListA.Num());

			TArray<TMap<FName, FString>> TraitPropertiesB;
			TraitPropertiesB.AddDefaulted(NodeTemplateTraitListB.Num());
			TraitPropertiesB[0].Add(TEXT("Child"), ToString<FTraitWithOneChild::FSharedData>(TEXT("Child"), FAnimNextTraitHandle(NodeHandles[0])));

			TArray<TMap<FName, FString>> TraitPropertiesC;
			TraitPropertiesC.AddDefaulted(NodeTemplateTraitListC.Num());
			TraitPropertiesC[0].Add(TEXT("Child"), ToString<FTraitWithOneChild::FSharedData>(TEXT("Child"), FAnimNextTraitHandle(NodeHandles[0])));
			TraitPropertiesC[1].Add(TEXT("Child"), ToString<FTraitWithOneChild::FSharedData>(TEXT("Child"), FAnimNextTraitHandle(NodeHandles[1])));

			TArray<TMap<FName, FString>> TraitPropertiesD;
			TraitPropertiesD.AddDefaulted(NodeTemplateTraitListD.Num());
			FAnimNextTraitHandle ChildrenHandlesD[2] = { FAnimNextTraitHandle(NodeHandles[0]), FAnimNextTraitHandle(NodeHandles[2], 1)};
			TraitPropertiesD[0].Add(TEXT("Children"), ToString<FTraitWithChildren::FSharedData>(TEXT("Children"), ChildrenHandlesD));

			TraitWriter.BeginNodeWriting();
			TraitWriter.WriteNode(NodeHandles[0],
				[&TraitPropertiesA](uint32 TraitIndex, FName PropertyName)
				{
					return TraitPropertiesA[TraitIndex][PropertyName];
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.WriteNode(NodeHandles[1],
				[&TraitPropertiesB](uint32 TraitIndex, FName PropertyName)
				{
					return TraitPropertiesB[TraitIndex][PropertyName];
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.WriteNode(NodeHandles[2],
				[&TraitPropertiesC](uint32 TraitIndex, FName PropertyName)
				{
					return TraitPropertiesC[TraitIndex][PropertyName];
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.WriteNode(NodeHandles[3],
				[&TraitPropertiesD](uint32 TraitIndex, FName PropertyName)
				{
					return TraitPropertiesD[TraitIndex][PropertyName];
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.EndNodeWriting();

			AddErrorIfFalse(TraitWriter.GetErrorState() == FTraitWriter::EErrorState::None, "FAnimationAnimNextRuntimeTest_IHierarchy -> Failed to write traits");
			GraphSharedDataArchiveBuffer = TraitWriter.GetGraphSharedData();
			GraphReferencedObjects = TraitWriter.GetGraphReferencedObjects();
			GraphReferencedSoftObjects = TraitWriter.GetGraphReferencedSoftObjects();
		}

		// Read our graph
		FTestUtils::LoadFromArchiveBuffer(*AnimationGraph, NodeHandles, GraphSharedDataArchiveBuffer);

		TSharedPtr<FAnimNextGraphInstance> GraphInstance = AnimationGraph->AllocateInstance();

		FExecutionContext Context(*GraphInstance.Get());

		{
			FMemMark Mark(FMemStack::Get());

			FAnimNextTraitHandle RootHandle(NodeHandles[3]);	// Point to NodeD, first base trait

			FTraitPtr NodeDPtr = Context.AllocateNodeInstance(*GraphInstance.Get(), RootHandle);
			AddErrorIfFalse(NodeDPtr.IsValid(), "FAnimationAnimNextRuntimeTest_IHierarchy -> Failed to allocate root node instance");

			FTraitStackBinding StackNodeD;
			AddErrorIfFalse(Context.GetStack(NodeDPtr, StackNodeD), "FAnimationAnimNextRuntimeTest_IHierarchy -> Failed to bind to trait stack");

			TTraitBinding<IHierarchy> HierarchyBindingNodeD;
			AddErrorIfFalse(StackNodeD.GetInterface(HierarchyBindingNodeD), "FAnimationAnimNextRuntimeTest_IHierarchy -> IHierarchy not found");

			FChildrenArray ChildrenNodeD;
			HierarchyBindingNodeD.GetChildren(Context, ChildrenNodeD);

			AddErrorIfFalse(ChildrenNodeD.Num() == 2, "FAnimationAnimNextRuntimeTest_IHierarchy -> Expected 2 children");
			AddErrorIfFalse(HierarchyBindingNodeD.GetNumChildren(Context) == 2, "FAnimationAnimNextRuntimeTest_IHierarchy -> Expected 2 children");
			AddErrorIfFalse(ChildrenNodeD[0].IsValid() && ChildrenNodeD[0].GetNodeInstance()->GetNodeHandle() == NodeHandles[0], "FAnimationAnimNextRuntimeTest_IHierarchy -> Expected child: NodeA");
			AddErrorIfFalse(ChildrenNodeD[1].IsValid() && ChildrenNodeD[1].GetNodeInstance()->GetNodeHandle() == NodeHandles[2], "FAnimationAnimNextRuntimeTest_IHierarchy -> Expected child: NodeC");

			ChildrenNodeD.Reset();
			IHierarchy::GetStackChildren(Context, StackNodeD, ChildrenNodeD);

			AddErrorIfFalse(ChildrenNodeD.Num() == 2, "FAnimationAnimNextRuntimeTest_IHierarchy -> Expected 2 children");
			AddErrorIfFalse(IHierarchy::GetNumStackChildren(Context, StackNodeD) == 2, "FAnimationAnimNextRuntimeTest_IHierarchy -> Expected 2 children");
			AddErrorIfFalse(ChildrenNodeD[0].IsValid() && ChildrenNodeD[0].GetNodeInstance()->GetNodeHandle() == NodeHandles[0], "FAnimationAnimNextRuntimeTest_IHierarchy -> Expected child: NodeA");
			AddErrorIfFalse(ChildrenNodeD[1].IsValid() && ChildrenNodeD[1].GetNodeInstance()->GetNodeHandle() == NodeHandles[2], "FAnimationAnimNextRuntimeTest_IHierarchy -> Expected child: NodeC");

			{
				FTraitStackBinding StackNodeC;
				AddErrorIfFalse(Context.GetStack(ChildrenNodeD[1], StackNodeC), "FAnimationAnimNextRuntimeTest_IHierarchy -> Failed to bind to trait stack");

				TTraitBinding<IHierarchy> HierarchyBindingNodeC;
				AddErrorIfFalse(StackNodeC.GetInterface(HierarchyBindingNodeC), "FAnimationAnimNextRuntimeTest_IHierarchy -> IHierarchy not found");

				FChildrenArray ChildrenNodeC;
				HierarchyBindingNodeC.GetChildren(Context, ChildrenNodeC);

				AddErrorIfFalse(ChildrenNodeC.Num() == 1, "FAnimationAnimNextRuntimeTest_IHierarchy -> Expected 1 child");
				AddErrorIfFalse(HierarchyBindingNodeC.GetNumChildren(Context) == 1, "FAnimationAnimNextRuntimeTest_IHierarchy -> Expected 1 child");
				AddErrorIfFalse(ChildrenNodeC[0].IsValid() && ChildrenNodeC[0].GetNodeInstance()->GetNodeHandle() == NodeHandles[1], "FAnimationAnimNextRuntimeTest_IHierarchy -> Expected child: NodeB");

				ChildrenNodeC.Reset();
				IHierarchy::GetStackChildren(Context, StackNodeC, ChildrenNodeC);

				AddErrorIfFalse(ChildrenNodeC.Num() == 1, "FAnimationAnimNextRuntimeTest_IHierarchy -> Expected 1 child");
				AddErrorIfFalse(IHierarchy::GetNumStackChildren(Context, StackNodeC) == 1, "FAnimationAnimNextRuntimeTest_IHierarchy -> Expected 1 child");
				AddErrorIfFalse(ChildrenNodeC[0].IsValid() && ChildrenNodeC[0].GetNodeInstance()->GetNodeHandle() == NodeHandles[1], "FAnimationAnimNextRuntimeTest_IHierarchy -> Expected child: NodeB");

				{
					FTraitStackBinding StackNodeB;
					AddErrorIfFalse(Context.GetStack(ChildrenNodeC[0], StackNodeB), "FAnimationAnimNextRuntimeTest_IHierarchy -> Failed to bind to trait stack");

					TTraitBinding<IHierarchy> HierarchyBindingNodeB;
					AddErrorIfFalse(StackNodeB.GetInterface(HierarchyBindingNodeB), "FAnimationAnimNextRuntimeTest_IHierarchy -> IHierarchy not found");

					FChildrenArray ChildrenNodeB;
					HierarchyBindingNodeB.GetChildren(Context, ChildrenNodeB);

					AddErrorIfFalse(ChildrenNodeB.Num() == 1, "FAnimationAnimNextRuntimeTest_IHierarchy -> Expected 1 child");
					AddErrorIfFalse(HierarchyBindingNodeB.GetNumChildren(Context) == 1, "FAnimationAnimNextRuntimeTest_IHierarchy -> Expected 1 child");
					AddErrorIfFalse(ChildrenNodeB[0].IsValid() && ChildrenNodeB[0].GetNodeInstance()->GetNodeHandle() == NodeHandles[0], "FAnimationAnimNextRuntimeTest_IHierarchy -> Expected child: NodeA");

					ChildrenNodeB.Reset();
					IHierarchy::GetStackChildren(Context, StackNodeB, ChildrenNodeB);

					AddErrorIfFalse(ChildrenNodeB.Num() == 1, "FAnimationAnimNextRuntimeTest_IHierarchy -> Expected 1 child");
					AddErrorIfFalse(IHierarchy::GetNumStackChildren(Context, StackNodeB) == 1, "FAnimationAnimNextRuntimeTest_IHierarchy -> Expected 1 child");
					AddErrorIfFalse(ChildrenNodeB[0].IsValid() && ChildrenNodeB[0].GetNodeInstance()->GetNodeHandle() == NodeHandles[0], "FAnimationAnimNextRuntimeTest_IHierarchy -> Expected child: NodeA");
				}
			}
		}
	}

	Tests::FUtils::CleanupAfterTests();
	
	return true;
}

// --- Trait Interfaces IUpdate Test ---

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnimationAnimNextRuntimeTest_IUpdate, "Animation.AnimNext.Runtime.IUpdate", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAnimationAnimNextRuntimeTest_IUpdate::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;
	{
		AUTO_REGISTER_ANIM_TRAIT(FTraitWithNoChildren)
		AUTO_REGISTER_ANIM_TRAIT(FTraitWithOneChild)
		AUTO_REGISTER_ANIM_TRAIT(FTraitWithChildren)

		UFactory* GraphFactory = NewObject<UUAFAnimGraphFactory>();
		UUAFAnimGraph* AnimationGraph = CastChecked<UUAFAnimGraph>(GraphFactory->FactoryCreateNew(UUAFAnimGraph::StaticClass(), GetTransientPackage(), TEXT("TestAnimNextGraph"), RF_Transient, nullptr, nullptr, NAME_None));
		UE_RETURN_ON_ERROR(AnimationGraph != nullptr, "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Failed to create animation graph");

		FScopedClearNodeTemplateRegistry ScopedClearNodeTemplateRegistry;
		FNodeTemplateRegistry& Registry = FNodeTemplateRegistry::Get();

		// We create a few node templates
		// Template A has a single trait with no children
		TArray<FTraitUID> NodeTemplateTraitListA;
		NodeTemplateTraitListA.Add(FTraitWithNoChildren::TraitUID);

		// Template B has a single trait with one child, it doesn't update
		TArray<FTraitUID> NodeTemplateTraitListB;
		NodeTemplateTraitListB.Add(FTraitWithOneChild::TraitUID);

		// Template C has a single trait with children
		TArray<FTraitUID> NodeTemplateTraitListC;
		NodeTemplateTraitListC.Add(FTraitWithChildren::TraitUID);

		// Populate our node template registry
		TArray<uint8> NodeTemplateBufferA, NodeTemplateBufferB, NodeTemplateBufferC;
		const FNodeTemplate* NodeTemplateA = FNodeTemplateBuilder::BuildNodeTemplate(NodeTemplateTraitListA, NodeTemplateBufferA);
		const FNodeTemplate* NodeTemplateB = FNodeTemplateBuilder::BuildNodeTemplate(NodeTemplateTraitListB, NodeTemplateBufferB);
		const FNodeTemplate* NodeTemplateC = FNodeTemplateBuilder::BuildNodeTemplate(NodeTemplateTraitListC, NodeTemplateBufferC);

		// Build our graph, it as follow (each node template has a single node instance):
		// NodeA has no children
		// NodeB has one child: NodeA (it doesn't update)
		// NodeC (root) has two children: NodeA and NodeB

		TArray<FNodeHandle> NodeHandles;

		// Write our graph
		TArray<uint8> GraphSharedDataArchiveBuffer;
		TArray<TObjectPtr<UObject>> GraphReferencedObjects;
		TArray<FSoftObjectPath> GraphReferencedSoftObjects;
		{
			FTraitWriter TraitWriter;

			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplateC));	// Root node
			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplateA));
			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplateB));

			// We don't have trait properties
			TArray<TMap<FName, FString>> TraitPropertiesA;
			TraitPropertiesA.AddDefaulted(NodeTemplateTraitListA.Num());

			TArray<TMap<FName, FString>> TraitPropertiesB;
			TraitPropertiesB.AddDefaulted(NodeTemplateTraitListB.Num());
			TraitPropertiesB[0].Add(TEXT("Child"), ToString<FTraitWithOneChild::FSharedData>(TEXT("Child"), FAnimNextTraitHandle(NodeHandles[1])));

			TArray<TMap<FName, FString>> TraitPropertiesC;
			TraitPropertiesC.AddDefaulted(NodeTemplateTraitListC.Num());
			FAnimNextTraitHandle ChildrenHandlesC[2] = { FAnimNextTraitHandle(NodeHandles[1]), FAnimNextTraitHandle(NodeHandles[2])};
			TraitPropertiesC[0].Add(TEXT("Children"), ToString<FTraitWithChildren::FSharedData>(TEXT("Children"), ChildrenHandlesC));

			TraitWriter.BeginNodeWriting();
			TraitWriter.WriteNode(NodeHandles[0],
				[&TraitPropertiesC](uint32 TraitIndex, FName PropertyName)
				{
					return TraitPropertiesC[TraitIndex][PropertyName];
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.WriteNode(NodeHandles[1],
				[&TraitPropertiesA](uint32 TraitIndex, FName PropertyName)
				{
					return TraitPropertiesA[TraitIndex][PropertyName];
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.WriteNode(NodeHandles[2],
				[&TraitPropertiesB](uint32 TraitIndex, FName PropertyName)
				{
					return TraitPropertiesB[TraitIndex][PropertyName];
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.EndNodeWriting();

			AddErrorIfFalse(TraitWriter.GetErrorState() == FTraitWriter::EErrorState::None, "FAnimationAnimNextRuntimeTest_IUpdate -> Failed to write traits");
			GraphSharedDataArchiveBuffer = TraitWriter.GetGraphSharedData();
			GraphReferencedObjects = TraitWriter.GetGraphReferencedObjects();
			GraphReferencedSoftObjects = TraitWriter.GetGraphReferencedSoftObjects();
		}

		// Read our graph
		FTestUtils::LoadFromArchiveBuffer(*AnimationGraph, NodeHandles, GraphSharedDataArchiveBuffer);

		TSharedPtr<FAnimNextGraphInstance> GraphInstance = AnimationGraph->AllocateInstance();

		FExecutionContext Context(*GraphInstance.Get());

		{
			TArray<FTraitUID> UpdatedTraits;
			Private::UpdatedTraits = &UpdatedTraits;

			// Call pre/post update on our graph
			FUpdateGraphContext UpdateGraphContext(*GraphInstance.Get(), 0.0333f);
			UpdateGraph(UpdateGraphContext);

			AddErrorIfFalse(UpdatedTraits.Num() == 6, "FAnimationAnimNextRuntimeTest_IUpdate -> Expected 6 nodes to have been visited during the update traversal");
			AddErrorIfFalse(UpdatedTraits[0] == FTraitWithChildren::TraitUID, "FAnimationAnimNextRuntimeTest_IUpdate -> Unexpected update order");		// NodeC
			AddErrorIfFalse(UpdatedTraits[1] == FTraitWithNoChildren::TraitUID, "FAnimationAnimNextRuntimeTest_IUpdate -> Unexpected update order");		// NodeA
			AddErrorIfFalse(UpdatedTraits[2] == FTraitWithNoChildren::TraitUID, "FAnimationAnimNextRuntimeTest_IUpdate -> Unexpected update order");		// NodeA
			AddErrorIfFalse(UpdatedTraits[3] == FTraitWithNoChildren::TraitUID, "FAnimationAnimNextRuntimeTest_IUpdate -> Unexpected update order");		// NodeB -> NodeA
			AddErrorIfFalse(UpdatedTraits[4] == FTraitWithNoChildren::TraitUID, "FAnimationAnimNextRuntimeTest_IUpdate -> Unexpected update order");		// NodeB -> NodeA
			AddErrorIfFalse(UpdatedTraits[5] == FTraitWithChildren::TraitUID, "FAnimationAnimNextRuntimeTest_IUpdate -> Unexpected update order");		// NodeC

			Private::UpdatedTraits = nullptr;
		}
	}
	Tests::FUtils::CleanupAfterTests();

	return true;
}

// --- Trait Interfaces IEvaluate Test ---

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnimationAnimNextRuntimeTest_IEvaluate, "Animation.AnimNext.Runtime.IEvaluate", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAnimationAnimNextRuntimeTest_IEvaluate::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;
	{
		AUTO_REGISTER_ANIM_TRAIT(FTraitWithNoChildren)
		AUTO_REGISTER_ANIM_TRAIT(FTraitWithOneChild)
		AUTO_REGISTER_ANIM_TRAIT(FTraitWithChildren)

		UFactory* GraphFactory = NewObject<UUAFAnimGraphFactory>();
		UUAFAnimGraph* AnimationGraph = CastChecked<UUAFAnimGraph>(GraphFactory->FactoryCreateNew(UUAFAnimGraph::StaticClass(), GetTransientPackage(), TEXT("TestAnimNextGraph"), RF_Transient, nullptr, nullptr, NAME_None));
		UE_RETURN_ON_ERROR(AnimationGraph != nullptr, "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Failed to create animation graph");

		FScopedClearNodeTemplateRegistry ScopedClearNodeTemplateRegistry;
		FNodeTemplateRegistry& Registry = FNodeTemplateRegistry::Get();

		// We create a few node templates
		// Template A has a single trait with no children
		TArray<FTraitUID> NodeTemplateTraitListA;
		NodeTemplateTraitListA.Add(FTraitWithNoChildren::TraitUID);

		// Template B has a single trait with one child, it doesn't evaluate
		TArray<FTraitUID> NodeTemplateTraitListB;
		NodeTemplateTraitListB.Add(FTraitWithOneChild::TraitUID);

		// Template C has a single trait with children
		TArray<FTraitUID> NodeTemplateTraitListC;
		NodeTemplateTraitListC.Add(FTraitWithChildren::TraitUID);

		// Populate our node template registry
		TArray<uint8> NodeTemplateBufferA, NodeTemplateBufferB, NodeTemplateBufferC;
		const FNodeTemplate* NodeTemplateA = FNodeTemplateBuilder::BuildNodeTemplate(NodeTemplateTraitListA, NodeTemplateBufferA);
		const FNodeTemplate* NodeTemplateB = FNodeTemplateBuilder::BuildNodeTemplate(NodeTemplateTraitListB, NodeTemplateBufferB);
		const FNodeTemplate* NodeTemplateC = FNodeTemplateBuilder::BuildNodeTemplate(NodeTemplateTraitListC, NodeTemplateBufferC);

		// Build our graph, it as follow (each node template has a single node instance):
		// NodeA has no children
		// NodeB has one child: NodeA (it doesn't evaluate)
		// NodeC (root) has two children: NodeA and NodeB

		TArray<FNodeHandle> NodeHandles;

		// Write our graph
		TArray<uint8> GraphSharedDataArchiveBuffer;
		TArray<TObjectPtr<UObject>> GraphReferencedObjects;
		TArray<FSoftObjectPath> GraphReferencedSoftObjects;
		{
			FTraitWriter TraitWriter;

			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplateC));	// Root node
			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplateA));
			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplateB));

			// We don't have trait properties
			TArray<TMap<FName, FString>> TraitPropertiesA;
			TraitPropertiesA.AddDefaulted(NodeTemplateTraitListA.Num());

			TArray<TMap<FName, FString>> TraitPropertiesB;
			TraitPropertiesB.AddDefaulted(NodeTemplateTraitListB.Num());
			TraitPropertiesB[0].Add(TEXT("Child"), ToString<FTraitWithOneChild::FSharedData>(TEXT("Child"), FAnimNextTraitHandle(NodeHandles[1])));

			TArray<TMap<FName, FString>> TraitPropertiesC;
			TraitPropertiesC.AddDefaulted(NodeTemplateTraitListC.Num());

			FAnimNextTraitHandle ChildrenHandlesC[2] = { FAnimNextTraitHandle(NodeHandles[1]), FAnimNextTraitHandle(NodeHandles[2])};
			TraitPropertiesC[0].Add(TEXT("Children"), ToString<FTraitWithChildren::FSharedData>(TEXT("Children"), ChildrenHandlesC));

			TraitWriter.BeginNodeWriting();
			TraitWriter.WriteNode(NodeHandles[0],
				[&TraitPropertiesC](uint32 TraitIndex, FName PropertyName)
				{
					return TraitPropertiesC[TraitIndex][PropertyName];
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.WriteNode(NodeHandles[1],
				[&TraitPropertiesA](uint32 TraitIndex, FName PropertyName)
				{
					return TraitPropertiesA[TraitIndex][PropertyName];
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.WriteNode(NodeHandles[2],
				[&TraitPropertiesB](uint32 TraitIndex, FName PropertyName)
				{
					return TraitPropertiesB[TraitIndex][PropertyName];
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.EndNodeWriting();

			AddErrorIfFalse(TraitWriter.GetErrorState() == FTraitWriter::EErrorState::None, "FAnimationAnimNextRuntimeTest_IEvaluate -> Failed to write traits");
			GraphSharedDataArchiveBuffer = TraitWriter.GetGraphSharedData();
			GraphReferencedObjects = TraitWriter.GetGraphReferencedObjects();
			GraphReferencedSoftObjects = TraitWriter.GetGraphReferencedSoftObjects();
		}

		// Read our graph
		FTestUtils::LoadFromArchiveBuffer(*AnimationGraph, NodeHandles, GraphSharedDataArchiveBuffer);

		TSharedPtr<FAnimNextGraphInstance> GraphInstance = AnimationGraph->AllocateInstance();

		{
			TArray<FTraitUID> EvaluatedTraits;
			Private::EvaluatedTraits = &EvaluatedTraits;

			// Call pre/post evaluate on our graph
			UE::UAF::FEvaluateGraphContext EvaluateGraphContext(*GraphInstance.Get(), UE::UAF::FReferencePose(), 0);
			(void)EvaluateGraph(EvaluateGraphContext);

			AddErrorIfFalse(EvaluatedTraits.Num() == 6, "FAnimationAnimNextRuntimeTest_IEvaluate -> Expected 6 nodes to have been visited during the evaluate traversal");
			AddErrorIfFalse(EvaluatedTraits[0] == FTraitWithChildren::TraitUID, "FAnimationAnimNextRuntimeTest_IEvaluate -> Unexpected evaluate order");		// NodeC
			AddErrorIfFalse(EvaluatedTraits[1] == FTraitWithNoChildren::TraitUID, "FAnimationAnimNextRuntimeTest_IEvaluate -> Unexpected evaluate order");			// NodeA
			AddErrorIfFalse(EvaluatedTraits[2] == FTraitWithNoChildren::TraitUID, "FAnimationAnimNextRuntimeTest_IEvaluate -> Unexpected evaluate order");			// NodeA
			AddErrorIfFalse(EvaluatedTraits[3] == FTraitWithNoChildren::TraitUID, "FAnimationAnimNextRuntimeTest_IEvaluate -> Unexpected evaluate order");			// NodeB -> NodeA
			AddErrorIfFalse(EvaluatedTraits[4] == FTraitWithNoChildren::TraitUID, "FAnimationAnimNextRuntimeTest_IEvaluate -> Unexpected evaluate order");			// NodeB -> NodeA
			AddErrorIfFalse(EvaluatedTraits[5] == FTraitWithChildren::TraitUID, "FAnimationAnimNextRuntimeTest_IEvaluate -> Unexpected evaluate order");		// NodeC

			Private::EvaluatedTraits = nullptr;
		}
	}
	Tests::FUtils::CleanupAfterTests();

	return true;
}

// --- Trait Interfaces IScopedInterface Test ---

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnimationAnimNextRuntimeTest_IScopedInterface, "Animation.AnimNext.Runtime.IScopedInterface", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAnimationAnimNextRuntimeTest_IScopedInterface::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;
	{
		AUTO_REGISTER_ANIM_TRAIT(FTraitWithOneChild)
		AUTO_REGISTER_ANIM_TRAIT(FScopedTagTrait)
		AUTO_REGISTER_ANIM_TRAIT(FTestScopedTagTrait)

		UFactory* GraphFactory = NewObject<UUAFAnimGraphFactory>();
		UUAFAnimGraph* AnimationGraph = CastChecked<UUAFAnimGraph>(GraphFactory->FactoryCreateNew(UUAFAnimGraph::StaticClass(), GetTransientPackage(), TEXT("TestAnimNextGraph"), RF_Transient, nullptr, nullptr, NAME_None));
		UE_RETURN_ON_ERROR(AnimationGraph != nullptr, "FAnimationAnimNextRuntimeTest_GraphTraitEvent -> Failed to create animation graph");

		FScopedClearNodeTemplateRegistry ScopedClearNodeTemplateRegistry;
		FNodeTemplateRegistry& Registry = FNodeTemplateRegistry::Get();

		// We create a few node templates
		// Template A has a single child and tests for our tag
		TArray<FTraitUID> NodeTemplateTraitList0;
		NodeTemplateTraitList0.Add(FTraitWithOneChild::TraitUID);
		NodeTemplateTraitList0.Add(FTestScopedTagTrait::TraitUID);

		// Template B has a single child, it tests and pushes our tag
		TArray<FTraitUID> NodeTemplateTraitList1;
		NodeTemplateTraitList1.Add(FTraitWithOneChild::TraitUID);
		NodeTemplateTraitList1.Add(FTestScopedTagTrait::TraitUID);	// Test after push/pop
		NodeTemplateTraitList1.Add(FScopedTagTrait::TraitUID);
		NodeTemplateTraitList1.Add(FTestScopedTagTrait::TraitUID);	// Test before push/pop

		// Populate our node template registry
		TArray<uint8> NodeTemplateBuffer0, NodeTemplateBuffer1;
		const FNodeTemplate* NodeTemplate0 = FNodeTemplateBuilder::BuildNodeTemplate(NodeTemplateTraitList0, NodeTemplateBuffer0);
		const FNodeTemplate* NodeTemplate1 = FNodeTemplateBuilder::BuildNodeTemplate(NodeTemplateTraitList1, NodeTemplateBuffer1);

		// Build our graph, it as follow:
		// NodeA has no child (tag is scoped)
		// NodeB has one child: NodeA (NodeB adds the scoped tag)
		// NodeC (root) has one child: NodeB (no tag scoped)

		TArray<FNodeHandle> NodeHandles;

		// Write our graph
		TArray<uint8> GraphSharedDataArchiveBuffer;
		TArray<TObjectPtr<UObject>> GraphReferencedObjects;
		TArray<FSoftObjectPath> GraphReferencedSoftObjects;
		{
			FTraitWriter TraitWriter;

			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplate0));	// NodeC (root node)
			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplate1));	// NodeB
			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplate0));	// NodeA

			// We don't have trait properties
			TArray<TMap<FName, FString>> TraitPropertiesA;
			TraitPropertiesA.AddDefaulted(NodeTemplateTraitList0.Num());
			TraitPropertiesA[0].Add(TEXT("Child"), ToString<FTraitWithOneChild::FSharedData>(TEXT("Child"), FAnimNextTraitHandle()));

			TArray<TMap<FName, FString>> TraitPropertiesB;
			TraitPropertiesB.AddDefaulted(NodeTemplateTraitList1.Num());
			TraitPropertiesB[0].Add(TEXT("Child"), ToString<FTraitWithOneChild::FSharedData>(TEXT("Child"), FAnimNextTraitHandle(NodeHandles[2])));

			TArray<TMap<FName, FString>> TraitPropertiesC;
			TraitPropertiesC.AddDefaulted(NodeTemplateTraitList0.Num());
			TraitPropertiesC[0].Add(TEXT("Child"), ToString<FTraitWithOneChild::FSharedData>(TEXT("Child"), FAnimNextTraitHandle(NodeHandles[1])));

			TraitWriter.BeginNodeWriting();
			TraitWriter.WriteNode(NodeHandles[0],
				[&TraitPropertiesC](uint32 TraitIndex, FName PropertyName)
				{
					return TraitPropertiesC[TraitIndex][PropertyName];
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.WriteNode(NodeHandles[1],
				[&TraitPropertiesB](uint32 TraitIndex, FName PropertyName)
				{
					return TraitPropertiesB[TraitIndex][PropertyName];
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.WriteNode(NodeHandles[2],
				[&TraitPropertiesA](uint32 TraitIndex, FName PropertyName)
				{
					return TraitPropertiesA[TraitIndex][PropertyName];
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.EndNodeWriting();

			AddErrorIfFalse(TraitWriter.GetErrorState() == FTraitWriter::EErrorState::None, "FAnimationAnimNextRuntimeTest_IScopedInterface -> Failed to write traits");
			GraphSharedDataArchiveBuffer = TraitWriter.GetGraphSharedData();
			GraphReferencedObjects = TraitWriter.GetGraphReferencedObjects();
			GraphReferencedSoftObjects = TraitWriter.GetGraphReferencedSoftObjects();
		}

		// Read our graph
		FTestUtils::LoadFromArchiveBuffer(*AnimationGraph, NodeHandles, GraphSharedDataArchiveBuffer);

		TSharedPtr<FAnimNextGraphInstance> GraphInstance = AnimationGraph->AllocateInstance();

		FExecutionContext Context(*GraphInstance.Get());

		{
			TArray<bool> IsTagInScope;
			Private::IsTagInScope = &IsTagInScope;

			// Call pre/post update on our graph
			Private::AutoPopTag = true;
			FUpdateGraphContext UpdateGraphContext(*GraphInstance.Get(), 0.0333f);
			UpdateGraph(UpdateGraphContext);

			AddErrorIfFalse(IsTagInScope.Num() == 8, "FAnimationAnimNextRuntimeTest_IScopedInterface -> Unexpected number of entries");
			AddErrorIfFalse(IsTagInScope[0] == false, "FAnimationAnimNextRuntimeTest_IScopedInterface -> Unexpected scoped tag state");		// NodeC::PreUpdate (template 0)
			AddErrorIfFalse(IsTagInScope[1] == false, "FAnimationAnimNextRuntimeTest_IScopedInterface -> Unexpected scoped tag state");		// NodeB::Before::PreUpdate (template 1)
			AddErrorIfFalse(IsTagInScope[2] == true, "FAnimationAnimNextRuntimeTest_IScopedInterface -> Unexpected scoped tag state");		// NodeB::After::PreUpdate (template 1)
			AddErrorIfFalse(IsTagInScope[3] == true, "FAnimationAnimNextRuntimeTest_IScopedInterface -> Unexpected scoped tag state");		// NodeA::PreUpdate (template 0)
			AddErrorIfFalse(IsTagInScope[4] == true, "FAnimationAnimNextRuntimeTest_IScopedInterface -> Unexpected scoped tag state");		// NodeA::PostUpdate (template 0)
			AddErrorIfFalse(IsTagInScope[5] == true, "FAnimationAnimNextRuntimeTest_IScopedInterface -> Unexpected scoped tag state");		// NodeB::Before::PostUpdate (template 1)
			AddErrorIfFalse(IsTagInScope[6] == true, "FAnimationAnimNextRuntimeTest_IScopedInterface -> Unexpected scoped tag state");		// NodeB::After::PostUpdate (template 1)
			AddErrorIfFalse(IsTagInScope[7] == false, "FAnimationAnimNextRuntimeTest_IScopedInterface -> Unexpected scoped tag state");		// NodeC::PostUpdate (template 0)

			IsTagInScope.Reset();
			Private::AutoPopTag = false;

			// Call pre/post update on our graph
			UpdateGraph(UpdateGraphContext);

			AddErrorIfFalse(IsTagInScope.Num() == 8, "FAnimationAnimNextRuntimeTest_IScopedInterface -> Unexpected number of entries");
			AddErrorIfFalse(IsTagInScope[0] == false, "FAnimationAnimNextRuntimeTest_IScopedInterface -> Unexpected scoped tag state");		// NodeC::PreUpdate (template 0)
			AddErrorIfFalse(IsTagInScope[1] == false, "FAnimationAnimNextRuntimeTest_IScopedInterface -> Unexpected scoped tag state");		// NodeB::Before::PreUpdate (template 1)
			AddErrorIfFalse(IsTagInScope[2] == true, "FAnimationAnimNextRuntimeTest_IScopedInterface -> Unexpected scoped tag state");		// NodeB::After::PreUpdate (template 1)
			AddErrorIfFalse(IsTagInScope[3] == true, "FAnimationAnimNextRuntimeTest_IScopedInterface -> Unexpected scoped tag state");		// NodeA::PreUpdate (template 0)
			AddErrorIfFalse(IsTagInScope[4] == true, "FAnimationAnimNextRuntimeTest_IScopedInterface -> Unexpected scoped tag state");		// NodeA::PostUpdate (template 0)
			AddErrorIfFalse(IsTagInScope[5] == true, "FAnimationAnimNextRuntimeTest_IScopedInterface -> Unexpected scoped tag state");		// NodeB::Before::PostUpdate (template 1)
			AddErrorIfFalse(IsTagInScope[6] == false, "FAnimationAnimNextRuntimeTest_IScopedInterface -> Unexpected scoped tag state");		// NodeB::After::PostUpdate (template 1)
			AddErrorIfFalse(IsTagInScope[7] == false, "FAnimationAnimNextRuntimeTest_IScopedInterface -> Unexpected scoped tag state");		// NodeC::PostUpdate (template 0)

			Private::UpdatedTraits = nullptr;
		}
	}
	Tests::FUtils::CleanupAfterTests();

	return true;
}

#endif
