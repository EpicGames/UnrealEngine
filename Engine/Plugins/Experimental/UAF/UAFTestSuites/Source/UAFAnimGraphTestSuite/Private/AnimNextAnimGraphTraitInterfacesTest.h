// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/TraitHandle.h"
#include "TraitCore/TraitSharedData.h"

#include "AnimNextAnimGraphTraitInterfacesTest.generated.h"

USTRUCT()
struct FTraitWithOneChildSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY()
	FAnimNextTraitHandle Child;
};

USTRUCT()
struct FTraitWithChildrenSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY()
	FAnimNextTraitHandle Children[2];
};
