// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "AnimNextInlineSubGraphTraitTest.h"

#include "AnimNextRuntimeTest.h"
#include "AnimNextTest.h"
#include "CQTest.h"

#include "TraitCore/Trait.h"
#include "TraitCore/TraitRegistry.h"
#include "TraitCore/TraitWriter.h"
#include "TraitCore/ExecutionContext.h"
#include "TraitCore/NodeInstance.h"
#include "TraitCore/NodeTemplateBuilder.h"
#include "TraitCore/NodeTemplateRegistry.h"
#include "TraitInterfaces/IDiscreteBlend.h"
#include "TraitInterfaces/IHierarchy.h"
#include "TraitInterfaces/IUpdate.h"
#include "Factory/AnimGraphFactory.h"
#include "Factory/AnimNextFactoryParams.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextAnimationGraphFactory.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Traits/InlineSubGraphTraitData.h"

//****************************************************************************
// AnimNext Inline SubGraph Trait Tests
//****************************************************************************

namespace UE::UAF
{
	// --- Leaf trait that tracks update calls for verifying sub-graph traversal ---
	struct FInlineSubGraphTest_LeafTrait final : FBaseTrait, IUpdate
	{
		DECLARE_ANIM_TRAIT(FInlineSubGraphTest_LeafTrait, FBaseTrait)

		using FSharedData = FInlineSubGraphTest_LeafTraitSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			int32 PreUpdateCount = 0;
			int32 PostUpdateCount = 0;
		};

		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override
		{
			FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
			InstanceData->PreUpdateCount++;
			IUpdate::PreUpdate(Context, Binding, TraitState);
		}

		virtual void PostUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override
		{
			FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
			InstanceData->PostUpdateCount++;
			IUpdate::PostUpdate(Context, Binding, TraitState);
		}
	};

	#define TRAIT_INTERFACE_ENUMERATOR_LEAF(GeneratorMacro) \
		GeneratorMacro(IUpdate) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FInlineSubGraphTest_LeafTrait, TRAIT_INTERFACE_ENUMERATOR_LEAF, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR_LEAF
}

TEST_CLASS(FInlineSubGraphTraitTest, "Animation.AnimNext.Runtime.InlineSubGraphTrait")
{
	UE::UAF::FTraitUID InlineSubGraphUID;

	BEFORE_EACH()
	{
		const UE::UAF::FTraitRegistry& Registry = UE::UAF::FTraitRegistry::Get();
		const UE::UAF::FTrait* Trait = Registry.Find(FUAFInlineSubGraphTraitSharedData::StaticStruct());
		ASSERT_THAT(IsNotNull(Trait, "InlineSubGraph trait must be registered"));
		InlineSubGraphUID = Trait->GetTraitUID();
	}

	AFTER_EACH()
	{
		UE::UAF::Tests::FUtils::CleanupAfterTests();
	}

	// --- Helper: build a graph containing the inline subgraph trait via FAnimGraphFactory ---
	const UUAFAnimGraph* BuildInlineSubGraphGraph(TObjectPtr<const UUAFAnimGraph> InnerGraph = nullptr)
	{
		using namespace UE::UAF;

		FAnimNextFactoryParams Params;
		Params.AddTraitStruct<FUAFInlineSubGraphTraitSharedData>(ETraitVariableMapping::None, 0);

		if (InnerGraph)
		{
			Params.AccessTraitStruct<FUAFInlineSubGraphTraitSharedData>(0,
				[&InnerGraph](FUAFInlineSubGraphTraitSharedData& SharedData)
				{
					SharedData.Graph = InnerGraph;
				});
		}

		return FAnimGraphFactory::BuildGraph(Params.GetBuilder());
	}

	// --- Helper: build a simple graph with a single leaf trait ---
	const UUAFAnimGraph* BuildLeafGraph()
	{
		using namespace UE::UAF;

		FAnimNextFactoryParams Params;
		Params.AddTraitStruct<FAnimNextTraitSharedData>(ETraitVariableMapping::None, 0);

		return FAnimGraphFactory::BuildGraph(Params.GetBuilder());
	}

	// --- Test: SharedData struct properties ---
	TEST_METHOD(SharedData_Has_Expected_Properties)
	{
		const UScriptStruct* SharedDataStruct = FUAFInlineSubGraphTraitSharedData::StaticStruct();
		ASSERT_THAT(IsNotNull(SharedDataStruct));

		ASSERT_THAT(IsNotNull(SharedDataStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FUAFInlineSubGraphTraitSharedData, Graph))));
		ASSERT_THAT(IsNotNull(SharedDataStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FUAFInlineSubGraphTraitSharedData, ReferencePoseChild))));

		const FProperty* InputsProperty = SharedDataStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FUAFInlineSubGraphTraitSharedData, Inputs));
		ASSERT_THAT(IsNotNull(InputsProperty));
		ASSERT_THAT(IsNotNull(CastField<FArrayProperty>(InputsProperty), "Inputs should be an array property"));
	}

	// --- Test: InputBinding struct properties ---
	TEST_METHOD(InputBinding_Has_Expected_Properties)
	{
		const UScriptStruct* BindingStruct = FUAFInlineSubGraphInputBinding::StaticStruct();
		ASSERT_THAT(IsNotNull(BindingStruct));

		ASSERT_THAT(IsNotNull(BindingStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FUAFInlineSubGraphInputBinding, Input))));
		ASSERT_THAT(IsNotNull(BindingStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FUAFInlineSubGraphInputBinding, Variable))));
	}

	// --- Test: Trait registration and interface implementation ---
	TEST_METHOD(Registration_Has_Expected_Interfaces)
	{
		using namespace UE::UAF;

		const FTraitRegistry& Registry = FTraitRegistry::Get();
		const FTrait* Trait = Registry.Find(InlineSubGraphUID);
		ASSERT_THAT(IsNotNull(Trait));

		ASSERT_THAT(AreEqual(InlineSubGraphUID.GetUID(), Trait->GetTraitUID().GetUID()));
		ASSERT_THAT(AreEqual(FUAFInlineSubGraphTraitSharedData::StaticStruct(), Trait->GetTraitSharedDataStruct()));
		ASSERT_THAT(AreEqual(ETraitMode::Base, Trait->GetTraitMode()));

		ASSERT_THAT(IsNotNull(Trait->GetTraitInterface(IUpdate::InterfaceUID), "Should implement IUpdate"));
		ASSERT_THAT(IsNotNull(Trait->GetTraitInterface(IHierarchy::InterfaceUID), "Should implement IHierarchy"));
		ASSERT_THAT(IsNotNull(Trait->GetTraitInterface(IDiscreteBlend::InterfaceUID), "Should implement IDiscreteBlend"));
		ASSERT_THAT(IsNotNull(Trait->GetTraitInterface(IUpdateTraversal::InterfaceUID), "Should implement IUpdateTraversal"));
	}

	// --- Test: Default state with no graph set ---
	TEST_METHOD(Defaults_No_Graph_Has_No_Children)
	{
		using namespace UE::UAF;

		const UUAFAnimGraph* AnimGraph = BuildInlineSubGraphGraph();
		ASSERT_THAT(IsNotNull(AnimGraph));

		TSharedPtr<FAnimNextGraphInstance> GraphInstance = AnimGraph->AllocateInstance();
		ASSERT_THAT(IsTrue(GraphInstance.IsValid()));

		FExecutionContext Context(*GraphInstance.Get());
		FMemMark Mark(FMemStack::Get());

		FTraitStackBinding StackBinding;
		ASSERT_THAT(IsTrue(Context.GetStack(GraphInstance->GetGraphRootPtr(), StackBinding)));

		TTraitBinding<IHierarchy> HierarchyBinding;
		ASSERT_THAT(IsTrue(StackBinding.GetInterface(HierarchyBinding)));
		ASSERT_THAT(AreEqual(0u, HierarchyBinding.GetNumChildren(Context)));

		TTraitBinding<IDiscreteBlend> BlendBinding;
		ASSERT_THAT(IsTrue(StackBinding.GetInterface(BlendBinding)));
		ASSERT_THAT(AreEqual(INDEX_NONE, BlendBinding.GetBlendDestinationChildIndex(Context)));
	}

	// --- Test: Blend weight for invalid child index ---
	TEST_METHOD(BlendWeight_Invalid_Index_Returns_Negative)
	{
		using namespace UE::UAF;

		const UUAFAnimGraph* AnimGraph = BuildInlineSubGraphGraph();
		ASSERT_THAT(IsNotNull(AnimGraph));

		TSharedPtr<FAnimNextGraphInstance> GraphInstance = AnimGraph->AllocateInstance();
		FExecutionContext Context(*GraphInstance.Get());
		FMemMark Mark(FMemStack::Get());

		FTraitStackBinding StackBinding;
		ASSERT_THAT(IsTrue(Context.GetStack(GraphInstance->GetGraphRootPtr(), StackBinding)));

		TTraitBinding<IDiscreteBlend> BlendBinding;
		ASSERT_THAT(IsTrue(StackBinding.GetInterface(BlendBinding)));

		const float InvalidWeight = BlendBinding.GetBlendWeight(Context, 999);
		ASSERT_THAT(IsTrue(InvalidWeight < 0.0f, "Blend weight for invalid child index should be negative"));
	}

	// --- Test: Inner graph hosting after update ---
	TEST_METHOD(InnerGraph_Hosting_Creates_Child_After_Update)
	{
		using namespace UE::UAF;

		AUTO_REGISTER_ANIM_TRAIT(FInlineSubGraphTest_LeafTrait)

		// Build inner graph with a leaf trait via the factory
		FAnimNextFactoryParams InnerParams;
		InnerParams.AddTraitStruct<FInlineSubGraphTest_LeafTraitSharedData>(ETraitVariableMapping::None, 0);
		const UUAFAnimGraph* InnerGraph = FAnimGraphFactory::BuildGraph(InnerParams.GetBuilder());
		ASSERT_THAT(IsNotNull(InnerGraph));

		// Build outer graph pointing to the inner graph
		const UUAFAnimGraph* OuterGraph = BuildInlineSubGraphGraph(InnerGraph);
		ASSERT_THAT(IsNotNull(OuterGraph));

		TSharedPtr<FAnimNextGraphInstance> GraphInstance = OuterGraph->AllocateInstance();
		ASSERT_THAT(IsTrue(GraphInstance.IsValid()));

		// Run update to trigger PreUpdate which activates the inner graph
		FUpdateGraphContext UpdateGraphContext(*GraphInstance.Get(), 0.0333f);
		UpdateGraph(UpdateGraphContext);

		FExecutionContext Context(*GraphInstance.Get());
		FTraitStackBinding StackBinding;
		ASSERT_THAT(IsTrue(Context.GetStack(GraphInstance->GetGraphRootPtr(), StackBinding)));

		TTraitBinding<IHierarchy> HierarchyBinding;
		ASSERT_THAT(IsTrue(StackBinding.GetInterface(HierarchyBinding)));
		ASSERT_THAT(AreEqual(1u, HierarchyBinding.GetNumChildren(Context), "Should have 1 child (the inner graph root)"));

		TTraitBinding<IDiscreteBlend> BlendBinding;
		ASSERT_THAT(IsTrue(StackBinding.GetInterface(BlendBinding)));

		const int32 DestIndex = BlendBinding.GetBlendDestinationChildIndex(Context);
		ASSERT_THAT(IsTrue(DestIndex != INDEX_NONE, "Destination child index should be valid"));

		if (DestIndex != INDEX_NONE)
		{
			const float ActiveWeight = BlendBinding.GetBlendWeight(Context, DestIndex);
			ASSERT_THAT(IsNear(1.0f, ActiveWeight, UE_SMALL_NUMBER, "Active child should have blend weight 1.0"));
		}
	}
};

#endif // WITH_DEV_AUTOMATION_TESTS
