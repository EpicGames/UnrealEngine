// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/TraitSharedData.h"
#include "AnimNextInlineSubGraphTraitTest.generated.h"

USTRUCT()
struct FInlineSubGraphTest_LeafTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY()
	int32 Value = 0;
};
