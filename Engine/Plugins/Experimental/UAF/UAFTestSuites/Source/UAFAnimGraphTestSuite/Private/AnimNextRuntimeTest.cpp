// Copyright Epic Games, Inc. All Rights Reserved.


#include "AnimNextRuntimeTest.h"

#include "TraitCore/TraitReader.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/RigUnit_AnimNextGraphEvaluator.h"
#include "Graph/RigUnit_AnimNextGraphRoot.h"
#include "Misc/AutomationTest.h"
#include "Serialization/MemoryReader.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::UAF
{
	FScopedClearNodeTemplateRegistry::FScopedClearNodeTemplateRegistry()
	{
		FNodeTemplateRegistry& NodeRegistry = FNodeTemplateRegistry::Get();

		OldTemplateBufferSize = NodeRegistry.TemplateBuffer.Num();
		OldTemplateUIDToHandleMap = NodeRegistry.TemplateUIDToHandleMap;
	}

	FScopedClearNodeTemplateRegistry::~FScopedClearNodeTemplateRegistry()
	{
		FNodeTemplateRegistry& NodeRegistry = FNodeTemplateRegistry::Get();

		NodeRegistry.TemplateBuffer.SetNum(OldTemplateBufferSize, EAllowShrinking::No);
		NodeRegistry.TemplateUIDToHandleMap = OldTemplateUIDToHandleMap;
	}

	FScopedNewNodeTemplateRegistry::FScopedNewNodeTemplateRegistry()
	{
		OriginalRegistry = FNodeTemplateRegistry::Swap(&TmpRegistry);
	}

	FScopedNewNodeTemplateRegistry::~FScopedNewNodeTemplateRegistry()
	{
		FNodeTemplateRegistry* PrevRegistry = FNodeTemplateRegistry::Swap(OriginalRegistry);
		checkf(PrevRegistry == &TmpRegistry, TEXT("Unexpected node template registry instance found"));
	}

	bool FTestUtils::LoadFromArchiveBuffer(UUAFAnimGraph& AnimationGraph, TArray<FNodeHandle>& NodeHandles, const TArray<uint8>& SharedDataArchiveBuffer)
	{
		FAnimNextGraphEvaluatorExecuteDefinition ExecuteDefinition;
		ExecuteDefinition.Hash = 0;
		ExecuteDefinition.MethodName = TEXT("Execute_0");

		// Manually add our entry point since we didn't go through a full RigVM graph
		AnimationGraph.EntryPoints.Reset();

		FAnimNextGraphEntryPoint& EntryPoint = AnimationGraph.EntryPoints.AddDefaulted_GetRef();
		EntryPoint.EntryPointName = AnimationGraph.DefaultEntryPoint;
		EntryPoint.RootTraitHandle = FAnimNextEntryPointHandle(NodeHandles[0]);
		AnimationGraph.ExecuteDefinition = ExecuteDefinition;
		AnimationGraph.SharedDataArchiveBuffer = SharedDataArchiveBuffer;
		AnimationGraph.GraphReferencedObjects.Empty();
		AnimationGraph.GraphReferencedSoftObjects.Empty();

		// Reconstruct our graph shared data
		FMemoryReader GraphSharedDataArchive(SharedDataArchiveBuffer);
		FTraitReader TraitReader(AnimationGraph.GraphReferencedObjects, AnimationGraph.GraphReferencedSoftObjects, GraphSharedDataArchive);

		const FTraitReader::EErrorState ErrorState = TraitReader.ReadGraph(AnimationGraph.SharedDataBuffer);
		if (ErrorState == FTraitReader::EErrorState::None)
		{
			AnimationGraph.ResolvedRootTraitHandles.Add(AnimationGraph.DefaultEntryPoint, TraitReader.ResolveEntryPointHandle(AnimationGraph.EntryPoints[0].RootTraitHandle));

			for (FNodeHandle& NodeHandle : NodeHandles)
			{
				NodeHandle = TraitReader.ResolveNodeHandle(NodeHandle);
			}

			// Make sure our execute method is registered
			FRigUnit_AnimNextGraphEvaluator::RegisterExecuteMethod(ExecuteDefinition);
			return true;
		}
		else
		{
			AnimationGraph.SharedDataBuffer.Empty(0);
			AnimationGraph.ResolvedRootTraitHandles.Add(FRigUnit_AnimNextGraphRoot::DefaultEntryPoint, FAnimNextTraitHandle());
			return false;
		}
	}
}
#endif
