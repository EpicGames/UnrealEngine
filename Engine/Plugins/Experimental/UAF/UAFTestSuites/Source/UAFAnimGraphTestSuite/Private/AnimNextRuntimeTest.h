// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "UObject/Class.h"
#include "UObject/UnrealType.h"

#include "TraitCore/TraitUID.h"
#include "TraitCore/NodeTemplate.h"
#include "TraitCore/NodeTemplateRegistry.h"

class UUAFAnimGraph;

namespace UE::UAF
{
	struct FNodeHandle;

	// Converts a property value into its string representation using UE reflection
	template<class TraitSharedDataType, typename PropertyType>
	static FString ToString(const FString& PropertyName, PropertyType PropertyValue)
	{
		const UScriptStruct* SharedDataStruct = TraitSharedDataType::StaticStruct();
		if (const FProperty* Property = SharedDataStruct->FindPropertyByName(*PropertyName))
		{
			void* PropertyDefaults = Property->AllocateAndInitializeValue();

			FString Result;

			if constexpr (std::is_pointer_v<PropertyType>)
			{
				// C-style array properties aren't handled by ExportText, we need to handle it manually

				const bool bIsCArray = Property->ArrayDim > 1;
				if (bIsCArray)
				{
					Result += TEXT("(");
				}

				for (int32 Index = 0; Index < Property->ArrayDim; ++Index)
				{
					Property->ExportText_Direct(Result, PropertyValue + Index, (PropertyType)PropertyDefaults + Index, nullptr, PPF_None, nullptr);

					if (Index + 1 < Property->ArrayDim)
					{
						Result += TEXT(",");
					}
				}

				if (bIsCArray)
				{
					Result += TEXT(")");
				}
			}
			else
			{
				Property->ExportText_Direct(Result, &PropertyValue, PropertyDefaults, nullptr, PPF_None);
			}

			Property->DestroyAndFreeValue(PropertyDefaults);

			return Result;
		}

		return FString();
	}

	// Captures the node template registry and restores its state when we go out of scope
	struct FScopedClearNodeTemplateRegistry final
	{
		UE_NONCOPYABLE(FScopedClearNodeTemplateRegistry);	// We disallow copy/move semantics

		UAFANIMGRAPHTESTSUITE_API FScopedClearNodeTemplateRegistry();
		UAFANIMGRAPHTESTSUITE_API ~FScopedClearNodeTemplateRegistry();

		// The size of the TemplateBuffer in the node registry when the scope began
		// We'll restore it to that size (it is append only) when the scope ends
		int32 OldTemplateBufferSize = 0;

		// The TemplateUIDToHandleMap in the node registry when the scope began
		// We'll restore it to this one when the scope ends
		TMap<uint32, FNodeTemplateRegistryHandle> OldTemplateUIDToHandleMap;
	};

	// Creates a temporary empty node template registry instance and swaps it for the current one until the end of the scope
	// Should only be used when testing the node registry itself
	struct FScopedNewNodeTemplateRegistry final
	{
		UE_NONCOPYABLE(FScopedNewNodeTemplateRegistry);	// We disallow copy/move semantics

		UAFANIMGRAPHTESTSUITE_API FScopedNewNodeTemplateRegistry();
		UAFANIMGRAPHTESTSUITE_API ~FScopedNewNodeTemplateRegistry();

		// The new temporary node registry instance we'll set as the global one
		FNodeTemplateRegistry TmpRegistry;

		// The old node registry instance we'll restore once we go out of scope
		FNodeTemplateRegistry* OriginalRegistry = nullptr;
	};

	struct FTestUtils final
	{
		// Loads the module data from the provided archive buffer and returns true on success, false otherwise
		// On success, we resolve every node handle provided as argument
		static UAFANIMGRAPHTESTSUITE_API bool LoadFromArchiveBuffer(UUAFAnimGraph& AnimationGraph, TArray<FNodeHandle>& NodeHandles, const TArray<uint8>& SharedDataArchiveBuffer);
	};
}

#endif
