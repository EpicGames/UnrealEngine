// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextTraitBaseTest.h"
#include "AnimNextRuntimeTest.h"
#include "AnimNextTest.h"
#include "Factory/AnimGraphBuilderContext.h"
#include "Factory/AnimGraphFactory.h"
#include "Factory/AnimNextSimpleAnimGraphBuilder.h"

#include "Misc/AutomationTest.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextTraitBaseTest)

#if WITH_DEV_AUTOMATION_TESTS

#include "Serialization/MemoryReader.h"

#include "TraitCore/Trait.h"
#include "TraitCore/TraitReader.h"
#include "TraitCore/TraitRegistry.h"
#include "TraitCore/TraitWriter.h"
#include "TraitCore/ExecutionContext.h"
#include "TraitCore/ITraitInterface.h"
#include "TraitCore/NodeInstance.h"
#include "TraitCore/NodeTemplateBuilder.h"
#include "TraitCore/NodeTemplateRegistry.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextAnimationGraphFactory.h"

//****************************************************************************
// AnimNext Runtime TraitBase Tests
//****************************************************************************

namespace UE::UAF
{
	namespace Private
	{
		static TArray<FTraitUID>* ConstructedTraits = nullptr;
		static TArray<FTraitUID>* DestructedTraits = nullptr;
	}

	struct IInterfaceA : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(IInterfaceA)

		virtual void FuncA(const FExecutionContext& Context, const TTraitBinding<IInterfaceA>& Binding) const;
	};

	template<>
	struct TTraitBinding<IInterfaceA> : FTraitBinding
	{
		void FuncA(const FExecutionContext& Context) const
		{
			GetInterface()->FuncA(Context, *this);
		}

	protected:
		const IInterfaceA* GetInterface() const { return GetInterfaceTyped<IInterfaceA>(); }
	};

	void IInterfaceA::FuncA(const FExecutionContext& Context, const TTraitBinding<IInterfaceA>& Binding) const
	{
		TTraitBinding<IInterfaceA> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			SuperBinding.FuncA(Context);
		}
	}

	//////////////////////////////////////////////////////////////////////////

	struct IInterfaceB : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(IInterfaceB)

		virtual void FuncB(const FExecutionContext& Context, const TTraitBinding<IInterfaceB>& Binding) const;
	};

	template<>
	struct TTraitBinding<IInterfaceB> : FTraitBinding
	{
		void FuncB(const FExecutionContext& Context) const
		{
			GetInterface()->FuncB(Context, *this);
		}

	protected:
		const IInterfaceB* GetInterface() const { return GetInterfaceTyped<IInterfaceB>(); }
	};

	void IInterfaceB::FuncB(const FExecutionContext& Context, const TTraitBinding<IInterfaceB>& Binding) const
	{
		TTraitBinding<IInterfaceB> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			SuperBinding.FuncB(Context);
		}
	}

	//////////////////////////////////////////////////////////////////////////

	struct IInterfaceC : ITraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(IInterfaceC)

		virtual void FuncC(const FExecutionContext& Context, const TTraitBinding<IInterfaceC>& Binding) const;
	};

	template<>
	struct TTraitBinding<IInterfaceC> : FTraitBinding
	{
		void FuncC(const FExecutionContext& Context) const
		{
			GetInterface()->FuncC(Context, *this);
		}

	protected:
		const IInterfaceC* GetInterface() const { return GetInterfaceTyped<IInterfaceC>(); }
	};

	void IInterfaceC::FuncC(const FExecutionContext& Context, const TTraitBinding<IInterfaceC>& Binding) const
	{
		TTraitBinding<IInterfaceC> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			SuperBinding.FuncC(Context);
		}
	}

	//////////////////////////////////////////////////////////////////////////

	struct FTraitA_Base : FBaseTrait, IInterfaceA
	{
		DECLARE_ANIM_TRAIT(FTraitA_Base, FBaseTrait)

		using FSharedData = FTraitA_BaseSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			FTraitUID TraitUID = FTraitA_Base::TraitUID;

			FInstanceData()
			{
				if (Private::ConstructedTraits != nullptr)
				{
					Private::ConstructedTraits->Add(FTraitA_Base::TraitUID);
				}
			}

			~FInstanceData()
			{
				if (Private::DestructedTraits != nullptr)
				{
					Private::DestructedTraits->Add(FTraitA_Base::TraitUID);
				}
			}
		};

		// IInterfaceA impl
		virtual void FuncA(const FExecutionContext& Context, const TTraitBinding<IInterfaceA>& Binding) const override
		{
		}
	};

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IInterfaceA) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FTraitA_Base, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	//////////////////////////////////////////////////////////////////////////

	struct FTraitAB_Add : FAdditiveTrait, IInterfaceA, IInterfaceB
	{
		DECLARE_ANIM_TRAIT(FTraitAB_Add, FAdditiveTrait)

		using FSharedData = FTraitAB_AddSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			FTraitUID TraitUID = FTraitAB_Add::TraitUID;

			FInstanceData()
			{
				if (Private::ConstructedTraits != nullptr)
				{
					Private::ConstructedTraits->Add(FTraitAB_Add::TraitUID);
				}
			}

			~FInstanceData()
			{
				if (Private::DestructedTraits != nullptr)
				{
					Private::DestructedTraits->Add(FTraitAB_Add::TraitUID);
				}
			}
		};

		// IInterfaceA impl
		virtual void FuncA(const FExecutionContext& Context, const TTraitBinding<IInterfaceA>& Binding) const override
		{
		}

		// IInterfaceB impl
		virtual void FuncB(const FExecutionContext& Context, const TTraitBinding<IInterfaceB>& Binding) const override
		{
		}
	};

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IInterfaceA) \
		GeneratorMacro(IInterfaceB) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FTraitAB_Add, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	//////////////////////////////////////////////////////////////////////////

	struct FTraitAC_Add : FAdditiveTrait, IInterfaceA, IInterfaceC
	{
		DECLARE_ANIM_TRAIT(FTraitAC_Add, FAdditiveTrait)

		using FSharedData = FTraitAC_AddSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			FTraitUID TraitUID = FTraitAC_Add::TraitUID;

			FInstanceData()
			{
				if (Private::ConstructedTraits != nullptr)
				{
					Private::ConstructedTraits->Add(FTraitAC_Add::TraitUID);
				}
			}

			~FInstanceData()
			{
				if (Private::DestructedTraits != nullptr)
				{
					Private::DestructedTraits->Add(FTraitAC_Add::TraitUID);
				}
			}
		};

		// IInterfaceA impl
		virtual void FuncA(const FExecutionContext& Context, const TTraitBinding<IInterfaceA>& Binding) const override
		{
		}

		// IInterfaceC impl
		virtual void FuncC(const FExecutionContext& Context, const TTraitBinding<IInterfaceC>& Binding) const override
		{
		}
	};

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IInterfaceA) \
		GeneratorMacro(IInterfaceC) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FTraitAC_Add, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR


	//////////////////////////////////////////////////////////////////////////

	struct FTraitSerialization_Base : FBaseTrait, IInterfaceA
	{
		DECLARE_ANIM_TRAIT(FTraitSerialization_Base, FBaseTrait)

		using FSharedData = FTraitSerialization_BaseSharedData;
	};

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IInterfaceA) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FTraitSerialization_Base, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	//////////////////////////////////////////////////////////////////////////

	struct FTraitSerialization_Add : FAdditiveTrait, IInterfaceB
	{
		DECLARE_ANIM_TRAIT(FTraitSerialization_Add, FAdditiveTrait)

		using FSharedData = FTraitSerialization_AddSharedData;
	};

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IInterfaceB) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FTraitSerialization_Add, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	//////////////////////////////////////////////////////////////////////////

	struct FTraitNativeSerialization_Add : FAdditiveTrait, IInterfaceC
	{
		DECLARE_ANIM_TRAIT(FTraitNativeSerialization_Add, FAdditiveTrait)

		using FSharedData = FTraitNativeSerialization_AddSharedData;
	};

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IInterfaceC) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FTraitNativeSerialization_Add, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnimationAnimNextRuntimeTest_TraitRegistry, "Animation.AnimNext.Runtime.TraitRegistry", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAnimationAnimNextRuntimeTest_TraitRegistry::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	FTraitRegistry& Registry = FTraitRegistry::Get();

	// Some traits already exist in the engine, keep track of them
	const uint32 NumAutoRegisteredTraits = Registry.GetNum();

	AddErrorIfFalse(!Registry.FindHandle(FTraitA_Base::TraitUID).IsValid(), "FAnimationAnimNextRuntimeTest_TraitRegistry -> Registry should not contain our trait");
	AddErrorIfFalse(!Registry.FindHandle(FTraitAB_Add::TraitUID).IsValid(), "FAnimationAnimNextRuntimeTest_TraitRegistry -> Registry should not contain our trait");
	AddErrorIfFalse(!Registry.FindHandle(FTraitAC_Add::TraitUID).IsValid(), "FAnimationAnimNextRuntimeTest_TraitRegistry -> Registry should not contain our trait");

	{
		// Auto register a trait
		AUTO_REGISTER_ANIM_TRAIT(FTraitA_Base)

		AddErrorIfFalse(Registry.GetNum() == NumAutoRegisteredTraits + 1, "FAnimationAnimNextRuntimeTest_TraitRegistry -> Registry should contain 1 new trait");

		FTraitRegistryHandle HandleA = Registry.FindHandle(FTraitA_Base::TraitUID);
		AddErrorIfFalse(HandleA.IsValid(), "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait should have registered automatically");
		AddErrorIfFalse(HandleA.IsStatic(), "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait should have been statically allocated");

		const FTrait* TraitA = Registry.Find(HandleA);
		AddErrorIfFalse(TraitA != nullptr, "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait should exist");
		if (TraitA != nullptr)
		{
			AddErrorIfFalse(TraitA->GetTraitUID() == FTraitA_Base::TraitUID, "FAnimationAnimNextRuntimeTest_TraitRegistry -> Unexpected trait instance type");

			{
				// Auto register another trait
				AUTO_REGISTER_ANIM_TRAIT(FTraitAB_Add)

				AddErrorIfFalse(Registry.GetNum() == NumAutoRegisteredTraits + 2, "FAnimationAnimNextRuntimeTest_TraitRegistry -> Registry should contain 2 new traits");

				FTraitRegistryHandle HandleAB = Registry.FindHandle(FTraitAB_Add::TraitUID);
				AddErrorIfFalse(HandleAB.IsValid(), "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait should have registered automatically");
				AddErrorIfFalse(HandleAB.IsStatic(), "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait should have been statically allocated");
				AddErrorIfFalse(HandleA != HandleAB, "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait handles should be different");

				const FTrait* TraitAB = Registry.Find(HandleAB);
				AddErrorIfFalse(TraitAB != nullptr, "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait should exist");
				if (TraitAB != nullptr)
				{
					AddErrorIfFalse(TraitAB->GetTraitUID() == FTraitAB_Add::TraitUID, "FAnimationAnimNextRuntimeTest_TraitRegistry -> Unexpected trait instance type");

					FTraitRegistryHandle HandleAC_0;
					{
						// Dynamically register a trait
						FTraitAC_Add TraitAC_0;
						Registry.Register(&TraitAC_0);

						AddErrorIfFalse(Registry.GetNum() == NumAutoRegisteredTraits + 3, "FAnimationAnimNextRuntimeTest_TraitRegistry -> Registry should contain 3 new traits");

						HandleAC_0 = Registry.FindHandle(FTraitAC_Add::TraitUID);
						AddErrorIfFalse(HandleAC_0.IsValid(), "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait should have registered automatically");
						AddErrorIfFalse(HandleAC_0.IsDynamic(), "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait should have been dynamically allocated");
						AddErrorIfFalse(HandleA != HandleAC_0, "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait handles should be different");

						const FTrait* TraitAC_0Ptr = Registry.Find(HandleAC_0);
						AddErrorIfFalse(TraitAC_0Ptr != nullptr, "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait should exist");
						if (TraitAC_0Ptr != nullptr)
						{
							AddErrorIfFalse(TraitAC_0Ptr->GetTraitUID() == FTraitAC_Add::TraitUID, "FAnimationAnimNextRuntimeTest_TraitRegistry -> Unexpected trait instance type");
							AddErrorIfFalse(&TraitAC_0 == TraitAC_0Ptr, "FAnimationAnimNextRuntimeTest_TraitRegistry -> Unexpected trait instance pointer");

							// Unregister our instances
							Registry.Unregister(&TraitAC_0);

							AddErrorIfFalse(Registry.GetNum() == NumAutoRegisteredTraits + 2, "FAnimationAnimNextRuntimeTest_TraitRegistry -> Registry should contain 2 extra traits");
							AddErrorIfFalse(!Registry.FindHandle(FTraitAC_Add::TraitUID).IsValid(), "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait should have unregistered");
						}
					}

					{
						// Dynamically register another trait, re-using the previous dynamic index
						FTraitAC_Add TraitAC_1;
						Registry.Register(&TraitAC_1);

						AddErrorIfFalse(Registry.GetNum() == NumAutoRegisteredTraits + 3, "FAnimationAnimNextRuntimeTest_TraitRegistry -> Registry should contain 3 new traits");

						FTraitRegistryHandle HandleAC_1 = Registry.FindHandle(FTraitAC_Add::TraitUID);
						AddErrorIfFalse(HandleAC_1.IsValid(), "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait should have registered automatically");
						AddErrorIfFalse(HandleAC_1.IsDynamic(), "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait should have been dynamically allocated");
						AddErrorIfFalse(HandleA != HandleAC_1, "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait handles should be different");
						AddErrorIfFalse(HandleAC_0 == HandleAC_1, "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait handles should be identical");

						const FTrait* TraitAC_1Ptr = Registry.Find(HandleAC_1);
						AddErrorIfFalse(TraitAC_1Ptr != nullptr, "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait should exist");
						if (TraitAC_1Ptr != nullptr)
						{
							AddErrorIfFalse(TraitAC_1Ptr->GetTraitUID() == FTraitAC_Add::TraitUID, "FAnimationAnimNextRuntimeTest_TraitRegistry -> Unexpected trait instance type");
							AddErrorIfFalse(&TraitAC_1 == TraitAC_1Ptr, "FAnimationAnimNextRuntimeTest_TraitRegistry -> Unexpected trait instance pointer");

							// Unregister our instances
							Registry.Unregister(&TraitAC_1);

							AddErrorIfFalse(Registry.GetNum() == NumAutoRegisteredTraits + 2, "FAnimationAnimNextRuntimeTest_TraitRegistry -> Registry should contain 2 extra traits");
							AddErrorIfFalse(!Registry.FindHandle(FTraitAC_Add::TraitUID).IsValid(), "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait should have unregistered");
						}
					}
				}
			}

			AddErrorIfFalse(Registry.GetNum() == NumAutoRegisteredTraits + 1, "FAnimationAnimNextRuntimeTest_TraitRegistry -> Registry should contain 1 extra trait");
			AddErrorIfFalse(!Registry.FindHandle(FTraitAB_Add::TraitUID).IsValid(), "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait should have unregistered automatically");
			AddErrorIfFalse(HandleA == Registry.FindHandle(FTraitA_Base::TraitUID), "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait handle should not have changed");
		}
	}

	AddErrorIfFalse(Registry.GetNum() == NumAutoRegisteredTraits, "FAnimationAnimNextRuntimeTest_TraitRegistry -> All traits should have unregistered");
	AddErrorIfFalse(!Registry.FindHandle(FTraitA_Base::TraitUID).IsValid(), "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait should have unregistered automatically");
	AddErrorIfFalse(!Registry.FindHandle(FTraitAB_Add::TraitUID).IsValid(), "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait should have unregistered automatically");
	AddErrorIfFalse(!Registry.FindHandle(FTraitAC_Add::TraitUID).IsValid(), "FAnimationAnimNextRuntimeTest_TraitRegistry -> Trait should have unregistered automatically");

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnimationAnimNextRuntimeTest_NodeTemplateRegistry, "Animation.AnimNext.Runtime.NodeTemplateRegistry", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAnimationAnimNextRuntimeTest_NodeTemplateRegistry::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	AUTO_REGISTER_ANIM_TRAIT(FTraitA_Base)
	AUTO_REGISTER_ANIM_TRAIT(FTraitAB_Add)
	AUTO_REGISTER_ANIM_TRAIT(FTraitAC_Add)

	FScopedNewNodeTemplateRegistry ScopedClearNodeTemplateRegistry;
	FNodeTemplateRegistry& Registry = FNodeTemplateRegistry::Get();

	TArray<FTraitUID> NodeTemplateTraitList;
	NodeTemplateTraitList.Add(FTraitA_Base::TraitUID);
	NodeTemplateTraitList.Add(FTraitAB_Add::TraitUID);
	NodeTemplateTraitList.Add(FTraitAC_Add::TraitUID);
	NodeTemplateTraitList.Add(FTraitA_Base::TraitUID);
	NodeTemplateTraitList.Add(FTraitAC_Add::TraitUID);

	// Populate our node template registry
	TArray<uint8> NodeTemplateBuffer0;
	const FNodeTemplate* NodeTemplate0 = FNodeTemplateBuilder::BuildNodeTemplate(NodeTemplateTraitList, NodeTemplateBuffer0);

	AddErrorIfFalse(Registry.GetNum() == 0, "FAnimationAnimNextRuntimeTest_NodeTemplateRegistry -> Registry should contain any templates");

	FNodeTemplateRegistryHandle TemplateHandle0 = Registry.FindOrAdd(NodeTemplate0);
	AddErrorIfFalse(Registry.GetNum() == 1, "FAnimationAnimNextRuntimeTest_NodeTemplateRegistry -> Registry should contain 1 template");
	AddErrorIfFalse(TemplateHandle0.IsValid(), "FAnimationAnimNextRuntimeTest_NodeTemplateRegistry -> Registry should contain our template");

	const uint32 TemplateSize0 = NodeTemplate0->GetNodeTemplateSize();
	const FNodeTemplate* NodeTemplate0_ = Registry.Find(TemplateHandle0);
	AddErrorIfFalse(NodeTemplate0_ != nullptr, "FAnimationAnimNextRuntimeTest_NodeTemplateRegistry -> Registry should contain our template");
	if (NodeTemplate0_ != nullptr)
	{
		AddErrorIfFalse(NodeTemplate0_ != NodeTemplate0, "FAnimationAnimNextRuntimeTest_NodeTemplateRegistry -> Template pointers should be different");
		AddErrorIfFalse(FMemory::Memcmp(NodeTemplate0, NodeTemplate0_, TemplateSize0) == 0, "FAnimationAnimNextRuntimeTest_NodeTemplateRegistry -> Templates should be identical");

		// Try and register a duplicate template
		TArray<uint8> NodeTemplateBuffer1;
		const FNodeTemplate* NodeTemplate1 = FNodeTemplateBuilder::BuildNodeTemplate(NodeTemplateTraitList, NodeTemplateBuffer1);
		if (NodeTemplate1 != nullptr)
		{
			AddErrorIfFalse(NodeTemplate0 != NodeTemplate1, "FAnimationAnimNextRuntimeTest_NodeTemplateRegistry -> Node template pointers should be different");
			AddErrorIfFalse(NodeTemplate0->GetUID() == NodeTemplate1->GetUID(), "FAnimationAnimNextRuntimeTest_NodeTemplateRegistry -> Node template UIDs should be identical");

			FNodeTemplateRegistryHandle TemplateHandle1 = Registry.FindOrAdd(NodeTemplate1);
			AddErrorIfFalse(Registry.GetNum() == 1, "FAnimationAnimNextRuntimeTest_NodeTemplateRegistry -> Registry should contain 1 template");
			AddErrorIfFalse(TemplateHandle0 == TemplateHandle1, "FAnimationAnimNextRuntimeTest_NodeTemplateRegistry -> Node template handles should be identical");

			// Try and register a new template
			TArray<FTraitUID> NodeTemplateTraitList2;
			NodeTemplateTraitList2.Add(FTraitA_Base::TraitUID);
			NodeTemplateTraitList2.Add(FTraitAB_Add::TraitUID);
			NodeTemplateTraitList2.Add(FTraitAC_Add::TraitUID);
			NodeTemplateTraitList2.Add(FTraitAC_Add::TraitUID);

			TArray<uint8> NodeTemplateBuffer2;
			const FNodeTemplate* NodeTemplate2 = FNodeTemplateBuilder::BuildNodeTemplate(NodeTemplateTraitList2, NodeTemplateBuffer2);
			if (NodeTemplate2 != nullptr)
			{
				AddErrorIfFalse(NodeTemplate0->GetUID() != NodeTemplate2->GetUID(), "FAnimationAnimNextRuntimeTest_NodeTemplateRegistry -> Node template UIDs should be different");

				FNodeTemplateRegistryHandle TemplateHandle2 = Registry.FindOrAdd(NodeTemplate2);
				AddErrorIfFalse(Registry.GetNum() == 2, "FAnimationAnimNextRuntimeTest_NodeTemplateRegistry -> Registry should contain 2 templates");
				AddErrorIfFalse(TemplateHandle0 != TemplateHandle2, "FAnimationAnimNextRuntimeTest_NodeTemplateRegistry -> Node template handles should be identical");
				AddErrorIfFalse(TemplateHandle2.IsValid(), "FAnimationAnimNextRuntimeTest_NodeTemplateRegistry -> Registry should contain our template");

				// Unregister our templates
				Registry.Unregister(NodeTemplate2);
			}
		}

		Registry.Unregister(NodeTemplate0);
	}

	AddErrorIfFalse(Registry.GetNum() == 0, "FAnimationAnimNextRuntimeTest_NodeTemplateRegistry -> Registry should contain 0 templates");

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnimationAnimNextRuntimeTest_NodeLifetime, "Animation.AnimNext.Runtime.NodeLifetime", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAnimationAnimNextRuntimeTest_NodeLifetime::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;
	{
		AUTO_REGISTER_ANIM_TRAIT(FTraitA_Base)
		AUTO_REGISTER_ANIM_TRAIT(FTraitAB_Add)
		AUTO_REGISTER_ANIM_TRAIT(FTraitAC_Add)

		UFactory* AnimationGraphFactory = NewObject<UUAFAnimGraphFactory>();
		UUAFAnimGraph* AnimationGraph = CastChecked<UUAFAnimGraph>(AnimationGraphFactory->FactoryCreateNew(UUAFAnimGraph::StaticClass(), GetTransientPackage(), TEXT("TestAnimNextModule"), RF_Transient, nullptr, nullptr, NAME_None));
		UE_RETURN_ON_ERROR(AnimationGraph != nullptr, "FAnimationAnimNextRuntimeTest_NodeLifetime -> Failed to create animation graph");

		FScopedClearNodeTemplateRegistry ScopedClearNodeTemplateRegistry;
		FNodeTemplateRegistry& Registry = FNodeTemplateRegistry::Get();

		TArray<FTraitUID> NodeTemplateTraitList;
		NodeTemplateTraitList.Add(FTraitA_Base::TraitUID);
		NodeTemplateTraitList.Add(FTraitAB_Add::TraitUID);
		NodeTemplateTraitList.Add(FTraitAC_Add::TraitUID);
		NodeTemplateTraitList.Add(FTraitA_Base::TraitUID);
		NodeTemplateTraitList.Add(FTraitAC_Add::TraitUID);

		// Populate our node template registry
		TArray<uint8> NodeTemplateBuffer0;
		const FNodeTemplate* NodeTemplate0 = FNodeTemplateBuilder::BuildNodeTemplate(NodeTemplateTraitList, NodeTemplateBuffer0);

		FNodeTemplateRegistryHandle TemplateHandle0 = Registry.FindOrAdd(NodeTemplate0);
		AddErrorIfFalse(TemplateHandle0.IsValid(), "FAnimationAnimNextRuntimeTest_NodeLifetime -> Registry should contain our template");

		TArray<FNodeHandle> NodeHandles;

		// Write our graph
		TArray<uint8> GraphSharedDataArchiveBuffer;
		TArray<TObjectPtr<UObject>> GraphReferencedObjects;
		TArray<FSoftObjectPath> GraphReferencedSoftObjects;
		{
			FTraitWriter TraitWriter;

			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplate0));
			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplate0));

			// We don't have trait properties

			TraitWriter.BeginNodeWriting();
			TraitWriter.WriteNode(NodeHandles[0],
				[](uint32 TraitIndex, FName PropertyName)
				{
					return FString();
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.WriteNode(NodeHandles[1],
				[](uint32 TraitIndex, FName PropertyName)
				{
					return FString();
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.EndNodeWriting();

			AddErrorIfFalse(TraitWriter.GetErrorState() == FTraitWriter::EErrorState::None, "FAnimationAnimNextRuntimeTest_NodeLifetime -> Failed to write traits");
			GraphSharedDataArchiveBuffer = TraitWriter.GetGraphSharedData();
			GraphReferencedObjects = TraitWriter.GetGraphReferencedObjects();
			GraphReferencedSoftObjects = TraitWriter.GetGraphReferencedSoftObjects();
		}

		// Read our graph
		FTestUtils::LoadFromArchiveBuffer(*AnimationGraph, NodeHandles, GraphSharedDataArchiveBuffer);

		TSharedPtr<FAnimNextGraphInstance> GraphInstance = AnimationGraph->AllocateInstance();

		FExecutionContext Context(*GraphInstance.Get());

		// Validate handle bookkeeping
		{
			FAnimNextTraitHandle TraitHandle00(NodeHandles[0], 0);	// Point to first node, first base trait

			// Allocate a node
			FTraitPtr TraitPtr00 = Context.AllocateNodeInstance(*GraphInstance.Get(), TraitHandle00);
			AddErrorIfFalse(TraitPtr00.IsValid(), "FAnimationAnimNextRuntimeTest_NodeLifetime -> Failed to allocate a node instance");
			AddErrorIfFalse(TraitPtr00.GetTraitIndex() == 0, "FAnimationAnimNextRuntimeTest_NodeLifetime -> Allocated trait pointer should point to root trait");
			AddErrorIfFalse(!TraitPtr00.IsWeak(), "FAnimationAnimNextRuntimeTest_NodeLifetime -> Allocated trait pointer should not be weak, we have no parent");
			AddErrorIfFalse(TraitPtr00.GetNodeInstance()->GetNodeHandle() == NodeHandles[0], "FAnimationAnimNextRuntimeTest_NodeLifetime -> Allocated node should point to the provided node handle");
			AddErrorIfFalse(TraitPtr00.GetNodeInstance()->GetReferenceCount() == 1, "FAnimationAnimNextRuntimeTest_NodeLifetime -> Allocated node should have a single reference");

			{
				FWeakTraitPtr WeakTraitPtr00(TraitPtr00);
				AddErrorIfFalse(WeakTraitPtr00.GetNodeInstance() == TraitPtr00.GetNodeInstance(), "FAnimationAnimNextRuntimeTest_NodeLifetime -> Weak reference should point to the same node instance");
				AddErrorIfFalse(WeakTraitPtr00.GetTraitIndex() == TraitPtr00.GetTraitIndex(), "FAnimationAnimNextRuntimeTest_NodeLifetime -> Weak reference should point to the same trait index");
				AddErrorIfFalse(TraitPtr00.GetNodeInstance()->GetReferenceCount() == 1, "FAnimationAnimNextRuntimeTest_NodeLifetime -> Weak references shouldn't increase ref count");
			}

			AddErrorIfFalse(TraitPtr00.GetNodeInstance()->GetReferenceCount() == 1, "FAnimationAnimNextRuntimeTest_NodeLifetime -> Weak references shouldn't decrease ref count");

			{
				FWeakTraitPtr WeakTraitPtr00 = TraitPtr00;
				AddErrorIfFalse(WeakTraitPtr00.GetNodeInstance() == TraitPtr00.GetNodeInstance(), "FAnimationAnimNextRuntimeTest_NodeLifetime -> Weak reference should point to the same node instance");
				AddErrorIfFalse(WeakTraitPtr00.GetTraitIndex() == TraitPtr00.GetTraitIndex(), "FAnimationAnimNextRuntimeTest_NodeLifetime -> Weak reference should point to the same trait index");
				AddErrorIfFalse(TraitPtr00.GetNodeInstance()->GetReferenceCount() == 1, "FAnimationAnimNextRuntimeTest_NodeLifetime -> Weak references shouldn't increase ref count");
			}

			AddErrorIfFalse(TraitPtr00.GetNodeInstance()->GetReferenceCount() == 1, "FAnimationAnimNextRuntimeTest_NodeLifetime -> Weak references shouldn't decrease ref count");

			{
				FTraitPtr TraitPtr00_1(TraitPtr00);
				AddErrorIfFalse(TraitPtr00_1.GetNodeInstance() == TraitPtr00.GetNodeInstance(), "FAnimationAnimNextRuntimeTest_NodeLifetime -> Strong reference should point to the same node instance");
				AddErrorIfFalse(TraitPtr00_1.GetTraitIndex() == TraitPtr00.GetTraitIndex(), "FAnimationAnimNextRuntimeTest_NodeLifetime -> Strong reference should point to the same trait index");
				AddErrorIfFalse(TraitPtr00.GetNodeInstance()->GetReferenceCount() == 2, "FAnimationAnimNextRuntimeTest_NodeLifetime -> Strong references should increase ref count");
			}

			AddErrorIfFalse(TraitPtr00.GetNodeInstance()->GetReferenceCount() == 1, "FAnimationAnimNextRuntimeTest_NodeLifetime -> Strong references should decrease ref count");

			{
				FTraitPtr TraitPtr00_1 = TraitPtr00;
				AddErrorIfFalse(TraitPtr00_1.GetNodeInstance() == TraitPtr00.GetNodeInstance(), "FAnimationAnimNextRuntimeTest_NodeLifetime -> Strong reference should point to the same node instance");
				AddErrorIfFalse(TraitPtr00_1.GetTraitIndex() == TraitPtr00.GetTraitIndex(), "FAnimationAnimNextRuntimeTest_NodeLifetime -> Strong reference should point to the same trait index");
				AddErrorIfFalse(TraitPtr00.GetNodeInstance()->GetReferenceCount() == 2, "FAnimationAnimNextRuntimeTest_NodeLifetime -> Strong references should increase ref count");
			}

			AddErrorIfFalse(TraitPtr00.GetNodeInstance()->GetReferenceCount() == 1, "FAnimationAnimNextRuntimeTest_NodeLifetime -> Strong references should decrease ref count");
		}

		// Validate parent support
		{
			FAnimNextTraitHandle TraitHandle00(NodeHandles[0], 0);		// Point to first node, first base trait
			FAnimNextTraitHandle TraitHandle03(NodeHandles[0], 3);		// Point to first node, second base trait
			FAnimNextTraitHandle TraitHandle10(NodeHandles[1], 0);		// Point to second node, first base trait

			// Allocate our first node
			FTraitPtr TraitPtr00 = Context.AllocateNodeInstance(*GraphInstance.Get(), TraitHandle00);
			AddErrorIfFalse(TraitPtr00.IsValid(), "FAnimationAnimNextRuntimeTest_NodeLifetime -> Failed to allocate a node instance");

			// Allocate a new node, using the first as a parent
			// Both traits live on the same node, the returned handle should be weak on the parent
			FTraitPtr TraitPtr03 = Context.AllocateNodeInstance(TraitPtr00, TraitHandle03);
			AddErrorIfFalse(TraitPtr03.IsValid(), "FAnimationAnimNextRuntimeTest_NodeLifetime -> Failed to allocate a node instance");
			AddErrorIfFalse(TraitPtr03.GetTraitIndex() == 3, "FAnimationAnimNextRuntimeTest_NodeLifetime -> Allocated trait pointer should point to fourth trait");
			AddErrorIfFalse(TraitPtr03.IsWeak(), "FAnimationAnimNextRuntimeTest_NodeLifetime -> Allocated trait pointer should be weak, we have the same parent");
			AddErrorIfFalse(TraitPtr03.GetNodeInstance()->GetNodeHandle() == NodeHandles[0], "FAnimationAnimNextRuntimeTest_NodeLifetime -> Allocated node should point to the provided node handle");
			AddErrorIfFalse(TraitPtr03.GetNodeInstance() == TraitPtr00.GetNodeInstance(), "FAnimationAnimNextRuntimeTest_NodeLifetime -> Handles should point to the same node instance");
			AddErrorIfFalse(TraitPtr03.GetNodeInstance()->GetReferenceCount() == 1, "FAnimationAnimNextRuntimeTest_NodeLifetime -> Allocated node should have one reference");

			// Allocate a new node, using the first as a parent
			// The second trait lives on a new node, a new node instance will be allocated
			FTraitPtr TraitPtr10 = Context.AllocateNodeInstance(TraitPtr00, TraitHandle10);
			AddErrorIfFalse(TraitPtr10.IsValid(), "FAnimationAnimNextRuntimeTest_NodeLifetime -> Failed to allocate a node instance");
			AddErrorIfFalse(TraitPtr10.GetTraitIndex() == 0, "FAnimationAnimNextRuntimeTest_NodeLifetime -> Allocated trait pointer should point to first trait");
			AddErrorIfFalse(!TraitPtr10.IsWeak(), "FAnimationAnimNextRuntimeTest_NodeLifetime -> Allocated trait pointer should not be weak, we have the same parent but a different node handle");
			AddErrorIfFalse(TraitPtr10.GetNodeInstance()->GetNodeHandle() == NodeHandles[1], "FAnimationAnimNextRuntimeTest_NodeLifetime -> Allocated node should point to the provided node handle");
			AddErrorIfFalse(TraitPtr10.GetNodeInstance() != TraitPtr00.GetNodeInstance(), "FAnimationAnimNextRuntimeTest_NodeLifetime -> Handles should not point to the same node instance");
			AddErrorIfFalse(TraitPtr10.GetNodeInstance()->GetReferenceCount() == 1, "FAnimationAnimNextRuntimeTest_NodeLifetime -> Allocated node should have one reference");
		}

		// Validate constructors and destructors
		{
			TArray<FTraitUID> ConstructedTraits;
			TArray<FTraitUID> DestructedTraits;

			Private::ConstructedTraits = &ConstructedTraits;
			Private::DestructedTraits = &DestructedTraits;

			{
				FAnimNextTraitHandle TraitHandle00(NodeHandles[0], 0);		// Point to first node, first base trait

				// Allocate our node instance
				FTraitPtr TraitPtr00 = Context.AllocateNodeInstance(*GraphInstance.Get(), TraitHandle00);
				AddErrorIfFalse(TraitPtr00.IsValid(), "FAnimationAnimNextRuntimeTest_NodeLifetime -> Failed to allocate a node instance");

				// Validate instance constructors
				AddErrorIfFalse(ConstructedTraits.Num() == 5, "FAnimationAnimNextRuntimeTest_NodeLifetime -> Expected all 5 traits to have been constructed");
				AddErrorIfFalse(DestructedTraits.Num() == 0, "FAnimationAnimNextRuntimeTest_NodeLifetime -> Expected no traits to have been destructed");
				AddErrorIfFalse(ConstructedTraits[0] == NodeTemplateTraitList[0], "FAnimationAnimNextRuntimeTest_NodeLifetime -> Unexpected constructor order");
				AddErrorIfFalse(ConstructedTraits[1] == NodeTemplateTraitList[1], "FAnimationAnimNextRuntimeTest_NodeLifetime -> Unexpected constructor order");
				AddErrorIfFalse(ConstructedTraits[2] == NodeTemplateTraitList[2], "FAnimationAnimNextRuntimeTest_NodeLifetime -> Unexpected constructor order");
				AddErrorIfFalse(ConstructedTraits[3] == NodeTemplateTraitList[3], "FAnimationAnimNextRuntimeTest_NodeLifetime -> Unexpected constructor order");
				AddErrorIfFalse(ConstructedTraits[4] == NodeTemplateTraitList[4], "FAnimationAnimNextRuntimeTest_NodeLifetime -> Unexpected constructor order");

				// Destruct our node instance
			}

			// Validate instance destructors
			AddErrorIfFalse(ConstructedTraits.Num() == 5, "FAnimationAnimNextRuntimeTest_NodeLifetime -> Expected no traits to have been constructed");
			AddErrorIfFalse(DestructedTraits.Num() == 5, "FAnimationAnimNextRuntimeTest_NodeLifetime -> Expected all 5 traits to have been destructed");
			AddErrorIfFalse(DestructedTraits[0] == NodeTemplateTraitList[4], "FAnimationAnimNextRuntimeTest_NodeLifetime -> Unexpected destructor order");
			AddErrorIfFalse(DestructedTraits[1] == NodeTemplateTraitList[3], "FAnimationAnimNextRuntimeTest_NodeLifetime -> Unexpected destructor order");
			AddErrorIfFalse(DestructedTraits[2] == NodeTemplateTraitList[2], "FAnimationAnimNextRuntimeTest_NodeLifetime -> Unexpected destructor order");
			AddErrorIfFalse(DestructedTraits[3] == NodeTemplateTraitList[1], "FAnimationAnimNextRuntimeTest_NodeLifetime -> Unexpected destructor order");
			AddErrorIfFalse(DestructedTraits[4] == NodeTemplateTraitList[0], "FAnimationAnimNextRuntimeTest_NodeLifetime -> Unexpected destructor order");

			Private::ConstructedTraits = nullptr;
			Private::DestructedTraits = nullptr;
		}
	}
	Tests::FUtils::CleanupAfterTests();
	
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnimationAnimNextRuntimeTest_TraitStackBinding, "Animation.AnimNext.Runtime.TraitStackBinding", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAnimationAnimNextRuntimeTest_TraitStackBinding::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	{
		AUTO_REGISTER_ANIM_TRAIT(FTraitA_Base)
		AUTO_REGISTER_ANIM_TRAIT(FTraitAB_Add)
		AUTO_REGISTER_ANIM_TRAIT(FTraitAC_Add)

		UFactory* GraphFactory = NewObject<UUAFAnimGraphFactory>();
		UUAFAnimGraph* AnimationGraph = CastChecked<UUAFAnimGraph>(GraphFactory->FactoryCreateNew(UUAFAnimGraph::StaticClass(), GetTransientPackage(), TEXT("TestAnimNextGraph"), RF_Transient, nullptr, nullptr, NAME_None));
		UE_RETURN_ON_ERROR(AnimationGraph != nullptr, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to create animation graph");

		FScopedClearNodeTemplateRegistry ScopedClearNodeTemplateRegistry;
		FNodeTemplateRegistry& Registry = FNodeTemplateRegistry::Get();

		// Build a node with 2 stacks
		TArray<FTraitUID> NodeTemplateTraitList;
		NodeTemplateTraitList.Add(FTraitA_Base::TraitUID);
		NodeTemplateTraitList.Add(FTraitAB_Add::TraitUID);
		NodeTemplateTraitList.Add(FTraitAC_Add::TraitUID);
		NodeTemplateTraitList.Add(FTraitA_Base::TraitUID);
		NodeTemplateTraitList.Add(FTraitAC_Add::TraitUID);

		// Populate our node template registry
		TArray<uint8> NodeTemplateBuffer0;
		const FNodeTemplate* NodeTemplate0 = FNodeTemplateBuilder::BuildNodeTemplate(NodeTemplateTraitList, NodeTemplateBuffer0);

		FNodeTemplateRegistryHandle TemplateHandle0 = Registry.FindOrAdd(NodeTemplate0);
		AddErrorIfFalse(TemplateHandle0.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Registry should contain our template");

		TArray<FNodeHandle> NodeHandles;

		// Write our graph
		TArray<uint8> GraphSharedDataArchiveBuffer;
		TArray<TObjectPtr<UObject>> GraphReferencedObjects;
		TArray<FSoftObjectPath> GraphReferencedSoftObjects;
		{
			FTraitWriter TraitWriter;

			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplate0));

			// We don't have trait properties

			TraitWriter.BeginNodeWriting();
			TraitWriter.WriteNode(NodeHandles[0],
				[](uint32 TraitIndex, FName PropertyName)
				{
					return FString();
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.EndNodeWriting();

			AddErrorIfFalse(TraitWriter.GetErrorState() == FTraitWriter::EErrorState::None, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to write traits");
			GraphSharedDataArchiveBuffer = TraitWriter.GetGraphSharedData();
			GraphReferencedObjects = TraitWriter.GetGraphReferencedObjects();
			GraphReferencedSoftObjects = TraitWriter.GetGraphReferencedSoftObjects();
		}

		// Read our graph
		FTestUtils::LoadFromArchiveBuffer(*AnimationGraph, NodeHandles, GraphSharedDataArchiveBuffer);

		TSharedPtr<FAnimNextGraphInstance> GraphInstance = AnimationGraph->AllocateInstance();

		FExecutionContext Context(*GraphInstance.Get());

		// Validate from the first base trait
		{
			FAnimNextTraitHandle TraitHandle0(NodeHandles[0], 0);		// Point to first node, first base trait

			FTraitPtr TraitPtr0 = Context.AllocateNodeInstance(*GraphInstance.Get(), TraitHandle0);
			AddErrorIfFalse(TraitPtr0.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to allocate a node instance");

			FTraitStackBinding Stack0;
			AddErrorIfFalse(Context.GetStack(TraitPtr0, Stack0), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to bind to trait stack");

			// Test the first stack
			{
				AddErrorIfFalse(Stack0.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait stack binding should be valid");
				AddErrorIfFalse(Stack0.GetBaseTraitPtr() == TraitPtr0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected base trait ptr");

				// Stack binding copying and equality testing
				{
					FTraitStackBinding Stack0Copy = Stack0;
					AddErrorIfFalse(Stack0Copy.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait stack binding should be valid");
					AddErrorIfFalse(Stack0Copy.GetBaseTraitPtr() == TraitPtr0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected base trait ptr");
					AddErrorIfFalse(Stack0 == Stack0Copy, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait stack bindings should be equal");
					AddErrorIfFalse(!(Stack0 != Stack0Copy), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait stack bindings should be equal");

					Stack0Copy.Reset();
					AddErrorIfFalse(!Stack0Copy.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait stack binding should be not valid");

					FTraitStackBinding Stack1;
					AddErrorIfFalse(Context.GetStack(FWeakTraitPtr(TraitPtr0.GetNodeInstance(), 1), Stack1), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to bind to trait stack");
					AddErrorIfFalse(Stack1.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait stack binding should be valid");
					AddErrorIfFalse(Stack1.GetBaseTraitPtr() == TraitPtr0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected base trait ptr");
					AddErrorIfFalse(Stack0 == Stack1, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait stack bindings should be equal");
				}

				// Stack binding iteration from top to bottom
				{
					FTraitBinding TraitBinding;
					AddErrorIfFalse(Stack0.GetTopTrait(TraitBinding), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to retrieve top trait");
					AddErrorIfFalse(TraitBinding.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait binding should be valid");
					AddErrorIfFalse(TraitBinding.GetStack() == &Stack0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected trait stack binding");
					AddErrorIfFalse(TraitBinding.GetTraitIndex() == 2, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected stack local trait index");
					AddErrorIfFalse(TraitBinding.GetTraitPtr().GetTraitIndex() == 2, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected node local trait index");

					AddErrorIfFalse(Stack0.GetParentTrait(TraitBinding, TraitBinding), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to retrieve parent trait");
					AddErrorIfFalse(TraitBinding.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait binding should be valid");
					AddErrorIfFalse(TraitBinding.GetStack() == &Stack0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected trait stack binding");
					AddErrorIfFalse(TraitBinding.GetTraitIndex() == 1, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected stack local trait index");
					AddErrorIfFalse(TraitBinding.GetTraitPtr().GetTraitIndex() == 1, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected node local trait index");

					AddErrorIfFalse(Stack0.GetParentTrait(TraitBinding, TraitBinding), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to retrieve parent trait");
					AddErrorIfFalse(TraitBinding.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait binding should be valid");
					AddErrorIfFalse(TraitBinding.GetStack() == &Stack0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected trait stack binding");
					AddErrorIfFalse(TraitBinding.GetTraitIndex() == 0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected stack local trait index");
					AddErrorIfFalse(TraitBinding.GetTraitPtr().GetTraitIndex() == 0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected node local trait index");
					AddErrorIfFalse(TraitBinding.GetTraitPtr() == Stack0.GetBaseTraitPtr(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected stack base trait ptr");

					AddErrorIfFalse(!Stack0.GetParentTrait(TraitBinding, TraitBinding), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected parent trait");
				}

				// Stack binding iteration from bottom to top
				{
					FTraitBinding TraitBinding;
					AddErrorIfFalse(Stack0.GetBaseTrait(TraitBinding), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to retrieve base trait");
					AddErrorIfFalse(TraitBinding.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait binding should be valid");
					AddErrorIfFalse(TraitBinding.GetStack() == &Stack0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected trait stack binding");
					AddErrorIfFalse(TraitBinding.GetTraitIndex() == 0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected stack local trait index");
					AddErrorIfFalse(TraitBinding.GetTraitPtr().GetTraitIndex() == 0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected node local trait index");
					AddErrorIfFalse(TraitBinding.GetTraitPtr() == Stack0.GetBaseTraitPtr(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected stack base trait ptr");

					AddErrorIfFalse(Stack0.GetChildTrait(TraitBinding, TraitBinding), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to retrieve child trait");
					AddErrorIfFalse(TraitBinding.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait binding should be valid");
					AddErrorIfFalse(TraitBinding.GetStack() == &Stack0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected trait stack binding");
					AddErrorIfFalse(TraitBinding.GetTraitIndex() == 1, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected stack local trait index");
					AddErrorIfFalse(TraitBinding.GetTraitPtr().GetTraitIndex() == 1, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected node local trait index");

					AddErrorIfFalse(Stack0.GetChildTrait(TraitBinding, TraitBinding), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to retrieve child trait");
					AddErrorIfFalse(TraitBinding.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait binding should be valid");
					AddErrorIfFalse(TraitBinding.GetStack() == &Stack0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected trait stack binding");
					AddErrorIfFalse(TraitBinding.GetTraitIndex() == 2, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected stack local trait index");
					AddErrorIfFalse(TraitBinding.GetTraitPtr().GetTraitIndex() == 2, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected node local trait index");

					AddErrorIfFalse(!Stack0.GetChildTrait(TraitBinding, TraitBinding), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected child trait");
				}

				// Stack binding arbitrary iteration
				{
					AddErrorIfFalse(Stack0.GetNumTraits() == 3, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected trait stack size");

					FTraitBinding TraitBinding;
					AddErrorIfFalse(Stack0.GetTrait(0, TraitBinding), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to retrieve trait");
					AddErrorIfFalse(TraitBinding.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait binding should be valid");
					AddErrorIfFalse(TraitBinding.GetStack() == &Stack0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected trait stack binding");
					AddErrorIfFalse(TraitBinding.GetTraitIndex() == 0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected stack local trait index");
					AddErrorIfFalse(TraitBinding.GetTraitPtr().GetTraitIndex() == 0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected node local trait index");
					AddErrorIfFalse(TraitBinding.GetTraitPtr() == Stack0.GetBaseTraitPtr(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected stack base trait ptr");

					AddErrorIfFalse(Stack0.GetTrait(1, TraitBinding), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to retrieve trait");
					AddErrorIfFalse(TraitBinding.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait binding should be valid");
					AddErrorIfFalse(TraitBinding.GetStack() == &Stack0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected trait stack binding");
					AddErrorIfFalse(TraitBinding.GetTraitIndex() == 1, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected stack local trait index");
					AddErrorIfFalse(TraitBinding.GetTraitPtr().GetTraitIndex() == 1, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected node local trait index");

					AddErrorIfFalse(Stack0.GetTrait(2, TraitBinding), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to retrieve trait");
					AddErrorIfFalse(TraitBinding.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait binding should be valid");
					AddErrorIfFalse(TraitBinding.GetStack() == &Stack0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected trait stack binding");
					AddErrorIfFalse(TraitBinding.GetTraitIndex() == 2, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected stack local trait index");
					AddErrorIfFalse(TraitBinding.GetTraitPtr().GetTraitIndex() == 2, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected node local trait index");
				}

				// Interface query validation
				{
					TTraitBinding<IInterfaceC> Binding0C;
					AddErrorIfFalse(Stack0.GetInterface(Binding0C), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceC not found");
					AddErrorIfFalse(Binding0C.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceC binding not valid");
					AddErrorIfFalse(Binding0C.GetInterfaceUID() == IInterfaceC::InterfaceUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected interface UID found in trait binding");
					AddErrorIfFalse(Binding0C.GetTraitPtr().GetTraitIndex() == 2, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceC not found on expected trait");
					AddErrorIfFalse(Binding0C.GetTraitPtr().GetNodeInstance()->GetNodeHandle() == NodeHandles[0], "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceC not found on expected node");
					AddErrorIfFalse(Binding0C.GetSharedData<FTraitAC_Add::FSharedData>()->TraitUID == FTraitAC_Add::TraitUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected shared data in trait binding");
					AddErrorIfFalse(Binding0C.GetInstanceData<FTraitAC_Add::FInstanceData>()->TraitUID == FTraitAC_Add::TraitUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected instance data in trait binding");

					TTraitBinding<IInterfaceC> Binding0CCachedValid = Binding0C;
					TTraitBinding<IInterfaceC> Binding0CCachedInvalid = Binding0C;	// Invalidated when we query for the super just below

					AddErrorIfFalse(!Stack0.GetInterfaceSuper(Binding0C, Binding0C), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceC should not be found");
					AddErrorIfFalse(!Binding0CCachedInvalid.GetStackInterfaceSuper(Binding0CCachedInvalid), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceC should not be found");

					TTraitBinding<IInterfaceA> Binding0ACasted;
					AddErrorIfFalse(Binding0CCachedValid.AsInterface(Binding0ACasted), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA should be found");
					AddErrorIfFalse(Binding0ACasted.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA binding not valid");
					AddErrorIfFalse(Binding0ACasted.GetInterfaceUID() == IInterfaceA::InterfaceUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected interface UID found in trait binding");
					AddErrorIfFalse(Binding0ACasted.GetTraitPtr().GetTraitIndex() == 2, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA not found on expected trait");
					AddErrorIfFalse(Binding0ACasted.GetTraitPtr().GetNodeInstance()->GetNodeHandle() == NodeHandles[0], "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA not found on expected node");
					AddErrorIfFalse(Binding0ACasted.GetSharedData<FTraitAC_Add::FSharedData>()->TraitUID == FTraitAC_Add::TraitUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected shared data in trait binding");
					AddErrorIfFalse(Binding0ACasted.GetInstanceData<FTraitAC_Add::FInstanceData>()->TraitUID == FTraitAC_Add::TraitUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected instance data in trait binding");
					AddErrorIfFalse(Binding0CCachedValid != Binding0ACasted, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait bindings should not be equal");

					TTraitBinding<IInterfaceB> Binding0BCasted;
					AddErrorIfFalse(!Binding0CCachedValid.AsInterface(Binding0BCasted), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceB should not be found");

					// Test the same interface query but from an existing trait binding
					TTraitBinding<IInterfaceC> Binding0CFromTrait;
					AddErrorIfFalse(Binding0CCachedValid.GetStackInterface(Binding0CFromTrait), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceC not found");
					AddErrorIfFalse(Binding0CFromTrait.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceC binding not valid");
					AddErrorIfFalse(Binding0CFromTrait.GetInterfaceUID() == IInterfaceC::InterfaceUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected interface UID found in trait binding");
					AddErrorIfFalse(Binding0CFromTrait.GetTraitPtr().GetTraitIndex() == 2, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceC not found on expected trait");
					AddErrorIfFalse(Binding0CFromTrait.GetTraitPtr().GetNodeInstance()->GetNodeHandle() == NodeHandles[0], "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceC not found on expected node");
					AddErrorIfFalse(Binding0CFromTrait.GetSharedData<FTraitAC_Add::FSharedData>()->TraitUID == FTraitAC_Add::TraitUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected shared data in trait binding");
					AddErrorIfFalse(Binding0CFromTrait.GetInstanceData<FTraitAC_Add::FInstanceData>()->TraitUID == FTraitAC_Add::TraitUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected instance data in trait binding");
					AddErrorIfFalse(Binding0CCachedValid == Binding0CFromTrait, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait bindings should be equal");

					TTraitBinding<IInterfaceB> Binding0B;
					AddErrorIfFalse(Stack0.GetInterface(Binding0B), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceB not found");
					AddErrorIfFalse(Binding0B.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceB binding not valid");
					AddErrorIfFalse(Binding0B.GetInterfaceUID() == IInterfaceB::InterfaceUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected interface UID found in trait binding");
					AddErrorIfFalse(Binding0B.GetTraitPtr().GetTraitIndex() == 1, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceB not found on expected trait");
					AddErrorIfFalse(Binding0B.GetTraitPtr().GetNodeInstance()->GetNodeHandle() == NodeHandles[0], "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceB not found on expected node");
					AddErrorIfFalse(Binding0B.GetSharedData<FTraitAB_Add::FSharedData>()->TraitUID == FTraitAB_Add::TraitUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected shared data in trait binding");
					AddErrorIfFalse(Binding0B.GetInstanceData<FTraitAB_Add::FInstanceData>()->TraitUID == FTraitAB_Add::TraitUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected instance data in trait binding");
					AddErrorIfFalse(!Stack0.GetInterfaceSuper(Binding0B, Binding0B), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceB should not be found");
					AddErrorIfFalse(!Binding0B.GetStackInterfaceSuper(Binding0B), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceB should not be found");

					TTraitBinding<IInterfaceA> Binding0A;
					AddErrorIfFalse(Stack0.GetInterface(Binding0A), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA not found");
					AddErrorIfFalse(Binding0A.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA binding not valid");
					AddErrorIfFalse(Binding0A.GetInterfaceUID() == IInterfaceA::InterfaceUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected interface UID found in trait binding");
					AddErrorIfFalse(Binding0A.GetTraitPtr().GetTraitIndex() == 2, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA not found on expected trait");
					AddErrorIfFalse(Binding0A.GetTraitPtr().GetNodeInstance()->GetNodeHandle() == NodeHandles[0], "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA not found on expected node");
					AddErrorIfFalse(Binding0A.GetSharedData<FTraitAC_Add::FSharedData>()->TraitUID == FTraitAC_Add::TraitUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected shared data in trait binding");
					AddErrorIfFalse(Binding0A.GetInstanceData<FTraitAC_Add::FInstanceData>()->TraitUID == FTraitAC_Add::TraitUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected instance data in trait binding");

					AddErrorIfFalse(Stack0.GetInterfaceSuper(Binding0A, Binding0A), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA not found");
					AddErrorIfFalse(Binding0A.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA binding not valid");
					AddErrorIfFalse(Binding0A.GetInterfaceUID() == IInterfaceA::InterfaceUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected interface UID found in trait binding");
					AddErrorIfFalse(Binding0A.GetTraitPtr().GetTraitIndex() == 1, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA not found on expected trait");
					AddErrorIfFalse(Binding0A.GetTraitPtr().GetNodeInstance()->GetNodeHandle() == NodeHandles[0], "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA not found on expected node");
					AddErrorIfFalse(Binding0A.GetSharedData<FTraitAB_Add::FSharedData>()->TraitUID == FTraitAB_Add::TraitUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected shared data in trait binding");
					AddErrorIfFalse(Binding0A.GetInstanceData<FTraitAB_Add::FInstanceData>()->TraitUID == FTraitAB_Add::TraitUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected instance data in trait binding");

					TTraitBinding<IInterfaceA> Binding0ACached = Binding0A;

					AddErrorIfFalse(Stack0.GetInterfaceSuper(Binding0A, Binding0A), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA not found");
					AddErrorIfFalse(Binding0A.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA binding not valid");
					AddErrorIfFalse(Binding0A.GetInterfaceUID() == IInterfaceA::InterfaceUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected interface UID found in trait binding");
					AddErrorIfFalse(Binding0A.GetTraitPtr().GetTraitIndex() == 0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA not found on expected trait");
					AddErrorIfFalse(Binding0A.GetTraitPtr().GetNodeInstance()->GetNodeHandle() == NodeHandles[0], "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA not found on expected node");
					AddErrorIfFalse(Binding0A.GetSharedData<FTraitA_Base::FSharedData>()->TraitUID == FTraitA_Base::TraitUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected shared data in trait binding");
					AddErrorIfFalse(Binding0A.GetInstanceData<FTraitA_Base::FInstanceData>()->TraitUID == FTraitA_Base::TraitUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected instance data in trait binding");
					AddErrorIfFalse(!Stack0.GetInterfaceSuper(Binding0A, Binding0A), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA should not be found");

					// Test the same interface query but from an existing trait binding
					TTraitBinding<IInterfaceA> Binding0AFromTrait;
					AddErrorIfFalse(Binding0ACached.GetStackInterfaceSuper(Binding0AFromTrait), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA not found");
					AddErrorIfFalse(Binding0AFromTrait.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA binding not valid");
					AddErrorIfFalse(Binding0AFromTrait.GetInterfaceUID() == IInterfaceA::InterfaceUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected interface UID found in trait binding");
					AddErrorIfFalse(Binding0AFromTrait.GetTraitPtr().GetTraitIndex() == 0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA not found on expected trait");
					AddErrorIfFalse(Binding0AFromTrait.GetTraitPtr().GetNodeInstance()->GetNodeHandle() == NodeHandles[0], "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA not found on expected node");
					AddErrorIfFalse(Binding0AFromTrait.GetSharedData<FTraitA_Base::FSharedData>()->TraitUID == FTraitA_Base::TraitUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected shared data in trait binding");
					AddErrorIfFalse(Binding0AFromTrait.GetInstanceData<FTraitA_Base::FInstanceData>()->TraitUID == FTraitA_Base::TraitUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected instance data in trait binding");
					AddErrorIfFalse(!Binding0AFromTrait.GetStackInterfaceSuper(Binding0AFromTrait), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA should not be found");
				}

				// Trait binding misc
				{
					FTraitBinding TraitBinding;
					AddErrorIfFalse(Stack0.GetTrait(0, TraitBinding), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to retrieve trait");
					AddErrorIfFalse(TraitBinding.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait binding should be valid");

					FTraitBinding TraitBinding1 = TraitBinding;
					AddErrorIfFalse(TraitBinding1.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait binding should be valid");
					AddErrorIfFalse(TraitBinding == TraitBinding1, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait bindings should be equal");
					AddErrorIfFalse(!(TraitBinding != TraitBinding1), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait bindings should be equal");
				}
			}

			FAnimNextTraitHandle TraitHandle3(NodeHandles[0], 3);		// Point to first node, second base trait

			FTraitPtr TraitPtr3 = Context.AllocateNodeInstance(*GraphInstance.Get(), TraitHandle3);
			AddErrorIfFalse(TraitPtr3.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to allocate a node instance");

			FTraitStackBinding Stack3;
			AddErrorIfFalse(Context.GetStack(TraitPtr3, Stack3), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to bind to trait stack");

			// Test the second stack
			{
				AddErrorIfFalse(Stack3.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait stack binding should be valid");
				AddErrorIfFalse(Stack3.GetBaseTraitPtr() == TraitPtr3, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected base trait ptr");

				// Stack binding copying and equality testing
				{
					AddErrorIfFalse(Stack0 != Stack3, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait stack bindings should not be equal");

					FTraitStackBinding Stack3Copy = Stack3;
					AddErrorIfFalse(Stack3Copy.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait stack binding should be valid");
					AddErrorIfFalse(Stack3Copy.GetBaseTraitPtr() == TraitPtr3, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected base trait ptr");
					AddErrorIfFalse(Stack3 == Stack3Copy, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait stack bindings should be equal");
					AddErrorIfFalse(!(Stack3 != Stack3Copy), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait stack bindings should be equal");

					Stack3Copy.Reset();
					AddErrorIfFalse(!Stack3Copy.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait stack binding should be not valid");

					FTraitStackBinding Stack4;
					AddErrorIfFalse(Context.GetStack(FWeakTraitPtr(TraitPtr3.GetNodeInstance(), 4), Stack4), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to bind to trait stack");
					AddErrorIfFalse(Stack4.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait stack binding should be valid");
					AddErrorIfFalse(Stack4.GetBaseTraitPtr() == TraitPtr3, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected base trait ptr");
					AddErrorIfFalse(Stack3 == Stack4, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait stack bindings should be equal");
				}

				// Stack binding iteration from top to bottom
				{
					FTraitBinding TraitBinding;
					AddErrorIfFalse(Stack3.GetTopTrait(TraitBinding), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to retrieve top trait");
					AddErrorIfFalse(TraitBinding.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait binding should be valid");
					AddErrorIfFalse(TraitBinding.GetStack() == &Stack3, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected trait stack binding");
					AddErrorIfFalse(TraitBinding.GetTraitIndex() == 1, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected stack local trait index");
					AddErrorIfFalse(TraitBinding.GetTraitPtr().GetTraitIndex() == 4, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected node local trait index");

					AddErrorIfFalse(Stack3.GetParentTrait(TraitBinding, TraitBinding), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to retrieve parent trait");
					AddErrorIfFalse(TraitBinding.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait binding should be valid");
					AddErrorIfFalse(TraitBinding.GetStack() == &Stack3, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected trait stack binding");
					AddErrorIfFalse(TraitBinding.GetTraitIndex() == 0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected stack local trait index");
					AddErrorIfFalse(TraitBinding.GetTraitPtr().GetTraitIndex() == 3, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected node local trait index");
					AddErrorIfFalse(TraitBinding.GetTraitPtr() == Stack3.GetBaseTraitPtr(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected stack base trait ptr");

					AddErrorIfFalse(!Stack3.GetParentTrait(TraitBinding, TraitBinding), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected parent trait");
				}

				// Stack binding iteration from bottom to top
				{
					FTraitBinding TraitBinding;
					AddErrorIfFalse(Stack3.GetBaseTrait(TraitBinding), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to retrieve base trait");
					AddErrorIfFalse(TraitBinding.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait binding should be valid");
					AddErrorIfFalse(TraitBinding.GetStack() == &Stack3, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected trait stack binding");
					AddErrorIfFalse(TraitBinding.GetTraitIndex() == 0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected stack local trait index");
					AddErrorIfFalse(TraitBinding.GetTraitPtr().GetTraitIndex() == 3, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected node local trait index");
					AddErrorIfFalse(TraitBinding.GetTraitPtr() == Stack3.GetBaseTraitPtr(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected stack base trait ptr");

					AddErrorIfFalse(Stack3.GetChildTrait(TraitBinding, TraitBinding), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to retrieve child trait");
					AddErrorIfFalse(TraitBinding.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait binding should be valid");
					AddErrorIfFalse(TraitBinding.GetStack() == &Stack3, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected trait stack binding");
					AddErrorIfFalse(TraitBinding.GetTraitIndex() == 1, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected stack local trait index");
					AddErrorIfFalse(TraitBinding.GetTraitPtr().GetTraitIndex() == 4, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected node local trait index");

					AddErrorIfFalse(!Stack3.GetChildTrait(TraitBinding, TraitBinding), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected child trait");
				}

				// Stack binding arbitrary iteration
				{
					AddErrorIfFalse(Stack3.GetNumTraits() == 2, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected trait stack size");

					FTraitBinding TraitBinding;
					AddErrorIfFalse(Stack3.GetTrait(0, TraitBinding), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to retrieve trait");
					AddErrorIfFalse(TraitBinding.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait binding should be valid");
					AddErrorIfFalse(TraitBinding.GetStack() == &Stack3, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected trait stack binding");
					AddErrorIfFalse(TraitBinding.GetTraitIndex() == 0, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected stack local trait index");
					AddErrorIfFalse(TraitBinding.GetTraitPtr().GetTraitIndex() == 3, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected node local trait index");
					AddErrorIfFalse(TraitBinding.GetTraitPtr() == Stack3.GetBaseTraitPtr(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected stack base trait ptr");

					AddErrorIfFalse(Stack3.GetTrait(1, TraitBinding), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Failed to retrieve trait");
					AddErrorIfFalse(TraitBinding.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Trait binding should be valid");
					AddErrorIfFalse(TraitBinding.GetStack() == &Stack3, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected trait stack binding");
					AddErrorIfFalse(TraitBinding.GetTraitIndex() == 1, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected stack local trait index");
					AddErrorIfFalse(TraitBinding.GetTraitPtr().GetTraitIndex() == 4, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected node local trait index");
				}

				// AC -> A
				// Interface query validation
				{
					TTraitBinding<IInterfaceC> Binding3C;
					AddErrorIfFalse(Stack3.GetInterface(Binding3C), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceC not found");
					AddErrorIfFalse(Binding3C.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceC binding not valid");
					AddErrorIfFalse(Binding3C.GetInterfaceUID() == IInterfaceC::InterfaceUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected interface UID found in trait binding");
					AddErrorIfFalse(Binding3C.GetTraitPtr().GetTraitIndex() == 4, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceC not found on expected trait");
					AddErrorIfFalse(Binding3C.GetTraitPtr().GetNodeInstance()->GetNodeHandle() == NodeHandles[0], "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceC not found on expected node");
					AddErrorIfFalse(Binding3C.GetSharedData<FTraitAC_Add::FSharedData>()->TraitUID == FTraitAC_Add::TraitUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected shared data in trait binding");
					AddErrorIfFalse(Binding3C.GetInstanceData<FTraitAC_Add::FInstanceData>()->TraitUID == FTraitAC_Add::TraitUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected instance data in trait binding");
					AddErrorIfFalse(!Stack3.GetInterfaceSuper(Binding3C, Binding3C), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceC should not be found");

					TTraitBinding<IInterfaceB> Binding3B;
					AddErrorIfFalse(!Stack3.GetInterface(Binding3B), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceB should not be found");
					AddErrorIfFalse(!Binding3B.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceB binding not valid");

					TTraitBinding<IInterfaceA> Binding3A;
					AddErrorIfFalse(Stack3.GetInterface(Binding3A), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA not found");
					AddErrorIfFalse(Binding3A.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA binding not valid");
					AddErrorIfFalse(Binding3A.GetInterfaceUID() == IInterfaceA::InterfaceUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected interface UID found in trait binding");
					AddErrorIfFalse(Binding3A.GetTraitPtr().GetTraitIndex() == 4, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA not found on expected trait");
					AddErrorIfFalse(Binding3A.GetTraitPtr().GetNodeInstance()->GetNodeHandle() == NodeHandles[0], "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA not found on expected node");
					AddErrorIfFalse(Binding3A.GetSharedData<FTraitAC_Add::FSharedData>()->TraitUID == FTraitAC_Add::TraitUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected shared data in trait binding");
					AddErrorIfFalse(Binding3A.GetInstanceData<FTraitAC_Add::FInstanceData>()->TraitUID == FTraitAC_Add::TraitUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected instance data in trait binding");

					AddErrorIfFalse(Stack3.GetInterfaceSuper(Binding3A, Binding3A), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA not found");
					AddErrorIfFalse(Binding3A.IsValid(), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA binding not valid");
					AddErrorIfFalse(Binding3A.GetInterfaceUID() == IInterfaceA::InterfaceUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected interface UID found in trait binding");
					AddErrorIfFalse(Binding3A.GetTraitPtr().GetTraitIndex() == 3, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA not found on expected trait");
					AddErrorIfFalse(Binding3A.GetTraitPtr().GetNodeInstance()->GetNodeHandle() == NodeHandles[0], "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA not found on expected node");
					AddErrorIfFalse(Binding3A.GetSharedData<FTraitA_Base::FSharedData>()->TraitUID == FTraitA_Base::TraitUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected shared data in trait binding");
					AddErrorIfFalse(Binding3A.GetInstanceData<FTraitA_Base::FInstanceData>()->TraitUID == FTraitA_Base::TraitUID, "FAnimationAnimNextRuntimeTest_TraitStackBinding -> Unexpected instance data in trait binding");
					AddErrorIfFalse(!Stack3.GetInterfaceSuper(Binding3A, Binding3A), "FAnimationAnimNextRuntimeTest_TraitStackBinding -> InterfaceA should not be found");
				}
			}
		}
	}

	Tests::FUtils::CleanupAfterTests();

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnimationAnimNextRuntimeTest_TraitSerialization, "Animation.AnimNext.Runtime.TraitSerialization", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAnimationAnimNextRuntimeTest_TraitSerialization::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	{
		FScopedClearNodeTemplateRegistry ScopedClearNodeTemplateRegistry;

		AUTO_REGISTER_ANIM_TRAIT(FTraitSerialization_Base)
		AUTO_REGISTER_ANIM_TRAIT(FTraitSerialization_Add)
		AUTO_REGISTER_ANIM_TRAIT(FTraitNativeSerialization_Add)

		UFactory* GraphFactory = NewObject<UUAFAnimGraphFactory>();
		UUAFAnimGraph* AnimationGraph = CastChecked<UUAFAnimGraph>(GraphFactory->FactoryCreateNew(UUAFAnimGraph::StaticClass(), GetTransientPackage(), TEXT("TestAnimNextGraph"), RF_Transient, nullptr, nullptr, NAME_None));
		UE_RETURN_ON_ERROR(AnimationGraph != nullptr, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Failed to create animation graph");

		TArray<FTraitUID> NodeTemplateTraitList;
		NodeTemplateTraitList.Add(FTraitSerialization_Base::TraitUID);
		NodeTemplateTraitList.Add(FTraitSerialization_Add::TraitUID);
		NodeTemplateTraitList.Add(FTraitNativeSerialization_Add::TraitUID);

		// Populate our node template registry
		TArray<uint8> NodeTemplateBuffer0;
		const FNodeTemplate* NodeTemplate0 = FNodeTemplateBuilder::BuildNodeTemplate(NodeTemplateTraitList, NodeTemplateBuffer0);

		FNodeTemplateRegistryHandle TemplateHandle0 = FNodeTemplateRegistry::Get().FindOrAdd(NodeTemplate0);
		AddErrorIfFalse(TemplateHandle0.IsValid(), "FAnimationAnimNextRuntimeTest_TraitSerialization -> Registry should contain our template");

		FTraitSerialization_Base::FSharedData TraitBaseRef0;
		TraitBaseRef0.Integer = 1651;
		TraitBaseRef0.IntegerArray[0] = 1071;
		TraitBaseRef0.IntegerArray[1] = -158;
		TraitBaseRef0.IntegerArray[2] = 88116;
		TraitBaseRef0.IntegerArray[3] = 0x417;
		TraitBaseRef0.IntegerTArray = { -8162, 88152, 0x8152f };
		TraitBaseRef0.Vector = FVector(0.1917, 12435.1, -18200.1726);
		TraitBaseRef0.VectorArray[0] = FVector(192.1716, -1927.115, 99176.12);
		TraitBaseRef0.VectorArray[1] = FVector(961.811, -18956.117, 81673.44);
		TraitBaseRef0.VectorTArray = { FVector(-1927.8771, 1826.9917, -123.1555), FVector(9177.011, -71.44, -917.88), FVector(123.91, 852.11, -81652.1) };
		TraitBaseRef0.String = TEXT("sample string 123");
		TraitBaseRef0.Name = FName(TEXT("sample name 999178"));

		FTraitSerialization_Add::FSharedData TraitAddRef0;
		TraitAddRef0.Integer = 16511;
		TraitAddRef0.IntegerArray[0] = 10711;
		TraitAddRef0.IntegerArray[1] = -1581;
		TraitAddRef0.IntegerArray[2] = 881161;
		TraitAddRef0.IntegerArray[3] = 0x4171;
		TraitAddRef0.IntegerTArray = { -81621, 881521, 0x8152f1 };
		TraitAddRef0.Vector = FVector(0.19171, 12435.11, -18200.17261);
		TraitAddRef0.VectorArray[0] = FVector(192.17161, -1927.1151, 99176.121);
		TraitAddRef0.VectorArray[1] = FVector(961.8111, -18956.1171, 81673.441);
		TraitAddRef0.VectorTArray = { FVector(-1927.87711, 1826.99171, -123.15551), FVector(9177.0111, -71.441, -917.881), FVector(123.911, 852.111, -81652.11) };
		TraitAddRef0.String = TEXT("sample string 1231");
		TraitAddRef0.Name = FName(TEXT("sample name 9991781"));

		FTraitNativeSerialization_Add::FSharedData TraitNativeRef0;
		TraitNativeRef0.Integer = 16514;
		TraitNativeRef0.IntegerArray[0] = 10714;
		TraitNativeRef0.IntegerArray[1] = -1584;
		TraitNativeRef0.IntegerArray[2] = 881164;
		TraitNativeRef0.IntegerArray[3] = 0x4174;
		TraitNativeRef0.IntegerTArray = { -81624, 881524, 0x8152f4 };
		TraitNativeRef0.Vector = FVector(0.19174, 12435.14, -18200.17264);
		TraitNativeRef0.VectorArray[0] = FVector(192.17164, -1927.1154, 99176.124);
		TraitNativeRef0.VectorArray[1] = FVector(961.8114, -18956.1174, 81673.444);
		TraitNativeRef0.VectorTArray = { FVector(-1927.87714, 1826.99174, -123.15554), FVector(9177.0114, -71.444, -917.884), FVector(123.914, 852.114, -81652.14) };
		TraitNativeRef0.String = TEXT("sample string 1234");
		TraitNativeRef0.Name = FName(TEXT("sample name 9991784"));

		FTraitSerialization_Base::FSharedData TraitBaseRef1;
		TraitBaseRef1.Integer = 16512;
		TraitBaseRef1.IntegerArray[0] = 10712;
		TraitBaseRef1.IntegerArray[1] = -1582;
		TraitBaseRef1.IntegerArray[2] = 881162;
		TraitBaseRef1.IntegerArray[3] = 0x4172;
		TraitBaseRef1.IntegerTArray = { -81622, 881522, 0x8152f2 };
		TraitBaseRef1.Vector = FVector(0.19172, 12435.12, -18200.17262);
		TraitBaseRef1.VectorArray[0] = FVector(192.17162, -1927.1152, 99176.122);
		TraitBaseRef1.VectorArray[1] = FVector(961.8112, -18956.1172, 81673.442);
		TraitBaseRef1.VectorTArray = { FVector(-1927.87712, 1826.99172, -123.15552), FVector(9177.0112, -71.442, -917.882), FVector(123.912, 852.112, -81652.12) };
		TraitBaseRef1.String = TEXT("sample string 1232");
		TraitBaseRef1.Name = FName(TEXT("sample name 9991782"));

		FTraitSerialization_Add::FSharedData TraitAddRef1;
		TraitAddRef1.Integer = 16513;
		TraitAddRef1.IntegerArray[0] = 10713;
		TraitAddRef1.IntegerArray[1] = -1583;
		TraitAddRef1.IntegerArray[2] = 881163;
		TraitAddRef1.IntegerArray[3] = 0x4173;
		TraitAddRef1.IntegerTArray = { -81623, 881523, 0x8152f3 };
		TraitAddRef1.Vector = FVector(0.19173, 12435.13, -18200.17263);
		TraitAddRef1.VectorArray[0] = FVector(192.17163, -1927.1153, 99176.123);
		TraitAddRef1.VectorArray[1] = FVector(961.8113, -18956.1173, 81673.443);
		TraitAddRef1.VectorTArray = { FVector(-1927.87713, 1826.99173, -123.15553), FVector(9177.0113, -71.443, -917.883), FVector(123.913, 852.113, -81652.13) };
		TraitAddRef1.String = TEXT("sample string 1233");
		TraitAddRef1.Name = FName(TEXT("sample name 9991783"));

		FTraitNativeSerialization_Add::FSharedData TraitNativeRef1;
		TraitNativeRef1.Integer = 16515;
		TraitNativeRef1.IntegerArray[0] = 10715;
		TraitNativeRef1.IntegerArray[1] = -1585;
		TraitNativeRef1.IntegerArray[2] = 881165;
		TraitNativeRef1.IntegerArray[3] = 0x4175;
		TraitNativeRef1.IntegerTArray = { -81625, 881525, 0x8152f5 };
		TraitNativeRef1.Vector = FVector(0.19175, 12435.15, -18200.17265);
		TraitNativeRef1.VectorArray[0] = FVector(192.17165, -1927.1155, 99176.125);
		TraitNativeRef1.VectorArray[1] = FVector(961.8115, -18956.1175, 81673.445);
		TraitNativeRef1.VectorTArray = { FVector(-1927.87715, 1826.99175, -123.15555), FVector(9177.0115, -71.445, -917.885), FVector(123.915, 852.115, -81652.15) };
		TraitNativeRef1.String = TEXT("sample string 1235");
		TraitNativeRef1.Name = FName(TEXT("sample name 9991785"));

		TArray<FNodeHandle> NodeHandles;

		// Write our graph
		TArray<uint8> GraphSharedDataArchiveBuffer;
		TArray<TObjectPtr<UObject>> GraphReferencedObjects;
		TArray<FSoftObjectPath> GraphReferencedSoftObjects;
		{
			FTraitWriter TraitWriter;

			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplate0));
			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplate0));

			// We don't have trait properties
			TArray<TMap<FName, FString>> TraitProperties0;
			TraitProperties0.AddDefaulted(NodeTemplateTraitList.Num());

			TraitProperties0[0].Add(TEXT("Integer"), ToString<FTraitSerialization_Base::FSharedData>(TEXT("Integer"), TraitBaseRef0.Integer));
			TraitProperties0[0].Add(TEXT("IntegerArray"), ToString<FTraitSerialization_Base::FSharedData>(TEXT("IntegerArray"), TraitBaseRef0.IntegerArray));
			TraitProperties0[0].Add(TEXT("IntegerTArray"), ToString<FTraitSerialization_Base::FSharedData>(TEXT("IntegerTArray"), TraitBaseRef0.IntegerTArray));
			TraitProperties0[0].Add(TEXT("Vector"), ToString<FTraitSerialization_Base::FSharedData>(TEXT("Vector"), TraitBaseRef0.Vector));
			TraitProperties0[0].Add(TEXT("VectorArray"), ToString<FTraitSerialization_Base::FSharedData>(TEXT("VectorArray"), TraitBaseRef0.VectorArray));
			TraitProperties0[0].Add(TEXT("VectorTArray"), ToString<FTraitSerialization_Base::FSharedData>(TEXT("VectorTArray"), TraitBaseRef0.VectorTArray));
			TraitProperties0[0].Add(TEXT("String"), ToString<FTraitSerialization_Base::FSharedData>(TEXT("String"), TraitBaseRef0.String));
			TraitProperties0[0].Add(TEXT("Name"), ToString<FTraitSerialization_Base::FSharedData>(TEXT("Name"), TraitBaseRef0.Name));

			TraitProperties0[1].Add(TEXT("Integer"), ToString<FTraitSerialization_Add::FSharedData>(TEXT("Integer"), TraitAddRef0.Integer));
			TraitProperties0[1].Add(TEXT("IntegerArray"), ToString<FTraitSerialization_Add::FSharedData>(TEXT("IntegerArray"), TraitAddRef0.IntegerArray));
			TraitProperties0[1].Add(TEXT("IntegerTArray"), ToString<FTraitSerialization_Add::FSharedData>(TEXT("IntegerTArray"), TraitAddRef0.IntegerTArray));
			TraitProperties0[1].Add(TEXT("Vector"), ToString<FTraitSerialization_Add::FSharedData>(TEXT("Vector"), TraitAddRef0.Vector));
			TraitProperties0[1].Add(TEXT("VectorArray"), ToString<FTraitSerialization_Add::FSharedData>(TEXT("VectorArray"), TraitAddRef0.VectorArray));
			TraitProperties0[1].Add(TEXT("VectorTArray"), ToString<FTraitSerialization_Add::FSharedData>(TEXT("VectorTArray"), TraitAddRef0.VectorTArray));
			TraitProperties0[1].Add(TEXT("String"), ToString<FTraitSerialization_Add::FSharedData>(TEXT("String"), TraitAddRef0.String));
			TraitProperties0[1].Add(TEXT("Name"), ToString<FTraitSerialization_Add::FSharedData>(TEXT("Name"), TraitAddRef0.Name));

			TraitProperties0[2].Add(TEXT("Integer"), ToString<FTraitNativeSerialization_Add::FSharedData>(TEXT("Integer"), TraitNativeRef0.Integer));
			TraitProperties0[2].Add(TEXT("IntegerArray"), ToString<FTraitNativeSerialization_Add::FSharedData>(TEXT("IntegerArray"), TraitNativeRef0.IntegerArray));
			TraitProperties0[2].Add(TEXT("IntegerTArray"), ToString<FTraitNativeSerialization_Add::FSharedData>(TEXT("IntegerTArray"), TraitNativeRef0.IntegerTArray));
			TraitProperties0[2].Add(TEXT("Vector"), ToString<FTraitNativeSerialization_Add::FSharedData>(TEXT("Vector"), TraitNativeRef0.Vector));
			TraitProperties0[2].Add(TEXT("VectorArray"), ToString<FTraitNativeSerialization_Add::FSharedData>(TEXT("VectorArray"), TraitNativeRef0.VectorArray));
			TraitProperties0[2].Add(TEXT("VectorTArray"), ToString<FTraitNativeSerialization_Add::FSharedData>(TEXT("VectorTArray"), TraitNativeRef0.VectorTArray));
			TraitProperties0[2].Add(TEXT("String"), ToString<FTraitNativeSerialization_Add::FSharedData>(TEXT("String"), TraitNativeRef0.String));
			TraitProperties0[2].Add(TEXT("Name"), ToString<FTraitNativeSerialization_Add::FSharedData>(TEXT("Name"), TraitNativeRef0.Name));

			TArray<TMap<FName, FString>> TraitProperties1;
			TraitProperties1.AddDefaulted(NodeTemplateTraitList.Num());

			TraitProperties1[0].Add(TEXT("Integer"), ToString<FTraitSerialization_Base::FSharedData>(TEXT("Integer"), TraitBaseRef1.Integer));
			TraitProperties1[0].Add(TEXT("IntegerArray"), ToString<FTraitSerialization_Base::FSharedData>(TEXT("IntegerArray"), TraitBaseRef1.IntegerArray));
			TraitProperties1[0].Add(TEXT("IntegerTArray"), ToString<FTraitSerialization_Base::FSharedData>(TEXT("IntegerTArray"), TraitBaseRef1.IntegerTArray));
			TraitProperties1[0].Add(TEXT("Vector"), ToString<FTraitSerialization_Base::FSharedData>(TEXT("Vector"), TraitBaseRef1.Vector));
			TraitProperties1[0].Add(TEXT("VectorArray"), ToString<FTraitSerialization_Base::FSharedData>(TEXT("VectorArray"), TraitBaseRef1.VectorArray));
			TraitProperties1[0].Add(TEXT("VectorTArray"), ToString<FTraitSerialization_Base::FSharedData>(TEXT("VectorTArray"), TraitBaseRef1.VectorTArray));
			TraitProperties1[0].Add(TEXT("String"), ToString<FTraitSerialization_Base::FSharedData>(TEXT("String"), TraitBaseRef1.String));
			TraitProperties1[0].Add(TEXT("Name"), ToString<FTraitSerialization_Base::FSharedData>(TEXT("Name"), TraitBaseRef1.Name));

			TraitProperties1[1].Add(TEXT("Integer"), ToString<FTraitSerialization_Add::FSharedData>(TEXT("Integer"), TraitAddRef1.Integer));
			TraitProperties1[1].Add(TEXT("IntegerArray"), ToString<FTraitSerialization_Add::FSharedData>(TEXT("IntegerArray"), TraitAddRef1.IntegerArray));
			TraitProperties1[1].Add(TEXT("IntegerTArray"), ToString<FTraitSerialization_Add::FSharedData>(TEXT("IntegerTArray"), TraitAddRef1.IntegerTArray));
			TraitProperties1[1].Add(TEXT("Vector"), ToString<FTraitSerialization_Add::FSharedData>(TEXT("Vector"), TraitAddRef1.Vector));
			TraitProperties1[1].Add(TEXT("VectorArray"), ToString<FTraitSerialization_Add::FSharedData>(TEXT("VectorArray"), TraitAddRef1.VectorArray));
			TraitProperties1[1].Add(TEXT("VectorTArray"), ToString<FTraitSerialization_Add::FSharedData>(TEXT("VectorTArray"), TraitAddRef1.VectorTArray));
			TraitProperties1[1].Add(TEXT("String"), ToString<FTraitSerialization_Add::FSharedData>(TEXT("String"), TraitAddRef1.String));
			TraitProperties1[1].Add(TEXT("Name"), ToString<FTraitSerialization_Add::FSharedData>(TEXT("Name"), TraitAddRef1.Name));

			TraitProperties1[2].Add(TEXT("Integer"), ToString<FTraitNativeSerialization_Add::FSharedData>(TEXT("Integer"), TraitNativeRef1.Integer));
			TraitProperties1[2].Add(TEXT("IntegerArray"), ToString<FTraitNativeSerialization_Add::FSharedData>(TEXT("IntegerArray"), TraitNativeRef1.IntegerArray));
			TraitProperties1[2].Add(TEXT("IntegerTArray"), ToString<FTraitNativeSerialization_Add::FSharedData>(TEXT("IntegerTArray"), TraitNativeRef1.IntegerTArray));
			TraitProperties1[2].Add(TEXT("Vector"), ToString<FTraitNativeSerialization_Add::FSharedData>(TEXT("Vector"), TraitNativeRef1.Vector));
			TraitProperties1[2].Add(TEXT("VectorArray"), ToString<FTraitNativeSerialization_Add::FSharedData>(TEXT("VectorArray"), TraitNativeRef1.VectorArray));
			TraitProperties1[2].Add(TEXT("VectorTArray"), ToString<FTraitNativeSerialization_Add::FSharedData>(TEXT("VectorTArray"), TraitNativeRef1.VectorTArray));
			TraitProperties1[2].Add(TEXT("String"), ToString<FTraitNativeSerialization_Add::FSharedData>(TEXT("String"), TraitNativeRef1.String));
			TraitProperties1[2].Add(TEXT("Name"), ToString<FTraitNativeSerialization_Add::FSharedData>(TEXT("Name"), TraitNativeRef1.Name));

			TraitWriter.BeginNodeWriting();
			TraitWriter.WriteNode(NodeHandles[0],
				[&TraitProperties0](uint32 TraitIndex, FName PropertyName)
				{
					return TraitProperties0[TraitIndex][PropertyName];
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.WriteNode(NodeHandles[1],
				[&TraitProperties1](uint32 TraitIndex, FName PropertyName)
				{
					return TraitProperties1[TraitIndex][PropertyName];
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.EndNodeWriting();

			AddErrorIfFalse(TraitWriter.GetErrorState() == FTraitWriter::EErrorState::None, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Failed to write traits");
			GraphSharedDataArchiveBuffer = TraitWriter.GetGraphSharedData();
			GraphReferencedObjects = TraitWriter.GetGraphReferencedObjects();
			GraphReferencedSoftObjects = TraitWriter.GetGraphReferencedSoftObjects();
		}

		// Clear out the node template registry to test registration on load
		{
			FScopedNewNodeTemplateRegistry ScopedClearNodeTemplateRegistryForLoad;

			AddErrorIfFalse(FNodeTemplateRegistry::Get().GetNum() == 0, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Registry should contain 0 templates");

			// Read our graph
			FTestUtils::LoadFromArchiveBuffer(*AnimationGraph, NodeHandles, GraphSharedDataArchiveBuffer);

			TSharedPtr<FAnimNextGraphInstance> GraphInstance = AnimationGraph->AllocateInstance();

			FExecutionContext Context(*GraphInstance.Get());

			// Validate trait serialization
			{
				FAnimNextTraitHandle TraitHandle0(NodeHandles[0], 0);		// Point to first node, first base trait
				FAnimNextTraitHandle TraitHandle1(NodeHandles[1], 0);		// Point to second node, first base trait

				FTraitPtr TraitPtr0 = Context.AllocateNodeInstance(*GraphInstance.Get(), TraitHandle0);
				AddErrorIfFalse(TraitPtr0.IsValid(), "FAnimationAnimNextRuntimeTest_TraitSerialization -> Failed to allocate a node instance");

				FTraitPtr TraitPtr1 = Context.AllocateNodeInstance(*GraphInstance.Get(), TraitHandle1);
				AddErrorIfFalse(TraitPtr1.IsValid(), "FAnimationAnimNextRuntimeTest_TraitSerialization -> Failed to allocate a node instance");

				FTraitStackBinding StackNode0;
				AddErrorIfFalse(Context.GetStack(TraitPtr0, StackNode0), "FAnimationAnimNextRuntimeTest_TraitSerialization -> Failed to bind to trait stack");

				FTraitStackBinding StackNode1;
				AddErrorIfFalse(Context.GetStack(TraitPtr1, StackNode1), "FAnimationAnimNextRuntimeTest_TraitSerialization -> Failed to bind to trait stack");

				// Validate shared data for base trait on node 0
				{
					TTraitBinding<IInterfaceA> BindingA0;
					AddErrorIfFalse(StackNode0.GetInterface(BindingA0), "FAnimationAnimNextRuntimeTest_TraitSerialization -> InterfaceA not found");

					const auto* SharedDataA0 = BindingA0.GetSharedData<FTraitSerialization_Base::FSharedData>();

					AddErrorIfFalse(SharedDataA0->Integer == TraitBaseRef0.Integer, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(FMemory::Memcmp(SharedDataA0->IntegerArray, TraitBaseRef0.IntegerArray, sizeof(TraitBaseRef0.IntegerArray)) == 0, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataA0->IntegerTArray == TraitBaseRef0.IntegerTArray, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataA0->Vector == TraitBaseRef0.Vector, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(FMemory::Memcmp(SharedDataA0->VectorArray, TraitBaseRef0.VectorArray, sizeof(TraitBaseRef0.VectorArray)) == 0, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataA0->VectorTArray == TraitBaseRef0.VectorTArray, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataA0->String == TraitBaseRef0.String, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataA0->Name == TraitBaseRef0.Name, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
				}

				// Validate shared data for additive trait on node 0
				{
					TTraitBinding<IInterfaceB> BindingB0;
					AddErrorIfFalse(StackNode0.GetInterface(BindingB0), "FAnimationAnimNextRuntimeTest_TraitSerialization -> InterfaceB not found");

					const auto* SharedDataB0 = BindingB0.GetSharedData<FTraitSerialization_Add::FSharedData>();

					AddErrorIfFalse(SharedDataB0->Integer == TraitAddRef0.Integer, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(FMemory::Memcmp(SharedDataB0->IntegerArray, TraitAddRef0.IntegerArray, sizeof(TraitAddRef0.IntegerArray)) == 0, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataB0->IntegerTArray == TraitAddRef0.IntegerTArray, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataB0->Vector == TraitAddRef0.Vector, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(FMemory::Memcmp(SharedDataB0->VectorArray, TraitAddRef0.VectorArray, sizeof(TraitAddRef0.VectorArray)) == 0, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataB0->VectorTArray == TraitAddRef0.VectorTArray, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataB0->String == TraitAddRef0.String, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataB0->Name == TraitAddRef0.Name, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
				}

				// Validate shared data for native trait on node 0
				{
					TTraitBinding<IInterfaceC> BindingC0;
					AddErrorIfFalse(StackNode0.GetInterface(BindingC0), "FAnimationAnimNextRuntimeTest_TraitSerialization -> InterfaceC not found");

					const auto* SharedDataC0 = BindingC0.GetSharedData<FTraitNativeSerialization_Add::FSharedData>();

					AddErrorIfFalse(SharedDataC0->Integer == TraitNativeRef0.Integer, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(FMemory::Memcmp(SharedDataC0->IntegerArray, TraitNativeRef0.IntegerArray, sizeof(TraitNativeRef0.IntegerArray)) == 0, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataC0->IntegerTArray == TraitNativeRef0.IntegerTArray, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataC0->Vector == TraitNativeRef0.Vector, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(FMemory::Memcmp(SharedDataC0->VectorArray, TraitNativeRef0.VectorArray, sizeof(TraitNativeRef0.VectorArray)) == 0, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataC0->VectorTArray == TraitNativeRef0.VectorTArray, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataC0->String == TraitNativeRef0.String, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataC0->Name == TraitNativeRef0.Name, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataC0->bSerializeCalled, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
				}

				// Validate shared data for base trait on node 1
				{
					TTraitBinding<IInterfaceA> BindingA1;
					AddErrorIfFalse(StackNode1.GetInterface(BindingA1), "FAnimationAnimNextRuntimeTest_TraitSerialization -> InterfaceA not found");

					const auto* SharedDataA1 = BindingA1.GetSharedData<FTraitSerialization_Base::FSharedData>();

					AddErrorIfFalse(SharedDataA1->Integer == TraitBaseRef1.Integer, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(FMemory::Memcmp(SharedDataA1->IntegerArray, TraitBaseRef1.IntegerArray, sizeof(TraitBaseRef1.IntegerArray)) == 0, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataA1->IntegerTArray == TraitBaseRef1.IntegerTArray, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataA1->Vector == TraitBaseRef1.Vector, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(FMemory::Memcmp(SharedDataA1->VectorArray, TraitBaseRef1.VectorArray, sizeof(TraitBaseRef1.VectorArray)) == 0, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataA1->VectorTArray == TraitBaseRef1.VectorTArray, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataA1->String == TraitBaseRef1.String, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataA1->Name == TraitBaseRef1.Name, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
				}

				// Validate shared data for additive trait on node 1
				{
					TTraitBinding<IInterfaceB> BindingB1;
					AddErrorIfFalse(StackNode1.GetInterface(BindingB1), "FAnimationAnimNextRuntimeTest_TraitSerialization -> InterfaceB not found");

					const auto* SharedDataB1 = BindingB1.GetSharedData<FTraitSerialization_Add::FSharedData>();

					AddErrorIfFalse(SharedDataB1->Integer == TraitAddRef1.Integer, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(FMemory::Memcmp(SharedDataB1->IntegerArray, TraitAddRef1.IntegerArray, sizeof(TraitAddRef1.IntegerArray)) == 0, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataB1->IntegerTArray == TraitAddRef1.IntegerTArray, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataB1->Vector == TraitAddRef1.Vector, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(FMemory::Memcmp(SharedDataB1->VectorArray, TraitAddRef1.VectorArray, sizeof(TraitAddRef1.VectorArray)) == 0, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataB1->VectorTArray == TraitAddRef1.VectorTArray, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataB1->String == TraitAddRef1.String, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataB1->Name == TraitAddRef1.Name, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
				}

				// Validate shared data for native trait on node 1
				{
					TTraitBinding<IInterfaceC> BindingC1;
					AddErrorIfFalse(StackNode1.GetInterface(BindingC1), "FAnimationAnimNextRuntimeTest_TraitSerialization -> InterfaceC not found");

					const auto* SharedDataC1 = BindingC1.GetSharedData<FTraitNativeSerialization_Add::FSharedData>();

					AddErrorIfFalse(SharedDataC1->Integer == TraitNativeRef1.Integer, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(FMemory::Memcmp(SharedDataC1->IntegerArray, TraitNativeRef1.IntegerArray, sizeof(TraitNativeRef1.IntegerArray)) == 0, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataC1->IntegerTArray == TraitNativeRef1.IntegerTArray, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataC1->Vector == TraitNativeRef1.Vector, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(FMemory::Memcmp(SharedDataC1->VectorArray, TraitNativeRef1.VectorArray, sizeof(TraitNativeRef1.VectorArray)) == 0, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataC1->VectorTArray == TraitNativeRef1.VectorTArray, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataC1->String == TraitNativeRef1.String, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataC1->Name == TraitNativeRef1.Name, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
					AddErrorIfFalse(SharedDataC1->bSerializeCalled, "FAnimationAnimNextRuntimeTest_TraitSerialization -> Unexpected serialized value");
				}
			}
		}
	}

	Tests::FUtils::CleanupAfterTests();
	
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnimationAnimNextRuntimeTest_SimpleGraphBuilder, "Animation.AnimNext.Runtime.SimpleGraphBuilder", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAnimationAnimNextRuntimeTest_SimpleGraphBuilder::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	{
		FScopedClearNodeTemplateRegistry ScopedClearNodeTemplateRegistry;

		AUTO_REGISTER_ANIM_TRAIT(FTraitSerialization_Base)
		AUTO_REGISTER_ANIM_TRAIT(FTraitSerialization_Add)
		AUTO_REGISTER_ANIM_TRAIT(FTraitNativeSerialization_Add)
	
		FTraitSerialization_Base::FSharedData TraitBaseRef0;
		TraitBaseRef0.Integer = 1651;
		TraitBaseRef0.IntegerArray[0] = 1071;
		TraitBaseRef0.IntegerArray[1] = -158;
		TraitBaseRef0.IntegerArray[2] = 88116;
		TraitBaseRef0.IntegerArray[3] = 0x417;
		TraitBaseRef0.IntegerTArray = { -8162, 88152, 0x8152f };
		TraitBaseRef0.Vector = FVector(0.1917, 12435.1, -18200.1726);
		TraitBaseRef0.VectorArray[0] = FVector(192.1716, -1927.115, 99176.12);
		TraitBaseRef0.VectorArray[1] = FVector(961.811, -18956.117, 81673.44);
		TraitBaseRef0.VectorTArray = { FVector(-1927.8771, 1826.9917, -123.1555), FVector(9177.011, -71.44, -917.88), FVector(123.91, 852.11, -81652.1) };
		TraitBaseRef0.String = TEXT("sample string 123");
		TraitBaseRef0.Name = FName(TEXT("sample name 999178"));

		FTraitSerialization_Add::FSharedData TraitAddRef0;
		TraitAddRef0.Integer = 16511;
		TraitAddRef0.IntegerArray[0] = 10711;
		TraitAddRef0.IntegerArray[1] = -1581;
		TraitAddRef0.IntegerArray[2] = 881161;
		TraitAddRef0.IntegerArray[3] = 0x4171;
		TraitAddRef0.IntegerTArray = { -81621, 881521, 0x8152f1 };
		TraitAddRef0.Vector = FVector(0.19171, 12435.11, -18200.17261);
		TraitAddRef0.VectorArray[0] = FVector(192.17161, -1927.1151, 99176.121);
		TraitAddRef0.VectorArray[1] = FVector(961.8111, -18956.1171, 81673.441);
		TraitAddRef0.VectorTArray = { FVector(-1927.87711, 1826.99171, -123.15551), FVector(9177.0111, -71.441, -917.881), FVector(123.911, 852.111, -81652.11) };
		TraitAddRef0.String = TEXT("sample string 1231");
		TraitAddRef0.Name = FName(TEXT("sample name 9991781"));

		FTraitNativeSerialization_Add::FSharedData TraitNativeRef0;
		TraitNativeRef0.Integer = 16514;
		TraitNativeRef0.IntegerArray[0] = 10714;
		TraitNativeRef0.IntegerArray[1] = -1584;
		TraitNativeRef0.IntegerArray[2] = 881164;
		TraitNativeRef0.IntegerArray[3] = 0x4174;
		TraitNativeRef0.IntegerTArray = { -81624, 881524, 0x8152f4 };
		TraitNativeRef0.Vector = FVector(0.19174, 12435.14, -18200.17264);
		TraitNativeRef0.VectorArray[0] = FVector(192.17164, -1927.1154, 99176.124);
		TraitNativeRef0.VectorArray[1] = FVector(961.8114, -18956.1174, 81673.444);
		TraitNativeRef0.VectorTArray = { FVector(-1927.87714, 1826.99174, -123.15554), FVector(9177.0114, -71.444, -917.884), FVector(123.914, 852.114, -81652.14) };
		TraitNativeRef0.String = TEXT("sample string 1234");
		TraitNativeRef0.Name = FName(TEXT("sample name 9991784"));
		
		{
			FAnimGraphBuilderContext AnimGraphBuilderContext;
			{
				FAnimNextSimpleAnimGraphBuilder Builder;
				Builder.AddTraitStructView(0, ETraitVariableMapping::All, TraitBaseRef0);
				Builder.AddTraitStructView(0, ETraitVariableMapping::All, TraitAddRef0);
				Builder.AddTraitStructView(0, ETraitVariableMapping::All, TraitNativeRef0);

				const bool bResult = Builder.Build(AnimGraphBuilderContext);
				AddErrorIfFalse(bResult, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Builder failed");
			}

			// Clear out the node template registry to test registration on load
			{
				FScopedNewNodeTemplateRegistry ScopedClearNodeTemplateRegistryForLoad;
				FNodeTemplateRegistry& Registry = FNodeTemplateRegistry::Get();

				AddErrorIfFalse(Registry.GetNum() == 0, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Registry should contain 0 templates");

				// Builder context loads from archive buffer
				const UUAFAnimGraph* AnimationGraph = AnimGraphBuilderContext.Build();
				UE_RETURN_ON_ERROR(AnimationGraph != nullptr, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Failed to create animation graph");

				TSharedPtr<FAnimNextGraphInstance> GraphInstance = AnimationGraph->AllocateInstance();

				FExecutionContext Context(*GraphInstance.Get());

				// Validate trait serialization
				{
					FTraitStackBinding StackNode;
					AddErrorIfFalse(Context.GetStack(GraphInstance->GetGraphRootPtr(), StackNode), "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Failed to bind to trait stack");

					// Validate shared data for base trait on node 0
					{
						TTraitBinding<IInterfaceA> BindingA0;
						AddErrorIfFalse(StackNode.GetInterface(BindingA0), "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> InterfaceA not found");

						const auto* SharedDataA0 = BindingA0.GetSharedData<FTraitSerialization_Base::FSharedData>();

						AddErrorIfFalse(SharedDataA0->Integer == TraitBaseRef0.Integer, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
						AddErrorIfFalse(FMemory::Memcmp(SharedDataA0->IntegerArray, TraitBaseRef0.IntegerArray, sizeof(TraitBaseRef0.IntegerArray)) == 0, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
						AddErrorIfFalse(SharedDataA0->IntegerTArray == TraitBaseRef0.IntegerTArray, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
						AddErrorIfFalse(SharedDataA0->Vector == TraitBaseRef0.Vector, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
						AddErrorIfFalse(FMemory::Memcmp(SharedDataA0->VectorArray, TraitBaseRef0.VectorArray, sizeof(TraitBaseRef0.VectorArray)) == 0, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
						AddErrorIfFalse(SharedDataA0->VectorTArray == TraitBaseRef0.VectorTArray, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
						AddErrorIfFalse(SharedDataA0->String == TraitBaseRef0.String, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
						AddErrorIfFalse(SharedDataA0->Name == TraitBaseRef0.Name, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
					}

					// Validate shared data for additive trait on node 0
					{
						TTraitBinding<IInterfaceB> BindingB0;
						AddErrorIfFalse(StackNode.GetInterface(BindingB0), "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> InterfaceB not found");

						const auto* SharedDataB0 = BindingB0.GetSharedData<FTraitSerialization_Add::FSharedData>();

						AddErrorIfFalse(SharedDataB0->Integer == TraitAddRef0.Integer, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
						AddErrorIfFalse(FMemory::Memcmp(SharedDataB0->IntegerArray, TraitAddRef0.IntegerArray, sizeof(TraitAddRef0.IntegerArray)) == 0, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
						AddErrorIfFalse(SharedDataB0->IntegerTArray == TraitAddRef0.IntegerTArray, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
						AddErrorIfFalse(SharedDataB0->Vector == TraitAddRef0.Vector, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
						AddErrorIfFalse(FMemory::Memcmp(SharedDataB0->VectorArray, TraitAddRef0.VectorArray, sizeof(TraitAddRef0.VectorArray)) == 0, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
						AddErrorIfFalse(SharedDataB0->VectorTArray == TraitAddRef0.VectorTArray, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
						AddErrorIfFalse(SharedDataB0->String == TraitAddRef0.String, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
						AddErrorIfFalse(SharedDataB0->Name == TraitAddRef0.Name, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
					}

					// Validate shared data for native trait on node 0
					{
						TTraitBinding<IInterfaceC> BindingC0;
						AddErrorIfFalse(StackNode.GetInterface(BindingC0), "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> InterfaceC not found");

						const auto* SharedDataC0 = BindingC0.GetSharedData<FTraitNativeSerialization_Add::FSharedData>();

						AddErrorIfFalse(SharedDataC0->Integer == TraitNativeRef0.Integer, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
						AddErrorIfFalse(FMemory::Memcmp(SharedDataC0->IntegerArray, TraitNativeRef0.IntegerArray, sizeof(TraitNativeRef0.IntegerArray)) == 0, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
						AddErrorIfFalse(SharedDataC0->IntegerTArray == TraitNativeRef0.IntegerTArray, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
						AddErrorIfFalse(SharedDataC0->Vector == TraitNativeRef0.Vector, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
						AddErrorIfFalse(FMemory::Memcmp(SharedDataC0->VectorArray, TraitNativeRef0.VectorArray, sizeof(TraitNativeRef0.VectorArray)) == 0, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
						AddErrorIfFalse(SharedDataC0->VectorTArray == TraitNativeRef0.VectorTArray, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
						AddErrorIfFalse(SharedDataC0->String == TraitNativeRef0.String, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
						AddErrorIfFalse(SharedDataC0->Name == TraitNativeRef0.Name, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
						AddErrorIfFalse(SharedDataC0->bSerializeCalled, "FAnimationAnimNextRuntimeTest_SimpleGraphBuilder -> Unexpected serialized value");
					}
				}
			}
		}
	}

	Tests::FUtils::CleanupAfterTests();

	return true;
}


#endif

FTraitA_BaseSharedData::FTraitA_BaseSharedData()
#if WITH_DEV_AUTOMATION_TESTS
	: TraitUID(UE::UAF::FTraitA_Base::TraitUID.GetUID())
#endif
{
}

FTraitAB_AddSharedData::FTraitAB_AddSharedData()
#if WITH_DEV_AUTOMATION_TESTS
	: TraitUID(UE::UAF::FTraitAB_Add::TraitUID.GetUID())
#endif
{
}

FTraitAC_AddSharedData::FTraitAC_AddSharedData()
#if WITH_DEV_AUTOMATION_TESTS
	: TraitUID(UE::UAF::FTraitAC_Add::TraitUID.GetUID())
#endif
{
}

bool FTraitNativeSerialization_AddSharedData::Serialize(FArchive& Ar)
{
	Ar << Integer;

	int32 integerArrayCount = sizeof(IntegerArray) / sizeof(IntegerArray[0]);
	Ar << integerArrayCount;
	for (int32 i = 0; i < integerArrayCount; ++i)
	{
		Ar << IntegerArray[i];
	}

	Ar << IntegerTArray;
	Ar << Vector;

	int32 vectorArrayCount = sizeof(VectorArray) / sizeof(VectorArray[0]);
	Ar << vectorArrayCount;
	for (int32 i = 0; i < vectorArrayCount; ++i)
	{
		Ar << VectorArray[i];
	}

	Ar << VectorTArray;
	Ar << String;
	Ar << Name;

	bSerializeCalled = true;

	return true;
}
