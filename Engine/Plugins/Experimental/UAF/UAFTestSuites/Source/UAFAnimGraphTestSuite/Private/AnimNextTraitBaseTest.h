// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/TraitSharedData.h"

#include "AnimNextTraitBaseTest.generated.h"

USTRUCT()
struct FTraitA_BaseSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY(meta = (Inline))
	uint32 TraitUID;

	FTraitA_BaseSharedData();
};

USTRUCT()
struct FTraitAB_AddSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY(meta = (Inline))
	uint32 TraitUID;

	FTraitAB_AddSharedData();
};

USTRUCT()
struct FTraitAC_AddSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY(meta = (Inline))
	uint32 TraitUID;

	FTraitAC_AddSharedData();
};

USTRUCT()
struct FTraitSerialization_BaseSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY(meta = (Inline))
	int32 Integer = 0;

	UPROPERTY(meta = (Inline))
	int32 IntegerArray[4] = { 0, 0, 0, 0 };

	UPROPERTY(meta = (Inline))
	TArray<int32> IntegerTArray;

	UPROPERTY(meta = (Inline))
	FVector Vector = FVector::ZeroVector;

	UPROPERTY(meta = (Inline))
	FVector VectorArray[2] = { FVector::ZeroVector, FVector::ZeroVector };

	UPROPERTY(meta = (Inline))
	TArray<FVector> VectorTArray;

	UPROPERTY(meta = (Inline))
	FString String;

	UPROPERTY(meta = (Inline))
	FName Name;
};

USTRUCT()
struct FTraitSerialization_AddSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY(meta = (Inline))
	int32 Integer = 0;

	UPROPERTY(meta = (Inline))
	int32 IntegerArray[4] = { 0, 0, 0, 0 };

	UPROPERTY(meta = (Inline))
	TArray<int32> IntegerTArray;

	UPROPERTY(meta = (Inline))
	FVector Vector = FVector::ZeroVector;

	UPROPERTY(meta = (Inline))
	FVector VectorArray[2] = { FVector::ZeroVector, FVector::ZeroVector };

	UPROPERTY(meta = (Inline))
	TArray<FVector> VectorTArray;

	UPROPERTY(meta = (Inline))
	FString String;

	UPROPERTY(meta = (Inline))
	FName Name;
};

USTRUCT()
struct FTraitNativeSerialization_AddSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY(meta = (Inline))
	int32 Integer = 0;

	UPROPERTY(meta = (Inline))
	int32 IntegerArray[4] = { 0, 0, 0, 0 };

	UPROPERTY(meta = (Inline))
	TArray<int32> IntegerTArray;

	UPROPERTY(meta = (Inline))
	FVector Vector = FVector::ZeroVector;

	UPROPERTY(meta = (Inline))
	FVector VectorArray[2] = { FVector::ZeroVector, FVector::ZeroVector };

	UPROPERTY(meta = (Inline))
	TArray<FVector> VectorTArray;

	UPROPERTY(meta = (Inline))
	FString String;

	UPROPERTY(meta = (Inline))
	FName Name;

	bool bSerializeCalled = false;

	bool Serialize(FArchive& Ar);
};

template<>
struct TStructOpsTypeTraits<FTraitNativeSerialization_AddSharedData> : public TStructOpsTypeTraitsBase2<FTraitNativeSerialization_AddSharedData>
{
	enum
	{
		WithSerializer = true,
	};
};
