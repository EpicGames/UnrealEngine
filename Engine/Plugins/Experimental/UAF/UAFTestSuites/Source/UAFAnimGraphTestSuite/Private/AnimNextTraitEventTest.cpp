// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextTraitEventTest.h"
#include "AnimNextRuntimeTest.h"
#include "AnimNextTest.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "TraitCore/Trait.h"
#include "TraitCore/TraitBinding.h"
#include "TraitCore/TraitEventList.h"
#include "TraitCore/TraitEvent.h"
#include "TraitCore/TraitEventRaising.h"
#include "TraitCore/TraitReader.h"
#include "TraitCore/TraitWriter.h"
#include "TraitCore/ExecutionContext.h"
#include "TraitCore/NodeTemplateBuilder.h"
#include "TraitCore/NodeTemplateRegistry.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextAnimationGraphFactory.h"

//****************************************************************************
// AnimNext Runtime TraitEvent Tests
//****************************************************************************

namespace UE::UAF
{
	struct FTraitCoreTest_EventAB_Base : FBaseTrait
	{
		DECLARE_ANIM_TRAIT(FTraitCoreTest_EventAB_Base, FBaseTrait)

		// We support static functions
		static ETraitStackPropagation OnEventA(const FExecutionContext& Context, FTraitBinding& Binding, FTraitCoreTest_EventA& Event)
		{
			Event.VisitedTraits.Add(FTraitCoreTest_EventAB_Base::TraitUID);
			return ETraitStackPropagation::Continue;
		}

		// We support const member functions
		ETraitStackPropagation OnEventB(const FExecutionContext& Context, FTraitBinding& Binding, FTraitCoreTest_EventB& Event) const
		{
			Event.VisitedTraits.Add(FTraitCoreTest_EventAB_Base::TraitUID);
			return ETraitStackPropagation::Continue;
		}
	};

	// Trait implementation boilerplate
	#define TRAIT_EVENT_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(FTraitCoreTest_EventAB_Base::OnEventA) \
		GeneratorMacro(FTraitCoreTest_EventAB_Base::OnEventB) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FTraitCoreTest_EventAB_Base, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_EVENT_ENUMERATOR

	struct FTraitCoreTest_EventAB_Add : FAdditiveTrait
	{
		DECLARE_ANIM_TRAIT(FTraitCoreTest_EventAB_Add, FAdditiveTrait)

		ETraitStackPropagation OnEventA(const FExecutionContext& Context, FTraitBinding& Binding, FTraitCoreTest_EventA& Event) const
		{
			Event.VisitedTraits.Add(FTraitCoreTest_EventAB_Add::TraitUID);
			return Event.bAlwaysForwardToBase ? ETraitStackPropagation::Continue : ETraitStackPropagation::Stop;
		}

		ETraitStackPropagation OnEventB(const FExecutionContext& Context, FTraitBinding& Binding, FTraitCoreTest_EventB& Event) const
		{
			Event.VisitedTraits.Add(FTraitCoreTest_EventAB_Add::TraitUID);
			return ETraitStackPropagation::Continue;
		}
	};

	// Trait implementation boilerplate
	#define TRAIT_EVENT_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(FTraitCoreTest_EventAB_Add::OnEventA) \
		GeneratorMacro(FTraitCoreTest_EventAB_Add::OnEventB) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FTraitCoreTest_EventAB_Add, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_EVENT_ENUMERATOR

	struct FTraitCoreTest_EventA_Add : FAdditiveTrait
	{
		DECLARE_ANIM_TRAIT(FTraitCoreTest_EventA_Add, FAdditiveTrait)

		ETraitStackPropagation OnEventA(const FExecutionContext& Context, FTraitBinding& Binding, FTraitCoreTest_EventA& Event) const
		{
			Event.VisitedTraits.Add(FTraitCoreTest_EventA_Add::TraitUID);
			return ETraitStackPropagation::Continue;
		}
	};

	// Trait implementation boilerplate
	#define TRAIT_EVENT_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(FTraitCoreTest_EventA_Add::OnEventA) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FTraitCoreTest_EventA_Add, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_EVENT_ENUMERATOR

	struct FTraitCoreTest_EventB_Add : FAdditiveTrait
	{
		DECLARE_ANIM_TRAIT(FTraitCoreTest_EventB_Add, FAdditiveTrait)

		ETraitStackPropagation OnEventB(const FExecutionContext& Context, FTraitBinding& Binding, FTraitCoreTest_EventB& Event) const
		{
			Event.VisitedTraits.Add(FTraitCoreTest_EventB_Add::TraitUID);
			return ETraitStackPropagation::Continue;
		}
	};

	// Trait implementation boilerplate
	#define TRAIT_EVENT_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(FTraitCoreTest_EventB_Add::OnEventB) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FTraitCoreTest_EventB_Add, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_EVENT_ENUMERATOR
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnimationAnimNextRuntimeTest_TraitEventLifetime, "Animation.AnimNext.Runtime.TraitEventLifetime", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAnimationAnimNextRuntimeTest_TraitEventLifetime::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	{
		FTraitEventLifetime Lifetime;
		AddErrorIfFalse(!Lifetime.IsTransient(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Default constructed lifetime should be expired");
		AddErrorIfFalse(!Lifetime.IsInfinite(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Default constructed lifetime should be expired");
		AddErrorIfFalse(Lifetime.IsExpired(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Default constructed lifetime should be expired");

		bool bIsExpired = Lifetime.Decrement();
		AddErrorIfFalse(bIsExpired, "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Decremented lifetime should be expired");
		AddErrorIfFalse(!Lifetime.IsTransient(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Decremented lifetime should be expired");
		AddErrorIfFalse(!Lifetime.IsInfinite(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Decremented lifetime should be expired");
		AddErrorIfFalse(Lifetime.IsExpired(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Decremented lifetime should be expired");
	}

	{
		FTraitEventLifetime Lifetime = FTraitEventLifetime::MakeTransient();
		AddErrorIfFalse(Lifetime.IsTransient(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Constructed lifetime should be transient");
		AddErrorIfFalse(!Lifetime.IsInfinite(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Constructed lifetime should be transient");
		AddErrorIfFalse(!Lifetime.IsExpired(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Constructed lifetime should be transient");

		bool bIsExpired = Lifetime.Decrement();
		AddErrorIfFalse(bIsExpired, "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Decremented lifetime should be expired");
		AddErrorIfFalse(!Lifetime.IsTransient(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Decremented lifetime should be expired");
		AddErrorIfFalse(!Lifetime.IsInfinite(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Decremented lifetime should be expired");
		AddErrorIfFalse(Lifetime.IsExpired(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Decremented lifetime should be expired");
	}

	{
		FTraitEventLifetime Lifetime = FTraitEventLifetime::MakeInfinite();
		AddErrorIfFalse(!Lifetime.IsTransient(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Constructed lifetime should be infinite");
		AddErrorIfFalse(Lifetime.IsInfinite(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Constructed lifetime should be infinite");
		AddErrorIfFalse(!Lifetime.IsExpired(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Constructed lifetime should be infinite");

		bool bIsExpired = Lifetime.Decrement();
		AddErrorIfFalse(!bIsExpired, "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Decremented lifetime should be infinite");
		AddErrorIfFalse(!Lifetime.IsTransient(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Decremented lifetime should be infinite");
		AddErrorIfFalse(Lifetime.IsInfinite(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Decremented lifetime should be infinite");
		AddErrorIfFalse(!Lifetime.IsExpired(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Decremented lifetime should be infinite");
	}

	{
		FTraitEventLifetime Lifetime = FTraitEventLifetime::MakeUntil(2);
		AddErrorIfFalse(!Lifetime.IsTransient(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Constructed lifetime should be finite");
		AddErrorIfFalse(!Lifetime.IsInfinite(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Constructed lifetime should be finite");
		AddErrorIfFalse(!Lifetime.IsExpired(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Constructed lifetime should be finite");

		bool bIsExpired = Lifetime.Decrement();
		AddErrorIfFalse(!bIsExpired, "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Decremented lifetime should be transient");
		AddErrorIfFalse(Lifetime.IsTransient(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Decremented lifetime should be transient");
		AddErrorIfFalse(!Lifetime.IsInfinite(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Decremented lifetime should be transient");
		AddErrorIfFalse(!Lifetime.IsExpired(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Decremented lifetime should be transient");

		bIsExpired = Lifetime.Decrement();
		AddErrorIfFalse(bIsExpired, "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Decremented lifetime should be expired");
		AddErrorIfFalse(!Lifetime.IsTransient(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Decremented lifetime should be expired");
		AddErrorIfFalse(!Lifetime.IsInfinite(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Decremented lifetime should be expired");
		AddErrorIfFalse(Lifetime.IsExpired(), "FAnimationAnimNextRuntimeTest_TraitEventLifetime -> Decremented lifetime should be expired");
	}
	
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnimationAnimNextRuntimeTest_TraitEvent, "Animation.AnimNext.Runtime.TraitEvent", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAnimationAnimNextRuntimeTest_TraitEvent::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	{
		FAnimNextTraitEvent Event;
		AddErrorIfFalse(Event.IsValid(), "FAnimationAnimNextRuntimeTest_TraitEvent -> Default constructed event should be valid");
		AddErrorIfFalse(!Event.IsHandled(), "FAnimationAnimNextRuntimeTest_TraitEvent -> Default constructed event should not be handled");
		AddErrorIfFalse(!Event.IsConsumed(), "FAnimationAnimNextRuntimeTest_TraitEvent -> Default constructed event should not be consumed");
		AddErrorIfFalse(!Event.IsExpired(), "FAnimationAnimNextRuntimeTest_TraitEvent -> Default constructed event should not be expired");
		AddErrorIfFalse(!Event.IsInfinite(), "FAnimationAnimNextRuntimeTest_TraitEvent -> Default constructed event should be transient");
		AddErrorIfFalse(Event.IsTransient(), "FAnimationAnimNextRuntimeTest_TraitEvent -> Default constructed event should be transient");

		FTraitEventList OutputEventList;
		bool bIsExpired = Event.DecrementLifetime(OutputEventList);
		AddErrorIfFalse(bIsExpired, "FAnimationAnimNextRuntimeTest_TraitEvent -> Event should be expired");
		AddErrorIfFalse(!Event.IsValid(), "FAnimationAnimNextRuntimeTest_TraitEvent -> Event should be expired");
		AddErrorIfFalse(!Event.IsHandled(), "FAnimationAnimNextRuntimeTest_TraitEvent -> Event should not be handled");
		AddErrorIfFalse(!Event.IsConsumed(), "FAnimationAnimNextRuntimeTest_TraitEvent -> Event should not be consumed");
		AddErrorIfFalse(Event.IsExpired(), "FAnimationAnimNextRuntimeTest_TraitEvent -> Event should be expired");
		AddErrorIfFalse(!Event.IsInfinite(), "FAnimationAnimNextRuntimeTest_TraitEvent -> Event should be expired");
		AddErrorIfFalse(!Event.IsTransient(), "FAnimationAnimNextRuntimeTest_TraitEvent -> Event should be expired");
	}

	{
		FAnimNextTraitEvent Event;
		AddErrorIfFalse(Event.IsValid(), "FAnimationAnimNextRuntimeTest_TraitEvent -> Default constructed event should be valid");
		AddErrorIfFalse(!Event.IsHandled(), "FAnimationAnimNextRuntimeTest_TraitEvent -> Default constructed event should not be handled");
		AddErrorIfFalse(!Event.IsConsumed(), "FAnimationAnimNextRuntimeTest_TraitEvent -> Default constructed event should not be consumed");

		Event.MarkHandled();
		AddErrorIfFalse(Event.IsValid(), "FAnimationAnimNextRuntimeTest_TraitEvent -> Event should be valid and handled");
		AddErrorIfFalse(Event.IsHandled(), "FAnimationAnimNextRuntimeTest_TraitEvent -> Event should be valid and handled");
		AddErrorIfFalse(!Event.IsConsumed(), "FAnimationAnimNextRuntimeTest_TraitEvent -> Event should not be consumed");

		Event.MarkConsumed();
		AddErrorIfFalse(!Event.IsValid(), "FAnimationAnimNextRuntimeTest_TraitEvent -> Event should be consumed");
		AddErrorIfFalse(Event.IsHandled(), "FAnimationAnimNextRuntimeTest_TraitEvent -> Event should be handled and consumed");
		AddErrorIfFalse(Event.IsConsumed(), "FAnimationAnimNextRuntimeTest_TraitEvent -> Event should be consumed");
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnimationAnimNextRuntimeTest_TraitEventRaising, "Animation.AnimNext.Runtime.TraitEventRaising", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAnimationAnimNextRuntimeTest_TraitEventRaising::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	// Test event raising on a node trait stack
	{
		AUTO_REGISTER_ANIM_TRAIT(FTraitCoreTest_EventAB_Base)
		AUTO_REGISTER_ANIM_TRAIT(FTraitCoreTest_EventAB_Add)
		AUTO_REGISTER_ANIM_TRAIT(FTraitCoreTest_EventA_Add)
		AUTO_REGISTER_ANIM_TRAIT(FTraitCoreTest_EventB_Add)

		UFactory* GraphFactory = NewObject<UUAFAnimGraphFactory>();
		UUAFAnimGraph* AnimationGraph = CastChecked<UUAFAnimGraph>(GraphFactory->FactoryCreateNew(UUAFAnimGraph::StaticClass(), GetTransientPackage(), TEXT("TestAnimNextGraph"), RF_Transient, nullptr, nullptr, NAME_None));
		UE_RETURN_ON_ERROR(AnimationGraph != nullptr, "FAnimationAnimNextEditorTest_GraphTraitOperations -> Failed to create animation graph");

		FScopedClearNodeTemplateRegistry ScopedClearNodeTemplateRegistry;
		FNodeTemplateRegistry& Registry = FNodeTemplateRegistry::Get();

		// Build a node
		TArray<FTraitUID> NodeTemplateTraitList;
		NodeTemplateTraitList.Add(FTraitCoreTest_EventAB_Base::TraitUID);
		NodeTemplateTraitList.Add(FTraitCoreTest_EventAB_Add::TraitUID);
		NodeTemplateTraitList.Add(FTraitCoreTest_EventA_Add::TraitUID);
		NodeTemplateTraitList.Add(FTraitCoreTest_EventB_Add::TraitUID);

		// Populate our node template registry
		TArray<uint8> NodeTemplateBuffer0;
		const FNodeTemplate* NodeTemplate0 = FNodeTemplateBuilder::BuildNodeTemplate(NodeTemplateTraitList, NodeTemplateBuffer0);

		FNodeTemplateRegistryHandle TemplateHandle0 = Registry.FindOrAdd(NodeTemplate0);
		AddErrorIfFalse(TemplateHandle0.IsValid(), "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Registry should contain our template");

		TArray<FNodeHandle> NodeHandles;

		// Write our graph
		TArray<uint8> GraphSharedDataArchiveBuffer;
		TArray<TObjectPtr<UObject>> GraphReferencedObjects;
		TArray<FSoftObjectPath> GraphReferencedSoftObjects;
		{
			FTraitWriter TraitWriter;

			NodeHandles.Add(TraitWriter.RegisterNode(*NodeTemplate0));

			// We don't have trait properties

			TraitWriter.BeginNodeWriting();
			TraitWriter.WriteNode(NodeHandles[0],
				[](uint32 TraitIndex, FName PropertyName)
				{
					return FString();
				},
				[](uint32 TraitIndex, FName PropertyName)
				{
					return MAX_uint16;
				});
			TraitWriter.EndNodeWriting();

			AddErrorIfFalse(TraitWriter.GetErrorState() == FTraitWriter::EErrorState::None, "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Failed to write traits");
			GraphSharedDataArchiveBuffer = TraitWriter.GetGraphSharedData();
			GraphReferencedObjects = TraitWriter.GetGraphReferencedObjects();
			GraphReferencedSoftObjects = TraitWriter.GetGraphReferencedSoftObjects();
		}

		// Read our graph
		FTestUtils::LoadFromArchiveBuffer(*AnimationGraph, NodeHandles, GraphSharedDataArchiveBuffer);

		TSharedPtr<FAnimNextGraphInstance> GraphInstance = AnimationGraph->AllocateInstance();

		FExecutionContext Context(*GraphInstance.Get());

		{
			FAnimNextTraitHandle TraitHandle0(NodeHandles[0], 0);	// Point to node base trait

			FTraitPtr TraitPtr0 = Context.AllocateNodeInstance(*GraphInstance.Get(), TraitHandle0);
			AddErrorIfFalse(TraitPtr0.IsValid(), "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Failed to allocate a node instance");

			// Node has 4 traits: AB base, AB add, A add, B add
			FTraitStackBinding Stack0;
			AddErrorIfFalse(Context.GetStack(TraitPtr0, Stack0), "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Failed to bind to trait stack");

			// Send events A and B, make sure every trait is visited
			{
				auto EventA = MakeTraitEvent<FTraitCoreTest_EventA>();
				RaiseTraitEvent(Context, Stack0, *EventA);

				UE_RETURN_ON_ERROR(EventA->VisitedTraits.Num() == 3, "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected number of traits visited");
				AddErrorIfFalse(EventA->VisitedTraits[0] == NodeTemplateTraitList[2], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");
				AddErrorIfFalse(EventA->VisitedTraits[1] == NodeTemplateTraitList[1], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");
				AddErrorIfFalse(EventA->VisitedTraits[2] == NodeTemplateTraitList[0], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");

				auto EventB = MakeTraitEvent<FTraitCoreTest_EventB>();
				RaiseTraitEvent(Context, Stack0, *EventB);

				UE_RETURN_ON_ERROR(EventB->VisitedTraits.Num() == 3, "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected number of traits visited");
				AddErrorIfFalse(EventB->VisitedTraits[0] == NodeTemplateTraitList[3], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");
				AddErrorIfFalse(EventB->VisitedTraits[1] == NodeTemplateTraitList[1], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");
				AddErrorIfFalse(EventB->VisitedTraits[2] == NodeTemplateTraitList[0], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");
			}

			// Send events A and B in a list, make sure results are the same as test above
			{
				auto EventA = MakeTraitEvent<FTraitCoreTest_EventA>();
				auto EventB = MakeTraitEvent<FTraitCoreTest_EventB>();

				FTraitEventList EventList;
				EventList.Push(EventA);
				EventList.Push(EventB);

				RaiseTraitEvents(Context, Stack0, EventList);

				UE_RETURN_ON_ERROR(EventA->VisitedTraits.Num() == 3, "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected number of traits visited");
				AddErrorIfFalse(EventA->VisitedTraits[0] == NodeTemplateTraitList[2], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");
				AddErrorIfFalse(EventA->VisitedTraits[1] == NodeTemplateTraitList[1], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");
				AddErrorIfFalse(EventA->VisitedTraits[2] == NodeTemplateTraitList[0], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");

				UE_RETURN_ON_ERROR(EventB->VisitedTraits.Num() == 3, "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected number of traits visited");
				AddErrorIfFalse(EventB->VisitedTraits[0] == NodeTemplateTraitList[3], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");
				AddErrorIfFalse(EventB->VisitedTraits[1] == NodeTemplateTraitList[1], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");
				AddErrorIfFalse(EventB->VisitedTraits[2] == NodeTemplateTraitList[0], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");
			}

			// Toggle AB add to block A, make sure B visits every trait but A doesn't visit the base
			{
				auto EventA = MakeTraitEvent<FTraitCoreTest_EventA>();
				EventA->bAlwaysForwardToBase = false;
				RaiseTraitEvent(Context, Stack0, *EventA);

				UE_RETURN_ON_ERROR(EventA->VisitedTraits.Num() == 2, "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected number of traits visited");
				AddErrorIfFalse(EventA->VisitedTraits[0] == NodeTemplateTraitList[2], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");
				AddErrorIfFalse(EventA->VisitedTraits[1] == NodeTemplateTraitList[1], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");

				auto EventB = MakeTraitEvent<FTraitCoreTest_EventB>();
				RaiseTraitEvent(Context, Stack0, *EventB);

				UE_RETURN_ON_ERROR(EventB->VisitedTraits.Num() == 3, "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected number of traits visited");
				AddErrorIfFalse(EventB->VisitedTraits[0] == NodeTemplateTraitList[3], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");
				AddErrorIfFalse(EventB->VisitedTraits[1] == NodeTemplateTraitList[1], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");
				AddErrorIfFalse(EventB->VisitedTraits[2] == NodeTemplateTraitList[0], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");
			}

			// Send events A and B in a list, make sure results are the same as test above
			{
				auto EventA = MakeTraitEvent<FTraitCoreTest_EventA>();
				EventA->bAlwaysForwardToBase = false;
				auto EventB = MakeTraitEvent<FTraitCoreTest_EventB>();

				FTraitEventList EventList;
				EventList.Push(EventA);
				EventList.Push(EventB);

				RaiseTraitEvents(Context, Stack0, EventList);

				UE_RETURN_ON_ERROR(EventA->VisitedTraits.Num() == 2, "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected number of traits visited");
				AddErrorIfFalse(EventA->VisitedTraits[0] == NodeTemplateTraitList[2], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");
				AddErrorIfFalse(EventA->VisitedTraits[1] == NodeTemplateTraitList[1], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");

				UE_RETURN_ON_ERROR(EventB->VisitedTraits.Num() == 3, "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected number of traits visited");
				AddErrorIfFalse(EventB->VisitedTraits[0] == NodeTemplateTraitList[3], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");
				AddErrorIfFalse(EventB->VisitedTraits[1] == NodeTemplateTraitList[1], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");
				AddErrorIfFalse(EventB->VisitedTraits[2] == NodeTemplateTraitList[0], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");
			}

			// Validate that invalid/consumed events are skipped
			{
				auto EventA = MakeTraitEvent<FTraitCoreTest_EventA>();
				EventA->MarkConsumed();

				RaiseTraitEvent(Context, Stack0, *EventA);

				AddErrorIfFalse(EventA->VisitedTraits.Num() == 0, "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected number of traits visited");

				auto EventB = MakeTraitEvent<FTraitCoreTest_EventB>();

				FTraitEventList EventList;
				EventList.Push(EventA);
				EventList.Push(EventB);

				RaiseTraitEvents(Context, Stack0, EventList);

				AddErrorIfFalse(EventA->VisitedTraits.Num() == 0, "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected number of traits visited");

				UE_RETURN_ON_ERROR(EventB->VisitedTraits.Num() == 3, "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected number of traits visited");
				AddErrorIfFalse(EventB->VisitedTraits[0] == NodeTemplateTraitList[3], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");
				AddErrorIfFalse(EventB->VisitedTraits[1] == NodeTemplateTraitList[1], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");
				AddErrorIfFalse(EventB->VisitedTraits[2] == NodeTemplateTraitList[0], "FAnimationAnimNextRuntimeTest_TraitEventRaising -> Unexpected trait visited");
			}
		}
	}

	Tests::FUtils::CleanupAfterTests();

	return true;
}

#endif
