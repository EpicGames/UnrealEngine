// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/TraitUID.h"
#include "TraitCore/TraitEvent.h"

struct FTraitCoreTest_EventA : public FAnimNextTraitEvent
{
	DECLARE_ANIM_TRAIT_EVENT(FTraitCoreTest_EventA, FAnimNextTraitEvent)

	bool bAlwaysForwardToBase = true;

	TArray<UE::UAF::FTraitUID> VisitedTraits;
};

struct FTraitCoreTest_EventB : public FAnimNextTraitEvent
{
	DECLARE_ANIM_TRAIT_EVENT(FTraitCoreTest_EventB, FAnimNextTraitEvent)

	TArray<UE::UAF::FTraitUID> VisitedTraits;
};
