// Copyright Epic Games, Inc. All Rights Reserved.

#include "GCTestsUtil.h"

#include "AnimNextRuntimeTest.h"
#include "AnimNextTest.h"
#include "UncookedOnlyUtils.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextAnimationGraphFactory.h"
#include "Graph/AnimNextAnimationGraph_EditorData.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Graph/RigDecorator_AnimNextCppTrait.h"
#include "Graph/RigUnit_AnimNextGraphRoot.h"
#include "Graph/RigUnit_AnimNextTraitStack.h"
#include "Misc/AutomationTest.h"
#include "Script/UAFRigVMComponent.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitRegistry.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IGarbageCollection.h"
#include "TraitInterfaces/IUpdate.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::UAF
{
	/** A test trait with a latent UObject property that correctly implements IGarbageCollection */
	struct FTestTrait_SharedDataGC : FBaseTrait, IUpdate, IGarbageCollection
	{
		DECLARE_ANIM_TRAIT(FTestTrait_SharedDataGC, FBaseTrait)

		using FSharedData = FUAFTestAnimSequenceSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			TWeakObjectPtr<UAnimSequence> WeakAnimSequence;
			uint8 UpdateCount = 0;
		};
		
		// IGarbageCollection impl
		virtual void AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const override
		{
			IGarbageCollection::AddReferencedObjects(Context, Binding, Collector);

			const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

			TObjectPtr<const UAnimSequence> AnimSequence = SharedData->GetAnimSequence(Binding);
			Collector.AddReferencedObject(AnimSequence);
		}

		// IUpdate impl
		virtual void PostUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override
		{
			IUpdate::PostUpdate(Context, Binding, TraitState);

			FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
			const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();

			if (InstanceData->UpdateCount == 0)
			{
				InstanceData->WeakAnimSequence = SharedData->GetAnimSequence(Binding);
				// This anim sequence was added to root so we could verify access to it here during the first update.
				// Now we'll remove it from root so it's now GC-able before the test calls CollectGarbage.
				InstanceData->WeakAnimSequence->RemoveFromRoot();
			}
			
			const bool bIsValid = InstanceData->WeakAnimSequence.IsValid() && !InstanceData->WeakAnimSequence->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed);
			
			FRigVMExecuteContext& ExecuteContext = Context.GetRootGraphInstance().GetOrAddComponent<FUAFRigVMComponent>().GetExtendedExecuteContext().GetPublicData();
			ExecuteContext.Logf(EMessageSeverity::Info, TEXT("%s"), bIsValid ? TEXT("Valid") : TEXT("Invalid"));

			++InstanceData->UpdateCount;
		}
	};

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IGarbageCollection) \
	
	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FTestTrait_SharedDataGC, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnimationUAFRuntimeTest_SharedDataGC, "Animation.UAF.Runtime.GCSharedDataWithGCInterface", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

/*
 * Tests that a trait that has a latent UObject property must implement IGarbageCollection.
 * FTestTrait_SharedDataGC correctly implements IGarbageCollection.
 * Also verifies that the UObject is not GC'd.
 */
bool FAnimationUAFRuntimeTest_SharedDataGC::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	{
		AUTO_REGISTER_ANIM_TRAIT(FTestTrait_SharedDataGC)
		FScopedClearNodeTemplateRegistry ScopedClearNodeTemplateRegistry;

		UFactory* GraphFactory = NewObject<UUAFAnimGraphFactory>();
		TObjectPtr<UUAFAnimGraph> AnimationGraph = CastChecked<UUAFAnimGraph>(GraphFactory->FactoryCreateNew(UUAFAnimGraph::StaticClass(), GetTransientPackage(), TEXT("TestAnimNextGraph"), RF_Transient, nullptr, nullptr, NAME_None));
		UE_RETURN_ON_ERROR(AnimationGraph != nullptr, "FAnimationUAFRuntimeTest_TraitGC -> Failed to create animation graph");

		TObjectPtr<UUAFAnimGraph_EditorData> EditorData = UncookedOnly::FUtils::GetEditorData<UUAFAnimGraph_EditorData>(AnimationGraph.Get());
		UE_RETURN_ON_ERROR(EditorData != nullptr, "FAnimationUAFRuntimeTest_TraitGC -> Failed to find module editor data");

		TObjectPtr<UAnimNextController> Controller = Cast<UAnimNextController>(EditorData->GetRigVMClient()->GetController(EditorData->GetRigVMClient()->GetDefaultModel()));
		UE_RETURN_ON_ERROR(Controller != nullptr, "FAnimationUAFRuntimeTest_TraitGC -> Failed to get RigVM controller");

		// Find graph entry point
		URigVMNode* MainEntryPointNode = Controller->GetGraph()->FindNodeByName(FRigUnit_AnimNextGraphRoot::StaticStruct()->GetFName());
		UE_RETURN_ON_ERROR(MainEntryPointNode != nullptr, "FAnimationUAFRuntimeTest_TraitGC -> Failed to find main entry point node");

		URigVMPin* BeginExecutePin = MainEntryPointNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AnimNextGraphRoot, Result));
		UE_RETURN_ON_ERROR(BeginExecutePin != nullptr && BeginExecutePin->GetDirection() == ERigVMPinDirection::Input, "FAnimationUAFRuntimeTest_TraitGC -> Failed to create entry point");

		URigVMUnitNode* DecoratorStackNode = nullptr;
		FString DisplayName;
		{
			// Suspend auto compilation until we have constructed a valid trait stack
			TGuardValue<bool> SuspendCompile(EditorData->bAutoRecompileVM, false);

			// Create an empty trait stack node
			DecoratorStackNode = Controller->AddUnitNode(FRigUnit_AnimNextTraitStack::StaticStruct(), FRigVMStruct::ExecuteName, FVector2D(0.0f, 0.0f), FString(), false);
			UE_RETURN_ON_ERROR(DecoratorStackNode != nullptr, "FAnimationUAFRuntimeTest_TraitGC -> Failed to create trait stack node");

			// Link our stack result to our entry point
			Controller->AddLink(DecoratorStackNode->GetPins()[0], MainEntryPointNode->GetPins()[0]);

			// Add a trait
			const UScriptStruct* CppDecoratorStruct = FRigDecorator_AnimNextCppDecorator::StaticStruct();
			UE_RETURN_ON_ERROR(CppDecoratorStruct != nullptr, "FAnimationUAFRuntimeTest_TraitGC -> Failed to get find Cpp trait static struct");

			const FTrait* Trait = FTraitRegistry::Get().Find(FTestTrait_SharedDataGC::TraitUID);
			UE_RETURN_ON_ERROR(Trait != nullptr, "FAnimationUAFRuntimeTest_TraitGC -> Failed to find test trait");

			UScriptStruct* ScriptStruct = Trait->GetTraitSharedDataStruct();
			UE_RETURN_ON_ERROR(ScriptStruct != nullptr, "FAnimationUAFRuntimeTest_TraitGC -> Failed to find trait shared data struct");

			FString DefaultValue;
			{
				const FRigDecorator_AnimNextCppDecorator DefaultCppDecoratorStructInstance;
				FRigDecorator_AnimNextCppDecorator CppDecoratorStructInstance;
				CppDecoratorStructInstance.DecoratorSharedDataStruct = ScriptStruct;

				UE_RETURN_ON_ERROR(CppDecoratorStructInstance.CanBeAddedToNode(DecoratorStackNode, nullptr), "FAnimationUAFRuntimeTest_TraitGC -> Trait cannot be added to trait stack node");

				FRigDecorator_AnimNextCppDecorator::StaticStruct()->ExportText(DefaultValue, &CppDecoratorStructInstance, &DefaultCppDecoratorStructInstance, nullptr, PPF_None, nullptr);
			}

			FString DisplayNameMetadata;
			ScriptStruct->GetStringMetaDataHierarchical(FRigVMStruct::DisplayNameMetaName, &DisplayNameMetadata);
			DisplayName = DisplayNameMetadata.IsEmpty() ? Trait->GetTraitName() : DisplayNameMetadata;

			const FName DecoratorName = Controller->AddTrait(
				DecoratorStackNode->GetFName(),
				*CppDecoratorStruct->GetPathName(),
				*DisplayName,
				DefaultValue, INDEX_NONE, true, true);
			UE_RETURN_ON_ERROR(DecoratorName == DisplayName, TEXT("FAnimationUAFRuntimeTest_TraitGC -> Unexpected trait name"));
		}

		// Set some values on our trait
		URigVMPin* DecoratorPin = DecoratorStackNode->FindPin(*DisplayName);
		UE_RETURN_ON_ERROR(DecoratorPin != nullptr, TEXT("FAnimationUAFRuntimeTest_TraitGC -> Failed to find trait pin"));

		{
			URigVMUnitNode* TestNode = Controller->AddUnitNode(FRigUnit_AnimSequenceTest::StaticStruct());
			UE_RETURN_ON_ERROR(TestNode != nullptr, TEXT("FAnimationUAFRuntimeTest_TraitGC -> Failed to add test node"));

			const bool bSuccess = Controller->AddLink(
				TestNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AnimSequenceTest, AnimSequence)),
				DecoratorPin->GetSubPins()[1]);	// TestAnimSequence

			UE_RETURN_ON_ERROR(bSuccess, TEXT("FAnimationUAFRuntimeTest_TraitGC -> Failed to add link"));
		}

		TSharedPtr<FAnimNextGraphInstance> GraphInstance = AnimationGraph->AllocateInstance();

		// We place the graph instance in a UObject so that its contained within something the reference collector is aware of
		TObjectPtr<UGraphInstanceHolder> GraphInstanceHolder = NewObject<UGraphInstanceHolder>();
		GraphInstanceHolder->AddToRoot();
		GraphInstanceHolder->GraphInstance = GraphInstance;

		TArray<FString> Messages;
		FRigVMRuntimeSettings RuntimeSettings;
		RuntimeSettings.SetLogFunction([&Messages](const FRigVMLogSettings& InLogSettings, const FRigVMExecuteContext* InContext, const FString& Message)
		{
			Messages.Add(Message);
		});
		GraphInstance->GetOrAddComponent<FUAFRigVMComponent>().GetExtendedExecuteContext().SetRuntimeSettings(RuntimeSettings);
		
		FUpdateGraphContext UpdateGraphContext(*GraphInstance.Get(), 1.0f / 30.0f);
		
		// First update, the anim sequence has been added to root so should definitely be alive
		// Our test trait should report it as valid in its IUpdate
		UE::UAF::UpdateGraph(UpdateGraphContext);

		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

		// Second update, the anim sequence should still be alive
		// Our test trait should report it as still valid in its IUpdate
		UE::UAF::UpdateGraph(UpdateGraphContext);

		UE_RETURN_ON_ERROR(Messages.Num() == 2, "FAnimationUAFRuntimeTest_TraitGC -> Not enough messages received");
		AddErrorIfFalse(Messages[0] == TEXT("Valid"), "FAnimationUAFRuntimeTest_TraitGC -> Update 1: Test AnimSequence should be valid");
		AddErrorIfFalse(Messages[1] == TEXT("Valid"), "FAnimationUAFRuntimeTest_TraitGC -> Update 2: Test AnimSequence should be valid");

		GraphInstanceHolder->GraphInstance = nullptr;
		GraphInstanceHolder->RemoveFromRoot();
	}

	Tests::FUtils::CleanupAfterTests();

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS