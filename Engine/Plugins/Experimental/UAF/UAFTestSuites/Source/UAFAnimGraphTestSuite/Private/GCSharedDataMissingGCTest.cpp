// Copyright Epic Games, Inc. All Rights Reserved.

#include "GCTestsUtil.h"

#include "AnimNextRuntimeTest.h"
#include "AnimNextTest.h"
#include "UncookedOnlyUtils.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextAnimationGraphFactory.h"
#include "Graph/AnimNextAnimationGraph_EditorData.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Graph/RigDecorator_AnimNextCppTrait.h"
#include "Graph/RigUnit_AnimNextGraphRoot.h"
#include "Graph/RigUnit_AnimNextTraitStack.h"
#include "Misc/AutomationTest.h"
#include "Script/UAFRigVMComponent.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitRegistry.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IUpdate.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::UAF
{
	/** A test trait with a latent UObject property that incorrectly does not implement IGarbageCollection */
	struct FTestTrait_SharedDataMissingGC : FBaseTrait
	{
		DECLARE_ANIM_TRAIT(FTestTrait_MissingGC, FBaseTrait)

		using FSharedData = FUAFTestAnimSequenceSharedData;
	};

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FTestTrait_SharedDataMissingGC, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAnimationUAFRuntimeTest_SharedDataMissingGC, "Animation.UAF.Runtime.GCSharedDataMissingGCInterface", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

/*
 * Tests that a trait that has a latent UObject property must implement IGarbageCollection.
 * FTestTrait_SharedDataMissingGC incorrectly doesn't implement IGarbageCollection which should report a compile error.
 * Also verifies that the UObject is GC'd correctly in this case where it has no references.
 */
bool FAnimationUAFRuntimeTest_SharedDataMissingGC::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	{
		AUTO_REGISTER_ANIM_TRAIT(FTestTrait_SharedDataMissingGC)
		FScopedClearNodeTemplateRegistry ScopedClearNodeTemplateRegistry;

		AddExpectedError(TEXT("Trait is connected to a UObject but does not implement IGarbageCollection"), EAutomationExpectedErrorFlags::Contains, 6);

		UFactory* GraphFactory = NewObject<UUAFAnimGraphFactory>();
		UUAFAnimGraph* AnimationGraph = CastChecked<UUAFAnimGraph>(GraphFactory->FactoryCreateNew(UUAFAnimGraph::StaticClass(), GetTransientPackage(), TEXT("TestAnimNextGraph"), RF_Transient, nullptr, nullptr, NAME_None));
		UE_RETURN_ON_ERROR(AnimationGraph != nullptr, "FAnimationUAFRuntimeTest_TraitMissingGC -> Failed to create animation graph");

		UUAFAnimGraph_EditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFAnimGraph_EditorData>(AnimationGraph);
		UE_RETURN_ON_ERROR(EditorData != nullptr, "FAnimationUAFRuntimeTest_TraitMissingGC -> Failed to find module editor data");

		UAnimNextController* Controller = Cast<UAnimNextController>(EditorData->GetRigVMClient()->GetController(EditorData->GetRigVMClient()->GetDefaultModel()));
		UE_RETURN_ON_ERROR(Controller != nullptr, "FAnimationUAFRuntimeTest_TraitMissingGC -> Failed to get RigVM controller");

		// Find graph entry point
		URigVMNode* MainEntryPointNode = Controller->GetGraph()->FindNodeByName(FRigUnit_AnimNextGraphRoot::StaticStruct()->GetFName());
		UE_RETURN_ON_ERROR(MainEntryPointNode != nullptr, "FAnimationUAFRuntimeTest_TraitMissingGC -> Failed to find main entry point node");

		URigVMPin* BeginExecutePin = MainEntryPointNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AnimNextGraphRoot, Result));
		UE_RETURN_ON_ERROR(BeginExecutePin != nullptr && BeginExecutePin->GetDirection() == ERigVMPinDirection::Input, "FAnimationUAFRuntimeTest_TraitMissingGC -> Failed to create entry point");

		URigVMUnitNode* DecoratorStackNode = nullptr;
		FString DisplayName;
		{
			// Suspend auto compilation until we have constructed a valid trait stack
			TGuardValue<bool> SuspendCompile(EditorData->bAutoRecompileVM, false);

			// Create an empty trait stack node
			DecoratorStackNode = Controller->AddUnitNode(FRigUnit_AnimNextTraitStack::StaticStruct(), FRigVMStruct::ExecuteName, FVector2D(0.0f, 0.0f), FString(), false);
			UE_RETURN_ON_ERROR(DecoratorStackNode != nullptr, "FAnimationUAFRuntimeTest_TraitMissingGC -> Failed to create trait stack node");

			// Link our stack result to our entry point
			Controller->AddLink(DecoratorStackNode->GetPins()[0], MainEntryPointNode->GetPins()[0]);

			// Add a trait
			const UScriptStruct* CppDecoratorStruct = FRigDecorator_AnimNextCppDecorator::StaticStruct();
			UE_RETURN_ON_ERROR(CppDecoratorStruct != nullptr, "FAnimationUAFRuntimeTest_TraitMissingGC -> Failed to get find Cpp trait static struct");

			const FTrait* Trait = FTraitRegistry::Get().Find(FTestTrait_SharedDataMissingGC::TraitUID);
			UE_RETURN_ON_ERROR(Trait != nullptr, "FAnimationUAFRuntimeTest_TraitMissingGC -> Failed to find test trait");

			UScriptStruct* ScriptStruct = Trait->GetTraitSharedDataStruct();
			UE_RETURN_ON_ERROR(ScriptStruct != nullptr, "FAnimationUAFRuntimeTest_TraitMissingGC -> Failed to find trait shared data struct");

			FString DefaultValue;
			{
				const FRigDecorator_AnimNextCppDecorator DefaultCppDecoratorStructInstance;
				FRigDecorator_AnimNextCppDecorator CppDecoratorStructInstance;
				CppDecoratorStructInstance.DecoratorSharedDataStruct = ScriptStruct;

				UE_RETURN_ON_ERROR(CppDecoratorStructInstance.CanBeAddedToNode(DecoratorStackNode, nullptr), "FAnimationUAFRuntimeTest_TraitMissingGC -> Trait cannot be added to trait stack node");

				FRigDecorator_AnimNextCppDecorator::StaticStruct()->ExportText(DefaultValue, &CppDecoratorStructInstance, &DefaultCppDecoratorStructInstance, nullptr, PPF_None, nullptr);
			}

			FString DisplayNameMetadata;
			ScriptStruct->GetStringMetaDataHierarchical(FRigVMStruct::DisplayNameMetaName, &DisplayNameMetadata);
			DisplayName = DisplayNameMetadata.IsEmpty() ? Trait->GetTraitName() : DisplayNameMetadata;

			const FName DecoratorName = Controller->AddTrait(
				DecoratorStackNode->GetFName(),
				*CppDecoratorStruct->GetPathName(),
				*DisplayName,
				DefaultValue, INDEX_NONE, true, true);
			UE_RETURN_ON_ERROR(DecoratorName == DisplayName, TEXT("FAnimationUAFRuntimeTest_TraitMissingGC -> Unexpected trait name"));
		}

		// Set some values on our trait
		URigVMPin* DecoratorPin = DecoratorStackNode->FindPin(*DisplayName);
		UE_RETURN_ON_ERROR(DecoratorPin != nullptr, TEXT("FAnimationUAFRuntimeTest_TraitMissingGC -> Failed to find trait pin"));

		{
			URigVMUnitNode* TestNode = Controller->AddUnitNode(FRigUnit_AnimSequenceTest::StaticStruct());
			UE_RETURN_ON_ERROR(TestNode != nullptr, TEXT("FAnimationUAFRuntimeTest_TraitMissingGC -> Failed to add test node"));

			const bool bSuccess = Controller->AddLink(
				TestNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AnimSequenceTest, AnimSequence)),
				DecoratorPin->GetSubPins()[1]);	// TestAnimSequence

			UE_RETURN_ON_ERROR(TestNode != nullptr, TEXT("FAnimationUAFRuntimeTest_TraitMissingGC -> Failed to add link"));
		}

		TSharedPtr<FAnimNextGraphInstance> GraphInstance = AnimationGraph->AllocateInstance();

		// No need to update the graph, the graph will not compile with this invalid trait
	}

	Tests::FUtils::CleanupAfterTests();

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS