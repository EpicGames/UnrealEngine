// Copyright Epic Games, Inc. All Rights Reserved.

#include "GCTestsUtil.h"

#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextAnimationGraphFactory.h"
#include "Misc/AutomationTest.h"

FRigUnit_AnimSequenceTest_Execute()
{
	if (!bCreated)
	{
		AnimSequence = NewObject<UAnimSequence>(GetTransientPackage(), NAME_None);
		AnimSequence->AddToRoot(); // Will be removed during the test
		bCreated = true;
	}
}

void UGraphInstanceHolder::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector) 
{
	UGraphInstanceHolder* This = CastChecked<UGraphInstanceHolder>(InThis);
	if (This->GraphInstance.IsValid())
	{
		Collector.AddPropertyReferencesWithStructARO(FAnimNextGraphInstance::StaticStruct(), This->GraphInstance.Get(), This);
	}

	Super::AddReferencedObjects(InThis, Collector);
}