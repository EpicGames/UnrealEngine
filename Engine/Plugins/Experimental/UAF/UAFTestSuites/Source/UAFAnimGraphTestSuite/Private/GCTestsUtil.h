// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraitCore/TraitBinding.h"
#include "TraitCore/TraitSharedData.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimTypes.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Graph/RigUnit_AnimNextBase.h"

#include "GCTestsUtil.generated.h"

USTRUCT()
struct FUAFTestAnimSequenceSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Default")
	TObjectPtr<UAnimSequence> AnimSequence;
	
	// Adding a UEnum here to test for false positives when validating trait stacks.
	// This pin shouldn't ever require GC handling so let's test it *doesn't* generate an error.
	UPROPERTY(EditAnywhere, Category = "Default")
	EAnimInterpolationType DummyProp0 = EAnimInterpolationType::Linear;

	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(AnimSequence) \
		GeneratorMacro(DummyProp0) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FUAFTestAnimSequenceSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

USTRUCT(meta = (DocumentationPolicy = "None"))
struct FRigUnit_AnimSequenceTest : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()
	
	RIGVM_METHOD()
	void Execute();

	UPROPERTY(EditAnywhere, Category = "Default", meta = (Output))
	TObjectPtr<UAnimSequence> AnimSequence;

	UPROPERTY()
	bool bCreated = false;
};

UCLASS()
class UGraphInstanceHolder : public UObject
{
	GENERATED_BODY()

public:
	TSharedPtr<FAnimNextGraphInstance> GraphInstance;

	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
};