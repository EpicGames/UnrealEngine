// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "AnimNextAnimGraphTraitGraphTest.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

namespace UE::UAF::AnimGraph::Tests
{
	class FModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override
		{
			// Register struct/class types required for contained tests
			static UScriptStruct* const AllowedStructTypes[] =
			{
				FTestDerivedVector::StaticStruct()
			};

			FRigVMRegistry& RigVMRegistry = FRigVMRegistry::Get();
			RigVMRegistry.RegisterStructTypes(AllowedStructTypes);
			
			static TPair<UClass*, FRigVMRegistry::ERegisterObjectOperation> const AllowedObjectTypes[] =
			{
				{ UUAFSharedVariables::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
			};

			
			RigVMRegistry.RegisterObjectTypes(AllowedObjectTypes);
		}
	};
}

IMPLEMENT_MODULE(UE::UAF::AnimGraph::Tests::FModule, UAFAnimGraphTestSuite)
