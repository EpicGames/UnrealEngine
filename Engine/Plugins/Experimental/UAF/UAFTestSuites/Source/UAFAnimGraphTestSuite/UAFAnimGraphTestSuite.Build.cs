// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class UAFAnimGraphTestSuite : ModuleRules
	{
		public UAFAnimGraphTestSuite(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"CQTest",
					"Engine",
					"UAF",
					"UAFTestSuite",
					"UAFAnimGraph",
					"RigVM", 
					"PythonScriptPlugin",
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"UnrealEd",
						"UAFUncookedOnly",
						"UAFEditor",
						"RigVMDeveloper",
						"UAFAnimGraphEditor",
						"UAFAnimGraphUncookedOnly",
					}
				);
			}
		}
	}
}