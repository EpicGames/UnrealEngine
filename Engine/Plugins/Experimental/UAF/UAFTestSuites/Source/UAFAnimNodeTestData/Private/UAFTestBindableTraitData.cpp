// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFTestBindableTraitData.h"
#include "TraitCore/TraitBinding.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FTestBindableTrait)
	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FTestBindableTrait, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
}
