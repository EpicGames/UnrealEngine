// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/TrajectoryTypes.h"
#include "UAF/AnimNodeCore/UAFAnimNodeData.h"
#include "BindableValue/UAFBindableTypes.h"
#include "UAFAnimNodeTestVars.generated.h"

/** Nested struct for FBindableStruct SubProperty binding tests. */
USTRUCT()
struct FUAFAnimNodeNestedTestStruct
{
	GENERATED_BODY()

	UPROPERTY()
	FVector Vec = FVector::ZeroVector;

	UPROPERTY()
	FQuat Quat = FQuat::Identity;

	UPROPERTY()
	FTransform Transform = FTransform::Identity;
};

/**
 * Minimal USTRUCT used only in UAFAnimNode LLT unit tests.
 * Provides a set of typed properties that cover the common resolved types
 * (bool, float, double, int32, FVector, FQuat) so that FBindableXxx runtime resolution
 * tests can exercise variable and sub-property binding against a live FUAFAssetInstance.
 */
USTRUCT()
struct FUAFAnimNodeTestVars
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Variables")
	bool bBool = false;

	UPROPERTY(EditAnywhere, Category = "Variables")
	float FloatVal = 0.0f;

	UPROPERTY(EditAnywhere, Category = "Variables")
	double DoubleVal = 0.0;

	UPROPERTY(EditAnywhere, Category = "Variables")
	int32 IntVal = 0;

	UPROPERTY(EditAnywhere, Category = "Variables")
	FVector VectorVar = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, Category = "Variables")
	FQuat QuatVar = FQuat::Identity;

	UPROPERTY(EditAnywhere, Category = "Variables")
	FTransform TransformVar = FTransform::Identity;

	UPROPERTY(EditAnywhere, Category = "Variables")
	FUAFAnimNodeNestedTestStruct NestedVar;
};

/** 10-property struct used by bulk-resolution performance benchmarks. */
USTRUCT()
struct FUAFAnimNodePerfVars10
{
	GENERATED_BODY()

	// Float targets/sources for variable-binding benchmark
	UPROPERTY() float f0 = 0.f;
	UPROPERTY() float f1 = 0.f;
	UPROPERTY() float f2 = 0.f;
	UPROPERTY() float f3 = 0.f;
	UPROPERTY() float f4 = 0.f;
	UPROPERTY() float f5 = 0.f;
	UPROPERTY() float f6 = 0.f;
	UPROPERTY() float f7 = 0.f;
	UPROPERTY() float f8 = 0.f;
	UPROPERTY() float f9 = 0.f;

	// Vector sources for subproperty-binding benchmark (bind .X → f0..f9)
	UPROPERTY() FVector v0 = FVector::ZeroVector;
	UPROPERTY() FVector v1 = FVector::ZeroVector;
	UPROPERTY() FVector v2 = FVector::ZeroVector;
	UPROPERTY() FVector v3 = FVector::ZeroVector;
	UPROPERTY() FVector v4 = FVector::ZeroVector;
	UPROPERTY() FVector v5 = FVector::ZeroVector;
	UPROPERTY() FVector v6 = FVector::ZeroVector;
	UPROPERTY() FVector v7 = FVector::ZeroVector;
	UPROPERTY() FVector v8 = FVector::ZeroVector;
	UPROPERTY() FVector v9 = FVector::ZeroVector;
};

namespace UE::UAF
{

/** Test enum for FBindableEnum exercises. */
UENUM()
enum class EUAFAnimNodeTestEnum : uint8
{
	Alpha = 0,
	Beta = 1,
	Gamma = 2,
};

USTRUCT(DisplayName = "Test Anim Node", meta=(Hidden))
struct FUAFTestAnimNodeData : public FUAFAnimNodeData
{
	GENERATED_BODY()

	// FUAFAnimNodeData impl
	virtual FUAFAnimNodePtr CreateInstance(FUAFAnimGraphUpdateContext& Context) const override { return nullptr; }

	UPROPERTY(EditAnywhere, Category = "NodeData")
	FBindableBool BoolVal = true;

	UPROPERTY(EditAnywhere, Category = "NodeData")
	FBindableFloat FloatVal = 1.0f;

	UPROPERTY(EditAnywhere, Category = "NodeData")
	FBindableDouble DoubleVal = 2.0;

	UPROPERTY(EditAnywhere, Category = "NodeData")
	FBindableInt32 Int32Val = 3;

	UPROPERTY(EditAnywhere, Category = "NodeData")
	FBindableInt64 Int64Val = 4;

	UPROPERTY(EditAnywhere, Category = "NodeData")
	FBindableByte ByteVal = 5;

	UPROPERTY(EditAnywhere, Category = "NodeData")
	FBindableName NameVal = FName(TEXT("TestName"));

	UPROPERTY(EditAnywhere, Category = "NodeData")
	FBindableVector VectorVal = FVector(1.0f, 2.0f, 3.0f);

	UPROPERTY(EditAnywhere, Category = "NodeData")
	FBindableQuat QuatVal = FQuat::Identity;

	UPROPERTY(EditAnywhere, Category = "NodeData")
	FBindableTransform TransformBindableVal = FTransform::Identity;

	UPROPERTY(EditAnywhere, Category = "NodeData")
	FBindableObject ObjectVal = FBindableObject();

	UPROPERTY(EditAnywhere, Category = "NodeData")
	FBindableObject ActorVal = FBindableObject(AActor::StaticClass());

	UPROPERTY(EditAnywhere, Category = "NodeData")
	FBindableEnum EnumVal = FBindableEnum(EUAFAnimNodeTestEnum::Beta);

	UPROPERTY(EditAnywhere, Category = "NodeData")
	FBindableStruct StructVal = FBindableStruct(FUAFAnimNodeTestVars{});

	UPROPERTY(EditAnywhere, Category = "NodeData")
	FBindableStruct TransformTrajectory = FBindableStruct(FTransformTrajectory());

	UPROPERTY(EditAnywhere, Category = "NodeData")
	FBindableStruct TransformVal = FBindableStruct(FTransform());
};

} // namespace UE::UAF
