// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitBinding.h"
#include "TraitCore/TraitSharedData.h"
#include "BindableValue/UAFBindableTypes.h"

#include "UAFTestBindableTraitData.generated.h"

/**
 * Minimal test-only trait shared data with a single FBindableBool property.
 * Used by CQTests to exercise SetPinDefaultValue round-trip on FBindable
 * types within trait pins, verifying that the RigVM controller detects
 * value changes through text comparison.
 */
USTRUCT()
struct FUAFTestBindableTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Test")
	FBindableBool bTestBool = false;

#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(bTestBool) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FUAFTestBindableTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{
	/**
	 * Minimal test-only trait that uses FUAFTestBindableTraitSharedData.
	 * Required so that FRigDecorator_AnimNextCppDecorator can resolve the
	 * shared data struct to a registered trait during AddTrait.
	 */
	struct FTestBindableTrait : FBaseTrait
	{
		DECLARE_ANIM_TRAIT(FTestBindableTrait, FBaseTrait)

		using FSharedData = FUAFTestBindableTraitSharedData;
	};
}
