// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UAFAnimNodeTestData : ModuleRules
{
	public UAFAnimNodeTestData(ReadOnlyTargetRules Target) : base(Target)
	{
		bAllowUETypesInNamespaces = true;

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"UAF",
			"UAFAnimNode",
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"UAFAnimGraph",
			"RigVM",
			"ControlRig",
			"Engine",
		});
	}
}
