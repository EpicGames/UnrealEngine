// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS
#if WITH_EDITOR

#include "UAFTestsUtilities.h"
#include "UAFTestBlueprintLibrary.h"
#include "CQTest.h"

#include "AnimNextFunctionReference.h"
#include "AnimNextRigVMAsset.h"
#include "Param/ParamCompatibility.h"
#include "Param/ParamUtils.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "BindableValue/UAFBindableTypes.h"
#include "BindableValue/UAFPropertyBinding.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Module/AnimNextModule.h"
#include "Module/AnimNextModuleFactory.h"
#include "Script/UAFRigVMComponent.h"
#include "UAFAssetInstance.h"
#include "UncookedOnlyUtils.h"

// ---------------------------------------------------------------------------
// Test fixture — friend of FUAFAssetInstance for accessing protected members
// ---------------------------------------------------------------------------

namespace UE::UAF::Tests
{

struct FUAFBindingTestFixture
{
	static void InitInstanceFromAsset(FUAFAssetInstance& InInstance, const UUAFRigVMAsset* InAsset)
	{
		InInstance.Asset = InAsset;
		InInstance.InitializeVariables();
		InInstance.CopyDefaultComponents();
		InInstance.BindDefaultComponents();
		// Ensure the RigVM component exists — it may not be in the asset's default
		// components, so GetComponent lazily creates it and calls OnBindToInstance
		// which sets up the VM, execution context, and external variable runtime data.
		[[maybe_unused]] FUAFRigVMComponent& RigVMComponent = InInstance.GetOrAddComponent<FUAFRigVMComponent>();
	}
};

} // namespace UE::UAF::Tests

// ---------------------------------------------------------------------------
// Helpers
// ---------------------------------------------------------------------------

namespace
{

using FFixture = UE::UAF::Tests::FUAFBindingTestFixture;

UUAFSystem* CreateAssetWithParameterlessFunction(
	const FString& InFunctionName,
	const FString& InReturnCPPType,
	const FString& InDefaultValue)
{
	UUAFSystem* Asset = CastChecked<UUAFSystem>(
		UAFTestsUtilities::CreateFactoryObject(
			NewObject<UUAFSystemFactory>(GetTransientPackage()),
			UUAFSystem::StaticClass()));

	URigVMLibraryNode* FuncNode = UAFTestsUtilities::AddFunctionNode(Asset, InFunctionName);
	UAFTestsUtilities::AddPin(Asset, FuncNode, ERigVMPinDirection::Output,
		TEXT("ReturnValue"), InReturnCPPType, NAME_None, InDefaultValue);

	// Mark the function as public so BuildFunctionHeadersContext includes it and
	// BuildFunctionWrapperEvents creates the proper wrapper entry + external variables.
	UUAFTestBlueprintLibrary::MarkFunctionAsPublic(Asset, FName(InFunctionName));

	// Recompile — now the function is public, BuildFunctionWrapperEvents will create
	// wrapper events with external variables, populating FunctionData on the asset.
	UUAFTestBlueprintLibrary::RecompileVM(Asset);

	return Asset;
}

/** Creates a binding using EventName only (no GUID) — for backward compat testing. */
FUAFPropertyBinding MakeFunctionBinding(FName InFunctionName, const UUAFRigVMAsset* InAsset)
{
	FUAFPropertyBinding Binding;
	Binding.SourceType = EUAFBindingSourceType::Function;
	Binding.SourceFunction = FAnimNextFunctionReference::FromEventName(
		FName(*UE::UAF::UncookedOnly::FUtils::MakeFunctionWrapperEventName(InFunctionName)),
		InAsset);
	return Binding;
}

/** Creates a binding using GUID + EventName (rename-robust). */
FUAFPropertyBinding MakeFunctionBindingWithGuid(FName InFunctionName, const UUAFRigVMAsset* InAsset)
{
	FUAFPropertyBinding Binding;
	Binding.SourceType = EUAFBindingSourceType::Function;

	FGuid FunctionGuid = UUAFTestBlueprintLibrary::GetFunctionGuid(InAsset, InFunctionName);

	// Build the reference with GUID via a temporary header
	FRigVMGraphFunctionHeader TempHeader;
	TempHeader.Name = InFunctionName;
	TempHeader.Variant.Guid = FunctionGuid;
	Binding.SourceFunction = FAnimNextFunctionReference::FromHeader(TempHeader, InAsset);

	return Binding;
}

/** Builds a minimal FRigVMGraphFunctionHeader with a single output argument of the given type. */
FRigVMGraphFunctionHeader MakeTestFunctionHeader(FName InCPPType, UObject* InCPPTypeObject = nullptr)
{
	FRigVMGraphFunctionHeader Header;
	Header.Name = TEXT("TestFunc");

	// Add execute context argument (always present, should be ignored by filters)
	FRigVMGraphFunctionArgument ExecArg;
	ExecArg.Name = TEXT("ExecuteContext");
	ExecArg.CPPType = TEXT("FAnimNextExecuteContext");
	ExecArg.Direction = ERigVMPinDirection::IO;
	Header.Arguments.Add(ExecArg);

	// Add output argument (return value)
	FRigVMGraphFunctionArgument OutputArg;
	OutputArg.Name = TEXT("ReturnValue");
	OutputArg.CPPType = InCPPType;
	OutputArg.CPPTypeObject = InCPPTypeObject;
	OutputArg.Direction = ERigVMPinDirection::Output;
	Header.Arguments.Add(OutputArg);

	return Header;
}

/** Tests return type compatibility using the same check as the function picker filter. */
bool IsReturnTypeCompatible(const FAnimNextParamType& InTargetType, FName InReturnCPPType, UObject* InCPPTypeObject = nullptr)
{
	FRigVMTemplateArgumentType RigVMArgType(InReturnCPPType, InCPPTypeObject);
	FAnimNextParamType ReturnType = FAnimNextParamType::FromRigVMTemplateArgument(RigVMArgType);
	return UE::UAF::FParamUtils::GetCompatibility(InTargetType, ReturnType).IsCompatibleWithDataLoss();
}

} // anonymous namespace

// ---------------------------------------------------------------------------
// Tests
// ---------------------------------------------------------------------------

TEST_CLASS(FBindableValueFunctionTests, "Animation.UAF.Functional")
{
	TEST_METHOD(FBindableFloat_FunctionBinding_ResolvesReturnValue)
	{
		TestCommandBuilder
			.Do(TEXT("Create asset, compile, and verify function binding resolves float"), [&]()
			{
				UUAFSystem* Asset = CreateAssetWithParameterlessFunction(
					TEXT("GetFloat"), TEXT("float"), TEXT("42.0"));
				ASSERT_THAT(IsNotNull(Asset));
				ASSERT_THAT(IsNotNull(Asset->GetRigVM()));

				FUAFAssetInstance Instance;
				FFixture::InitInstanceFromAsset(Instance, Asset);

				FBindableFloat Field;
				Field.SetConstantValue(0.0f);
				Field.SetBinding(MakeFunctionBinding(TEXT("GetFloat"), Asset));

				const float Result = Field.GetValue(&Instance);
				ASSERT_THAT(IsNear(Result, 42.0f, UE_KINDA_SMALL_NUMBER));
			});
	}

	TEST_METHOD(FBindableBool_FunctionBinding_ResolvesReturnValue)
	{
		TestCommandBuilder
			.Do(TEXT("Create asset, compile, and verify function binding resolves bool"), [&]()
			{
				UUAFSystem* Asset = CreateAssetWithParameterlessFunction(
					TEXT("GetBool"), TEXT("bool"), TEXT("true"));
				ASSERT_THAT(IsNotNull(Asset));

				FUAFAssetInstance Instance;
				FFixture::InitInstanceFromAsset(Instance, Asset);

				FBindableBool Field;
				Field.SetConstantValue(false);
				Field.SetBinding(MakeFunctionBinding(TEXT("GetBool"), Asset));

				ASSERT_THAT(IsTrue(Field.GetValue(&Instance)));
			});
	}

	TEST_METHOD(FBindableInt32_FunctionBinding_ResolvesReturnValue)
	{
		TestCommandBuilder
			.Do(TEXT("Create asset, compile, and verify function binding resolves int32"), [&]()
			{
				UUAFSystem* Asset = CreateAssetWithParameterlessFunction(
					TEXT("GetInt"), TEXT("int32"), TEXT("99"));
				ASSERT_THAT(IsNotNull(Asset));

				FUAFAssetInstance Instance;
				FFixture::InitInstanceFromAsset(Instance, Asset);

				FBindableInt32 Field;
				Field.SetConstantValue(0);
				Field.SetBinding(MakeFunctionBinding(TEXT("GetInt"), Asset));

				ASSERT_THAT(AreEqual(99, Field.GetValue(&Instance)));
			});
	}

	TEST_METHOD(FBindableFloat_FunctionBinding_OverridesConstantValue)
	{
		TestCommandBuilder
			.Do(TEXT("Verify function binding takes precedence over ConstantValue"), [&]()
			{
				UUAFSystem* Asset = CreateAssetWithParameterlessFunction(
					TEXT("GetFloat"), TEXT("float"), TEXT("42.0"));
				ASSERT_THAT(IsNotNull(Asset));

				FUAFAssetInstance Instance;
				FFixture::InitInstanceFromAsset(Instance, Asset);

				FBindableFloat Field;
				Field.SetConstantValue(999.0f);
				Field.SetBinding(MakeFunctionBinding(TEXT("GetFloat"), Asset));

				// Function result (42.0) should override ConstantValue (999.0)
				ASSERT_THAT(IsNear(Field.GetValue(&Instance), 42.0f, UE_KINDA_SMALL_NUMBER));
			});
	}

	TEST_METHOD(FBindableFloat_FunctionBinding_InvalidEventNameFallsBack)
	{
		TestCommandBuilder
			.Do(TEXT("Verify invalid function event name falls back to ConstantValue"), [&]()
			{
				UUAFSystem* Asset = CreateAssetWithParameterlessFunction(
					TEXT("GetFloat"), TEXT("float"), TEXT("42.0"));
				ASSERT_THAT(IsNotNull(Asset));

				FUAFAssetInstance Instance;
				FFixture::InitInstanceFromAsset(Instance, Asset);

				FBindableFloat Field;
				Field.SetConstantValue(999.0f);

				// Use a bogus event name that doesn't exist
				FUAFPropertyBinding Binding;
				Binding.SourceType = EUAFBindingSourceType::Function;
				Binding.SourceFunction = FAnimNextFunctionReference::FromEventName(
					TEXT("__InternalCall_NonExistentFunction"), Asset);
				Field.SetBinding(MoveTemp(Binding));

				// Should fall back to ConstantValue since function handle is invalid
				ASSERT_THAT(IsNear(Field.GetValue(&Instance), 999.0f, UE_KINDA_SMALL_NUMBER));
			});
	}

	// TODO: Add end-to-end test for functions with input variable arguments once the test
	// infrastructure supports full instance initialization with internal wrapper variables.
	// The runtime fix (ArgIndices.Last() + >= 1) and UI filter fix (skip bIsInputVariable)
	// are validated by manual editor testing with real system functions.

	// -----------------------------------------------------------------------
	// GUID-based function reference tests
	// -----------------------------------------------------------------------

	TEST_METHOD(FBindableFloat_GuidBinding_ResolvesAfterEventNameChange)
	{
		TestCommandBuilder
			.Do(TEXT("Verify GUID-based lookup works even when EventName is stale (simulates rename)"), [&]()
			{
				UUAFSystem* Asset = CreateAssetWithParameterlessFunction(
					TEXT("GetFloat"), TEXT("float"), TEXT("42.0"));
				ASSERT_THAT(IsNotNull(Asset));

				FUAFAssetInstance Instance;
				FFixture::InitInstanceFromAsset(Instance, Asset);

				// Create a GUID-based binding, then corrupt the EventName to simulate rename
				FUAFPropertyBinding Binding = MakeFunctionBindingWithGuid(TEXT("GetFloat"), Asset);
				// Override the EventName with a stale/wrong name
				FAnimNextFunctionReference StaleRef = FAnimNextFunctionReference::FromEventName(
					TEXT("__InternalCall_OldName"), Asset);
				// But keep the GUID from the original — build a reference with correct GUID + wrong EventName
				FRigVMGraphFunctionHeader HeaderWithStaleEvent;
				HeaderWithStaleEvent.Name = TEXT("OldName"); // wrong name
				HeaderWithStaleEvent.Variant.Guid = UUAFTestBlueprintLibrary::GetFunctionGuid(Asset, TEXT("GetFloat"));
				Binding.SourceFunction = FAnimNextFunctionReference::FromHeader(HeaderWithStaleEvent, Asset);

				FBindableFloat Field;
				Field.SetConstantValue(0.0f);
				Field.SetBinding(MoveTemp(Binding));

				// GUID lookup should find the function even though EventName is wrong
				ASSERT_THAT(IsNear(Field.GetValue(&Instance), 42.0f, UE_KINDA_SMALL_NUMBER));
			});
	}

	TEST_METHOD(FBindableFloat_EventNameFallback_WorksWithoutGuid)
	{
		TestCommandBuilder
			.Do(TEXT("Verify EventName fallback works when GUID is invalid (backward compat)"), [&]()
			{
				UUAFSystem* Asset = CreateAssetWithParameterlessFunction(
					TEXT("GetFloat"), TEXT("float"), TEXT("42.0"));
				ASSERT_THAT(IsNotNull(Asset));

				FUAFAssetInstance Instance;
				FFixture::InitInstanceFromAsset(Instance, Asset);

				// Create binding with EventName only (no GUID) — simulates old serialized data
				FBindableFloat Field;
				Field.SetConstantValue(0.0f);
				Field.SetBinding(MakeFunctionBinding(TEXT("GetFloat"), Asset));

				// Should resolve via EventName fallback
				ASSERT_THAT(IsNear(Field.GetValue(&Instance), 42.0f, UE_KINDA_SMALL_NUMBER));
			});
	}

	// -----------------------------------------------------------------------
	// Return type compatibility tests
	// -----------------------------------------------------------------------

	TEST_METHOD(ReturnTypeCompatibility_FloatToFloat_IsCompatible)
	{
		TestCommandBuilder
			.Do(TEXT("float return → FBindableFloat is compatible"), [&]()
			{
				const FAnimNextParamType FloatType = FAnimNextParamType::GetType<float>();
				ASSERT_THAT(IsTrue(IsReturnTypeCompatible(FloatType, TEXT("float"))));
			});
	}

	TEST_METHOD(ReturnTypeCompatibility_DoubleToFloat_IsCompatible)
	{
		TestCommandBuilder
			.Do(TEXT("double return → FBindableFloat is compatible (demotion)"), [&]()
			{
				const FAnimNextParamType FloatType = FAnimNextParamType::GetType<float>();
				ASSERT_THAT(IsTrue(IsReturnTypeCompatible(FloatType, TEXT("double"))));
			});
	}

	TEST_METHOD(ReturnTypeCompatibility_FloatToDouble_IsCompatible)
	{
		TestCommandBuilder
			.Do(TEXT("float return → FBindableDouble is compatible (promotion)"), [&]()
			{
				const FAnimNextParamType DoubleType = FAnimNextParamType::GetType<double>();
				ASSERT_THAT(IsTrue(IsReturnTypeCompatible(DoubleType, TEXT("float"))));
			});
	}

	TEST_METHOD(ReturnTypeCompatibility_BoolToBool_IsCompatible)
	{
		TestCommandBuilder
			.Do(TEXT("bool return → FBindableBool is compatible"), [&]()
			{
				const FAnimNextParamType BoolType = FAnimNextParamType::GetType<bool>();
				ASSERT_THAT(IsTrue(IsReturnTypeCompatible(BoolType, TEXT("bool"))));
			});
	}

	TEST_METHOD(ReturnTypeCompatibility_QuatToFloat_IsIncompatible)
	{
		TestCommandBuilder
			.Do(TEXT("FQuat return → FBindableFloat is incompatible"), [&]()
			{
				const FAnimNextParamType FloatType = FAnimNextParamType::GetType<float>();
				ASSERT_THAT(IsFalse(IsReturnTypeCompatible(FloatType, TEXT("FQuat"), TBaseStructure<FQuat>::Get())));
			});
	}

	TEST_METHOD(ReturnTypeCompatibility_VectorToFloat_IsIncompatible)
	{
		TestCommandBuilder
			.Do(TEXT("FVector return → FBindableFloat is incompatible"), [&]()
			{
				const FAnimNextParamType FloatType = FAnimNextParamType::GetType<float>();
				ASSERT_THAT(IsFalse(IsReturnTypeCompatible(FloatType, TEXT("FVector"), TBaseStructure<FVector>::Get())));
			});
	}
};

#endif // WITH_EDITOR
#endif // WITH_DEV_AUTOMATION_TESTS
