// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS
#if WITH_EDITOR

#include "UAFTestsUtilities.h"
#include "CQTest.h"
#include "Editor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextAnimationGraphFactory.h"
#include "Graph/RigUnit_AnimNextTraitStack.h"
#include "RigVMModel/RigVMController.h"
#include "RigVMModel/RigVMPin.h"
#include "UAFTestBlueprintLibrary.h"
#include "UAFTestBindableTraitData.h"
#include "BindableValue/UAFPropertyBinding.h"

// Exercises the FBindableBool public API (SetBinding, ClearBinding, Value mutation)
// and verifies the exported text changes are properly detected by the RigVM controller.
//
// The trait system decomposes FBindableBool into a ConstantValue sub-pin via the
// latent property system. This test verifies both:
//  - Direct constant value manipulation through the sub-pin
//  - The FBindableBool public API produces correctly differentiated export text,
//    which is the mechanism the RigVM controller uses for change detection
TEST_CLASS(FBindableValueTraitPinTests, "Animation.UAF.Functional")
{
	UUAFAnimGraph* AnimNextAnimationGraph = nullptr;
	UAssetEditorSubsystem* AssetEditorSubsystem = nullptr;
	URigVMController* Controller = nullptr;
	URigVMGraph* ControllerGraph = nullptr;
	const FString TraitStackNodeName = TEXT("AnimNextTraitStack");
	const FString TestTraitName = TEXT("FTestBindableTrait");
	const FString BoolPinName = GET_MEMBER_NAME_STRING_CHECKED(FUAFTestBindableTraitSharedData, bTestBool);
	// ConstantValue is private on FBindableBool -- validated via FindPin at runtime
	const FString ConstantValueName = TEXT("ConstantValue");

	BEFORE_EACH()
	{
		AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		ASSERT_THAT(IsNotNull(AssetEditorSubsystem));
	}

	// Verifies that the FBindableBool public API (SetBinding, ClearBinding)
	// produces export text where each distinct binding state maps to a unique
	// string. This is the core property the RigVM controller depends on: it
	// compares exported text to detect changes (RigVMController.cpp
	// SetPinDefaultValue). Without the always-emit __Binding=() marker,
	// clearing a binding would produce text identical to the original unbound
	// state, and the controller would silently discard the change.
	//
	// Uses ConstantValue=true (non-default) because ExportTextItem's internal
	// self-comparison technique (passes this as both Value and Defaults) relies
	// on sub-property values being non-default to survive the ExportText_Direct
	// Identical check. The LLT BindableValueSerializationTests follow the same
	// pattern.
	TEST_METHOD(FBindableBool_PublicAPI_ExportTextDiffersPerState)
	{
		TestCommandBuilder
			.Do(TEXT("Verify export text differs for each FBindableBool state"), [&]()
			{
				const UScriptStruct* BoolStruct = FBindableBool::StaticStruct();
				ASSERT_THAT(IsNotNull(BoolStruct));

				FUAFPropertyBinding TestBinding;
				TestBinding.SourceType = EUAFBindingSourceType::Variable;

				// --- State 1: unbound (ConstantValue=true, no binding) ---
				FBindableBool Unbound(true);

				// --- State 2: bound (ConstantValue=true, binding set) ---
				FBindableBool Bound(true);
				Bound.SetBinding(TestBinding);
				ASSERT_THAT(IsTrue(Bound.HasBinding()));

				// --- State 3: cleared (ConstantValue=true, binding set then cleared) ---
				FBindableBool Cleared(true);
				Cleared.SetBinding(TestBinding);
				Cleared.ClearBinding();
				ASSERT_THAT(IsFalse(Cleared.HasBinding()));

				// Export each state to text via UScriptStruct::ExportText, which
				// dispatches to FBindableBool::ExportTextItem (native override).
				// This is the same path the RigVM uses to serialize pin values.
				FString UnboundText, BoundText, ClearedText;
				BoolStruct->ExportText(UnboundText, &Unbound, nullptr, nullptr, 0, nullptr, true);
				BoolStruct->ExportText(BoundText, &Bound, nullptr, nullptr, 0, nullptr, true);
				BoolStruct->ExportText(ClearedText, &Cleared, nullptr, nullptr, 0, nullptr, true);

				// The RigVM controller compares pin text via string equality:
				//   if (InPin->GetDefaultValue() != ClampedDefaultValue)
				// Each distinct binding state must produce unique text so the
				// controller detects every transition.

				// Setting a binding must produce different text from unbound
				ASSERT_THAT(IsTrue(UnboundText != BoundText));

				// Clearing a binding must produce different text from bound state
				ASSERT_THAT(IsTrue(ClearedText != BoundText));

				// Both unbound and cleared should contain __Binding=() -- the
				// always-emit empty marker that ensures change detection works
				ASSERT_THAT(IsTrue(UnboundText.Contains(TEXT("__Binding=()"))));
				ASSERT_THAT(IsTrue(ClearedText.Contains(TEXT("__Binding=()"))));

				// The bound text must contain binding data, not just the empty marker
				ASSERT_THAT(IsTrue(BoundText.Contains(TEXT("__Binding="))));
				ASSERT_THAT(IsFalse(BoundText.Contains(TEXT("__Binding=()"))));
			});
	}

	// Verifies that FBindableBool on a trait pin round-trips constant value
	// changes through the RigVM controller's SetPinDefaultValue /
	// GetDefaultValue. With GENERATE_TRAIT_LATENT_PROPERTIES, the trait system
	// decomposes FBindableBool into a ConstantValue sub-pin for the literal
	// bool value.
	TEST_METHOD(FBindableBool_RoundTripThroughController)
	{
		TestCommandBuilder
			.Do(TEXT("Create AnimNext Animation Graph Asset"), [&]()
			{
				CreateAnimNextAnimationGraph();
			})
			.Then(TEXT("Create a Trait Stack node"), [&]()
			{
				CreateTraitStackNode();
			})
			.Then(TEXT("Add test trait with FBindableBool"), [&]()
			{
				const FString SharedDataPath = FUAFTestBindableTraitSharedData::StaticStruct()->GetPathName();
				const FString Properties = FString::Printf(
					TEXT("(DecoratorSharedDataStruct=\"/Script/CoreUObject.ScriptStruct'%s'\",Name=\"%s\")"),
					*SharedDataPath, *TestTraitName);

				const FName TraitName = Controller->AddTrait(
					FName(*TraitStackNodeName),
					FName(TEXT("/Script/AnimNextAnimGraph.RigDecorator_AnimNextCppDecorator")),
					FName(*TestTraitName),
					Properties);
				ASSERT_THAT(IsTrue(TraitName.ToString().Equals(TestTraitName)));
			})
			.Then(TEXT("Verify bTestBool pin has ConstantValue sub-pin with expected default"), [&]()
			{
				URigVMNode* StackNode = ControllerGraph->FindNodeByName(FName(*TraitStackNodeName));
				ASSERT_THAT(IsNotNull(StackNode));

				const FString BoolPinPath = FString::Printf(TEXT("%s.%s"), *TestTraitName, *BoolPinName);
				URigVMPin* BoolPin = StackNode->FindPin(BoolPinPath);
				ASSERT_THAT(IsNotNull(BoolPin));

				// Latent property decomposition creates sub-pins
				ASSERT_THAT(IsTrue(BoolPin->GetSubPins().Num() > 0));

				const FString ConstantValuePath = FString::Printf(TEXT("%s.%s.%s"), *TestTraitName, *BoolPinName, *ConstantValueName);
				URigVMPin* ConstantValuePin = StackNode->FindPin(ConstantValuePath);
				ASSERT_THAT(IsNotNull(ConstantValuePin));

				ASSERT_THAT(IsTrue(ConstantValuePin->GetDefaultValue() == TEXT("False")));
			})
			.Then(TEXT("Set ConstantValue to True via controller and verify round-trip"), [&]()
			{
				URigVMNode* StackNode = ControllerGraph->FindNodeByName(FName(*TraitStackNodeName));
				const FString ConstantValuePath = FString::Printf(TEXT("%s.%s.%s"), *TestTraitName, *BoolPinName, *ConstantValueName);
				URigVMPin* ConstantValuePin = StackNode->FindPin(ConstantValuePath);
				ASSERT_THAT(IsNotNull(ConstantValuePin));

				const FString ExpectedText = TEXT("True");

				ASSERT_THAT(IsTrue(Controller->SetPinDefaultValue(ConstantValuePin->GetPinPath(), ExpectedText)));
				ASSERT_THAT(IsTrue(ConstantValuePin->GetDefaultValue() == ExpectedText));
			})
			.Then(TEXT("Set ConstantValue back to False via controller and verify round-trip"), [&]()
			{
				URigVMNode* StackNode = ControllerGraph->FindNodeByName(FName(*TraitStackNodeName));
				const FString ConstantValuePath = FString::Printf(TEXT("%s.%s.%s"), *TestTraitName, *BoolPinName, *ConstantValueName);
				URigVMPin* ConstantValuePin = StackNode->FindPin(ConstantValuePath);
				ASSERT_THAT(IsNotNull(ConstantValuePin));

				const FString ExpectedText = TEXT("False");

				ASSERT_THAT(IsTrue(Controller->SetPinDefaultValue(ConstantValuePin->GetPinPath(), ExpectedText)));
				ASSERT_THAT(IsTrue(ConstantValuePin->GetDefaultValue() == ExpectedText));
			})
			.Then(TEXT("Verify parent pin text reflects sub-pin changes"), [&]()
			{
				URigVMNode* StackNode = ControllerGraph->FindNodeByName(FName(*TraitStackNodeName));

				// Set ConstantValue to True so the parent pin text changes
				const FString ConstantValuePath = FString::Printf(TEXT("%s.%s.%s"), *TestTraitName, *BoolPinName, *ConstantValueName);
				URigVMPin* ConstantValuePin = StackNode->FindPin(ConstantValuePath);
				ASSERT_THAT(IsNotNull(ConstantValuePin));
				Controller->SetPinDefaultValue(ConstantValuePin->GetPinPath(), TEXT("True"));

				const FString BoolPinPath = FString::Printf(TEXT("%s.%s"), *TestTraitName, *BoolPinName);
				URigVMPin* BoolPin = StackNode->FindPin(BoolPinPath);
				ASSERT_THAT(IsNotNull(BoolPin));

				const FString ParentValue = BoolPin->GetDefaultValue();
				UE_LOG(LogTemp, Display, TEXT("DIAG: ParentValue = '%s'"), *ParentValue);
				UE_LOG(LogTemp, Display, TEXT("DIAG: ConstantValue after set = '%s'"), *ConstantValuePin->GetDefaultValue());
				ASSERT_THAT(IsTrue(ParentValue.Contains(TEXT("True"))));
			});
	}

	// Verifies that FBindableBool binding state round-trips through text
	// export/import. Exercises SetBinding/ClearBinding with UScriptStruct
	// ExportText/ImportText -- the same serialization path the RigVM uses
	// for pin default values and the editor customization triggers via
	// NotifyPostChange.
	//
	// This test proves that:
	//  - Binding data survives ExportText -> ImportText round-trip
	//  - HasBinding() and GetBinding() return the correct state after import
	//  - Each distinct binding state produces unique export text
	TEST_METHOD(FBindableBool_BindingTextSerializationRoundTrip)
	{
		TestCommandBuilder
			.Do(TEXT("Export, import, and verify binding round-trip for each state"), [&]()
			{
				const UScriptStruct* BoolStruct = FBindableBool::StaticStruct();
				ASSERT_THAT(IsNotNull(BoolStruct));

				FUAFPropertyBinding TestBinding;
				TestBinding.SourceType = EUAFBindingSourceType::Variable;

				// --- State 1: bound (ConstantValue=true, binding set) ---
				FBindableBool Bound(true);
				Bound.SetBinding(TestBinding);

				FString BoundText;
				BoolStruct->ExportText(BoundText, &Bound, nullptr, nullptr, 0, nullptr, true);
				ASSERT_THAT(IsTrue(BoundText.Contains(TEXT("__Binding="))));
				ASSERT_THAT(IsFalse(BoundText.Contains(TEXT("__Binding=()"))));

				// Import bound text into a fresh instance via UScriptStruct::ImportText
				// (dispatches to native ImportTextItem for STRUCT_ImportTextItemNative)
				FBindableBool ImportedBound;
				ASSERT_THAT(IsNotNull(BoolStruct->ImportText(*BoundText, &ImportedBound, nullptr, 0, nullptr, BoolStruct->GetName())));
				ASSERT_THAT(IsTrue(ImportedBound.HasBinding()));
				ASSERT_THAT(IsTrue(ImportedBound.GetConstantValue() == true));

				// Verify the imported binding matches the original
				const FUAFPropertyBinding* ImportedBinding = ImportedBound.GetBinding();
				ASSERT_THAT(IsNotNull(ImportedBinding));
				ASSERT_THAT(IsTrue(ImportedBinding->SourceType == EUAFBindingSourceType::Variable));

				// --- State 2: cleared (ConstantValue=true, binding cleared) ---
				FBindableBool Cleared(true);
				Cleared.SetBinding(TestBinding);
				Cleared.ClearBinding();

				FString ClearedText;
				BoolStruct->ExportText(ClearedText, &Cleared, nullptr, nullptr, 0, nullptr, true);
				ASSERT_THAT(IsTrue(ClearedText.Contains(TEXT("__Binding=()"))));

				// Import cleared text
				FBindableBool ImportedCleared;
				ASSERT_THAT(IsNotNull(BoolStruct->ImportText(*ClearedText, &ImportedCleared, nullptr, 0, nullptr, BoolStruct->GetName())));
				ASSERT_THAT(IsFalse(ImportedCleared.HasBinding()));
				ASSERT_THAT(IsTrue(ImportedCleared.GetConstantValue() == true));

				// --- State 3: unbound (ConstantValue=true, never had binding) ---
				FBindableBool Unbound(true);
				FString UnboundText;
				BoolStruct->ExportText(UnboundText, &Unbound, nullptr, nullptr, 0, nullptr, true);

				// Each binding state must produce unique export text so that
				// the RigVM controller's text comparison detects every transition
				ASSERT_THAT(IsTrue(BoundText != UnboundText));
				ASSERT_THAT(IsTrue(BoundText != ClearedText));
			});
	}

protected:
	void CreateAnimNextAnimationGraph()
	{
		const TSubclassOf<UUAFAnimGraphFactory> AnimGraphFactoryClass = UUAFAnimGraphFactory::StaticClass();
		UObject* FactoryObject = UAFTestsUtilities::CreateFactoryObject(NewObject<UUAFAnimGraphFactory>(GetTransientPackage(), AnimGraphFactoryClass.Get()), UUAFAnimGraph::StaticClass(), "TestBindableTraitPinGraph");
		AnimNextAnimationGraph = CastChecked<UUAFAnimGraph>(FactoryObject);
		ASSERT_THAT(IsNotNull(AnimNextAnimationGraph));
		ASSERT_THAT(IsTrue(AssetEditorSubsystem->OpenEditorForAsset(AnimNextAnimationGraph)));
	}

	void CreateTraitStackNode()
	{
		Controller = UUAFTestBlueprintLibrary::GetControllerByName(AnimNextAnimationGraph, TEXT("RigVMGraph"));
		ASSERT_THAT(IsNotNull(Controller));

		ControllerGraph = Controller->GetGraph();
		ASSERT_THAT(IsNotNull(ControllerGraph));

		TestRunner->AddExpectedError(TEXT("Base Trait Data is Invalid. Please, select a new Base Trait."), EAutomationExpectedErrorFlags::Contains, -1);

		URigVMUnitNode* TraitStackNode = Controller->AddUnitNodeFromStructPath(
			FRigUnit_AnimNextTraitStack::StaticStruct()->GetPathName(),
			TEXT("Execute"),
			FVector2D::ZeroVector,
			TEXT("AnimNextTraitStack"));
		ASSERT_THAT(IsNotNull(TraitStackNode));
		ASSERT_THAT(IsNotNull(ControllerGraph->FindNodeByName(FName(*TraitStackNodeName))));
	}
};

#endif // WITH_EDITOR
#endif // WITH_DEV_AUTOMATION_TESTS
