// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS
#if WITH_EDITOR

#include "UAFTestsUtilities.h"
#include "CQTest.h"
#include "Editor.h"
#include "ScopedTransaction.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextAnimationGraphFactory.h"
#include "Graph/RigUnit_AnimNextTraitStack.h"
#include "RigVMFunctions/Simulation/RigVMFunction_Accumulate.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "Math/Vector2D.h"

TEST_CLASS(FAnimNextAnimationGraphTests, "Animation.UAF.Functional")
{
	UUAFAnimGraph* AnimNextAnimationGraph = nullptr;
	UAssetEditorSubsystem* AssetEditorSubsystem = nullptr;
	UUAFRigVMAssetEditorData* EditorData = nullptr;
	URigVMGraph* ControllerGraph = nullptr;
	UEdGraph* ParentGraph = nullptr;
	const FString TraitStackNodeName = "AnimNextTraitStack";
	const FString BlendByBoolTraitName = "FBlendByBoolTrait";
	const FString AccumulateTransformLerpNodeName = "RigVMFunction_AccumulateTransformLerp";
	const FString VariableName = "NewVariable";
	const FString GetVariableNodeName = "VariableNode";
	const FString SetVariableNodeName = "VariableNode_1";

	BEFORE_EACH()
	{
		AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		ASSERT_THAT(IsNotNull(AssetEditorSubsystem));
	}

	// QMetry Test Case:  UE-TC-10707
	TEST_METHOD(Create_Trait_Stack_Node_In_AnimNext_AnimationGraph)
	{
		TestCommandBuilder
			.Do(TEXT("Create AnimNext Animation Graph Asset"), [&]()
			{
				CreateAnimNextAnimationGraph();
			})
			.Then(TEXT("Create a Trait Stack node"), [&]()
			{
				CreateTraitStackNode();
			});
	}
	
	// QMetry Test Case:  UE-TC-10708
	TEST_METHOD(Add_Blend_By_Bool_To_Trait_Stack_Node_In_AnimNext_AnimationGraph)
	{
		TestCommandBuilder
			.Do(TEXT("Create AnimNext Animation Graph Asset"), [&]()
			{
				CreateAnimNextAnimationGraph();
			})
			.Then(TEXT("Create a Trait Stack node"), [&]()
			{
				CreateTraitStackNode();
			})
			.Then(TEXT("Add Blend by Bool to the Trait Stack node"), [&]()
			{
				URigVMController* Controller = EditorData->GetControllerByName("RigVMGraph");
				FName TraitName = Controller->AddTrait(FName(TraitStackNodeName), FName("/Script/AnimNextAnimGraph.RigDecorator_AnimNextCppDecorator"), FName(BlendByBoolTraitName), "(DecoratorSharedDataStruct=\"/Script/CoreUObject.ScriptStruct'/Script/AnimNextAnimGraph.AnimNextBlendByBoolTraitSharedData'\",Name=\"FBlendByBoolTrait\",TrueChild=(PackedTraitIndexAndNodeHandle=16777215),FalseChild=(PackedTraitIndexAndNodeHandle=16777215),bCondition=False,bAlwaysUpdateTrueChild=False)");
				ASSERT_THAT(IsTrue(TraitName.ToString().Equals(BlendByBoolTraitName)));
			});
	}
	
	// QMetry Test Case:  UE-TC-18823
	TEST_METHOD(Undo_Redo_In_AnimNext_AnimationGraph)
	{
		TestCommandBuilder
			.Do(TEXT("Create AnimNext Animation Graph Asset"), [&]()
			{
				CreateAnimNextAnimationGraph();
			})
			.Then(TEXT("Create a Trait Stack node"), [&]()
			{
				CreateTraitStackNode();
			})
			.Then(TEXT("Create an AccumulateLerp Transform node"), [&]()
			{
				TArray<UEdGraphPin*> FromPins;
				UEdGraphNode* AccumulateLerpTransformNode = UAFTestsUtilities::AddUnitNode(ParentGraph, FRigVMFunction_AccumulateTransformLerp::StaticStruct()->GetPathName(), FromPins, FVector2f(-525.0f, 125.0f));
				ASSERT_THAT(IsNotNull(AccumulateLerpTransformNode));
			})
			.Then(TEXT("Create a bool Variable"), [&]()
			{
				FScopedTransaction Transaction(NSLOCTEXT("Add_Variables", "AddVariables", "Add Variable(s)"));
				
				UAnimNextVariableEntry* VariableEntry = UAFTestsUtilities::AddVariable(AnimNextAnimationGraph, FAnimNextParamType::GetType<bool>(), VariableName, "false");
				ASSERT_THAT(IsNotNull(VariableEntry));
			})
			.Then(TEXT("Add Get Variable Node to Graph"), [&]()
			{
				TArray<UEdGraphPin*> FromPins;
				UEdGraphNode* GetVariableNode = UAFTestsUtilities::AddVariableNode(ParentGraph, AnimNextAnimationGraph, VariableName, FAnimNextParamType::GetType<bool>(), FAnimNextSchemaAction_Variable::EVariableAccessorChoice::Get, FromPins, FVector2f(-250.0f, 135.0f));
				ASSERT_THAT(IsNotNull(GetVariableNode));
			})
			.Then(TEXT("Add Set Variable Node to Graph"), [&]()
			{				
				TArray<UEdGraphPin*> FromPins;
				UEdGraphNode* SetVariableNode = UAFTestsUtilities::AddVariableNode(ParentGraph, AnimNextAnimationGraph, VariableName, FAnimNextParamType::GetType<bool>(), FAnimNextSchemaAction_Variable::EVariableAccessorChoice::Set, FromPins, FVector2f(-250.0f, 235.0f));
				ASSERT_THAT(IsNotNull(SetVariableNode));
			})
			.Until(TEXT("Execute Undo Set Variable Node"), [&]()
			{
				return GEditor->UndoTransaction();
			})
			.Until(TEXT("Execute Undo Get Variable Node"), [&]()
			{
				return GEditor->UndoTransaction();
			})
			.Until(TEXT("Execute Undo Create bool Variable"), [&]()
			{
				return GEditor->UndoTransaction();
			})
			.Until(TEXT("Execute Undo Create AccumulateLerp Transform Node"), [&]()
			{
				return GEditor->UndoTransaction();
			})
			.Until(TEXT("Execute Undo Create Trait Stack Node"), [&]()
			{
				return GEditor->UndoTransaction();
			})
			.Then(TEXT("Assert Undos"), [&]()
			{
				ASSERT_THAT(IsNull(ControllerGraph->FindNodeByName(FName(SetVariableNodeName))));
				ASSERT_THAT(IsNull(ControllerGraph->FindNodeByName(FName(GetVariableNodeName))));
				ASSERT_THAT(IsNull(ControllerGraph->FindNodeByName(FName(AccumulateTransformLerpNodeName))));
				ASSERT_THAT(IsNull(ControllerGraph->FindNodeByName(FName(TraitStackNodeName))));
				ASSERT_THAT(IsNull(EditorData->FindEntry(FName(VariableName))));
			})
			.Until(TEXT("Execute Redo Create Trait Stack Node"), [&]()
			{
				return GEditor->RedoTransaction();
			})
			.Until(TEXT("Execute Redo Create AccumulateLerp Transform Node"), [&]()
			{
				return GEditor->RedoTransaction();
			})
			.Until(TEXT("Execute Redo Create bool Variable"), [&]()
			{
				return GEditor->RedoTransaction();
			})
			.Until(TEXT("Execute Redo Create Get Variable Node"), [&]()
			{
				return GEditor->RedoTransaction();
			})
			.Until(TEXT("Execute Redo Create Set Variable Node"), [&]()
			{
				return GEditor->RedoTransaction();
			})
			.Then(TEXT("Assert Redos"), [&]()
			{
				ASSERT_THAT(IsNotNull(ControllerGraph->FindNodeByName(FName(SetVariableNodeName))));
				ASSERT_THAT(IsNotNull(ControllerGraph->FindNodeByName(FName(GetVariableNodeName))));
				ASSERT_THAT(IsNotNull(ControllerGraph->FindNodeByName(FName(AccumulateTransformLerpNodeName))));
				ASSERT_THAT(IsNotNull(ControllerGraph->FindNodeByName(FName(TraitStackNodeName))));
				ASSERT_THAT(IsNotNull(EditorData->FindEntry(FName(VariableName))));
			});
	}
	
protected:
	void CreateAnimNextAnimationGraph()
	{
		const TSubclassOf<UUAFAnimGraphFactory> AnimGraphFactoryClass = UUAFAnimGraphFactory::StaticClass();
		UObject* FactoryObject = UAFTestsUtilities::CreateFactoryObject(NewObject<UUAFAnimGraphFactory>(GetTransientPackage(), AnimGraphFactoryClass.Get()), UUAFAnimGraph::StaticClass(), "NewAnimNextAnimationGraph");
		AnimNextAnimationGraph = CastChecked<UUAFAnimGraph>(FactoryObject);
		ASSERT_THAT(IsNotNull(AnimNextAnimationGraph));
		ASSERT_THAT(IsTrue(AssetEditorSubsystem->OpenEditorForAsset(AnimNextAnimationGraph)));
	}
	
	void CreateTraitStackNode()
	{
		EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(AnimNextAnimationGraph);
		ASSERT_THAT(IsNotNull(EditorData));
		ControllerGraph = EditorData->GetControllerByName("RigVMGraph")->GetGraph();
		ASSERT_THAT(IsNotNull(ControllerGraph));				
		ParentGraph = Cast<UEdGraph>(EditorData->GetEditorObjectForRigVMGraph(ControllerGraph));
		ASSERT_THAT(IsNotNull(ParentGraph));
		
		TestRunner->AddExpectedError("Base Trait Data is Invalid. Please, select a new Base Trait.", EAutomationExpectedErrorFlags::Contains, -1);
		
		TArray<UEdGraphPin*> FromPins;
		UEdGraphNode* TraitStackNode = UAFTestsUtilities::AddUnitNode(ParentGraph, FRigUnit_AnimNextTraitStack::StaticStruct()->GetPathName(), FromPins, FVector2f(0.0f, 0.0f), "/Script/UAFAnimGraphUncookedOnly.AnimNextTraitStackUnitNode");
		ASSERT_THAT(IsNotNull(TraitStackNode));
		ASSERT_THAT(IsNotNull(ControllerGraph->FindNodeByName(FName(TraitStackNodeName))));	
	}
};

#endif // WITH_EDITOR
#endif // WITH_DEV_AUTOMATION_TESTS
