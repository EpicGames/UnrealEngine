// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS
#if WITH_EDITOR

#include "UAFTestsUtilities.h"
#include "CQTest.h"
#include "Editor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "AnimNextStateTree.h"
#include "AnimNextStateTreeFactory.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextAnimationGraphFactory.h"
#include "Module/AnimNextModule.h"
#include "Module/AnimNextModuleFactory.h"
#include "Variables/AnimNextSharedVariables.h"
#include "Variables/AnimNextSharedVariablesFactory.h"

// QMetry Test Case:  UE-TC-10715 
TEST_CLASS(FAssetCreationTests, "Animation.UAF.Functional")
{
	UObject* AnimNextAsset;
	UAssetEditorSubsystem* AssetEditorSubsystem;

	BEFORE_EACH()
	{
		AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		ASSERT_THAT(IsNotNull(AssetEditorSubsystem));
	}
	
	TEST_METHOD(Verify_StateTree_Asset_Creation)
	{
		TestCommandBuilder
			.Do(TEXT("Create AnimNext State Tree Asset"), [&]()
			{
				const TSubclassOf<UAnimNextStateTreeFactory> StateTreeFactoryClass = UAnimNextStateTreeFactory::StaticClass();
				UObject* FactoryObject = UAFTestsUtilities::CreateFactoryObject(NewObject<UAnimNextStateTreeFactory>(), UAnimNextStateTree::StaticClass(), "NewAnimNextStateTree");
				AnimNextAsset = CastChecked<UAnimNextStateTree>(FactoryObject);
			})
			.Then(TEXT("Assert AnimNext State Tree Asset Created and Open in Editor"), [&]()
			{
				ASSERT_THAT(IsNotNull(AnimNextAsset));
				ASSERT_THAT(IsTrue(AssetEditorSubsystem->OpenEditorForAsset(AnimNextAsset)));
			});
	}
	
	TEST_METHOD(Verify_AnimGraph_Asset_Creation)
	{
		TestCommandBuilder
			.Do(TEXT("Create AnimNext Animation Graph Asset"), [&]()
			{
				const TSubclassOf<UUAFAnimGraphFactory> AnimGraphFactoryClass = UUAFAnimGraphFactory::StaticClass();
				UObject* FactoryObject = UAFTestsUtilities::CreateFactoryObject(NewObject<UUAFAnimGraphFactory>(), UUAFAnimGraph::StaticClass(), "NewAnimNextAnimationGraph");
				AnimNextAsset = CastChecked<UUAFAnimGraph>(FactoryObject);
			})
			.Then(TEXT("Assert AnimNext Animation Graph Asset Created and Open in Editor"), [&]()
			{
				ASSERT_THAT(IsNotNull(AnimNextAsset));
				ASSERT_THAT(IsTrue(AssetEditorSubsystem->OpenEditorForAsset(AnimNextAsset)));
			});		
	}
	
	TEST_METHOD(Verify_Module_Asset_Creation)
	{
		TestCommandBuilder
			.Do(TEXT("Create AnimNext Module Asset"), [&]()
			{
				const TSubclassOf<UUAFSystemFactory> ModuleFactoryClass = UUAFSystemFactory::StaticClass();
				UObject* FactoryObject = UAFTestsUtilities::CreateFactoryObject(NewObject<UUAFSystemFactory>(), UUAFSystem::StaticClass(), "NewAnimNextModule");				
				AnimNextAsset = CastChecked<UUAFSystem>(FactoryObject);
			})
			.Then(TEXT("Assert AnimNext Module Asset Created and Open in Editor"), [&]()
			{
				ASSERT_THAT(IsNotNull(AnimNextAsset));
				ASSERT_THAT(IsTrue(AssetEditorSubsystem->OpenEditorForAsset(AnimNextAsset)));
			});
	}
	
	TEST_METHOD(Verify_SharedVariables_Asset_Creation)
	{
		TestCommandBuilder
			.Do(TEXT("Create AnimNext Shared Variables Asset"), [&]()
			{
				const TSubclassOf<UUAFSharedVariablesFactory> SharedVariablesFactoryClass = UUAFSharedVariablesFactory::StaticClass();
				UObject* FactoryObject = UAFTestsUtilities::CreateFactoryObject(NewObject<UUAFSharedVariablesFactory>(), UUAFSharedVariables::StaticClass(), "NewAnimNextSharedVariables");
				AnimNextAsset = CastChecked<UUAFSharedVariables>(FactoryObject);
			})
			.Then(TEXT("Assert AnimNext Shared Variables Asset Created and Open in Editor"), [&]()
			{
				ASSERT_THAT(IsNotNull(AnimNextAsset));
				ASSERT_THAT(IsTrue(AssetEditorSubsystem->OpenEditorForAsset(AnimNextAsset)));		
			});
	}	
};

#endif // WITH_EDITOR
#endif // WITH_DEV_AUTOMATION_TESTS
