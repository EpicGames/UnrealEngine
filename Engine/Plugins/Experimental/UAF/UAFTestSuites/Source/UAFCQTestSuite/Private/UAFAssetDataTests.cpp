// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFAssetDataTests.h"

#include "Injection/InjectionSiteTrait.h"
#include "Traits/BlendSpacePlayerTraitData.h"

#if WITH_DEV_AUTOMATION_TESTS
#if WITH_EDITOR

#include "CoreMinimal.h"
#include "UAFTestsUtilities.h"
#include "CQTest.h"
#include "Asset/UAFAnimGraphAssetData.h"
#include "Engine/SkeletalMesh.h"
#include "Factory/AnimGraphFactory.h"
#include "Traits/SequencePlayerTraitData.h"
#include "UAF/UAFAssetData.h"
#include "UAF/UAFAssetFactory.h"

#define LOCTEXT_NAMESPACE "UAFAssetDataTests"

TEST_CLASS(FUAFAssetDataTests, "Animation.UAF.Functional")
{
	static TObjectPtr<USkeletalMesh> GSkeletalMesh;
	static TObjectPtr<USkeleton> GSkeleton;
	static TObjectPtr<USkinnedAsset> GSkinnedAsset;
	
	BEFORE_ALL() 
	{
		GSkeleton = NewObject<USkeleton>();
		{
			FReferenceSkeletonModifier SkeletonModifier(GSkeleton);

			// Add a root bone
			SkeletonModifier.Add(FMeshBoneInfo(NAME_Root, FString(TEXT("Root")), -1), FTransform::Identity);	
		}

		GSkeletalMesh = NewObject<USkeletalMesh>();
		GSkeletalMesh->SetSkeleton(GSkeleton);
	}

	AFTER_ALL()
	{
		GSkeleton = nullptr;
		GSkeletalMesh = nullptr;
	}

	const float TestRateScale = 1.5f;
	const int TestNumFrames = 120;
	const FFrameRate TestFps = FFrameRate(60, 1);
	const float TestAnimLength = TestNumFrames / TestFps.AsDecimal();

	TObjectPtr<UAnimSequence> CreateAnimSequence()
	{
		Sequence = NewObject<UAnimSequence>();
		Sequence->SetSkeleton(GSkeleton);

		TScriptInterface<IAnimationDataController> Controller = Sequence->GetDataModel()->GetController();
		Controller->OpenBracket(LOCTEXT("CreateAnimSequence", "Creating Animation Sequence"), false);
		Controller->InitializeModel();
		Controller->SetFrameRate(TestFps, false);
		Controller->SetNumberOfFrames(TestNumFrames, false);
		Controller->CloseBracket(false);
		Sequence->WaitOnExistingCompression();

		return Sequence;
	}
	
	TObjectPtr<UUAFAssetDataTestObject> TestObject;
	FTopLevelAssetPath UAFAssetDataTestObjectClassPath;
	TEST_METHOD(Verify_Asset_Factory)
	{
		TestCommandBuilder
		.Do(TEXT("Register Test Asset Data"), [&]()
			{
				ASSERT_THAT(IsFalse(UE::UAF::FAssetDataFactory::GetRegisteredObjectClasses().Contains(UUAFAssetDataTestObject::StaticClass())))
				ASSERT_THAT(IsFalse(UE::UAF::FAssetDataFactory::GetRegisteredObjectClasses<FUAFGraphFactoryAsset_Test>().Contains(UUAFAssetDataTestObject::StaticClass())))
				ASSERT_THAT(IsFalse(UE::UAF::FAssetDataFactory::GetRegisteredObjectClasses<FUAFGraphFactoryAsset>().Contains(UUAFAssetDataTestObject::StaticClass())))
				
				UAFAssetDataTestObjectClassPath = UE::UAF::FAssetDataFactory::RegisterAsset<FUAFGraphFactoryAsset_Test, UUAFAssetDataTestObject>(
		[](const UUAFAssetDataTestObject* TestObject)
				{
						FUAFGraphFactoryAsset_Test TestAsset;
						TestAsset.TestData = TestObject->TestData;
						return TestAsset;
				});

				ASSERT_THAT(IsTrue(UE::UAF::FAssetDataFactory::GetRegisteredObjectClasses().Contains(UUAFAssetDataTestObject::StaticClass())))
				ASSERT_THAT(IsTrue(UE::UAF::FAssetDataFactory::GetRegisteredObjectClasses<FUAFGraphFactoryAsset_Test>().Contains(UUAFAssetDataTestObject::StaticClass())))
				ASSERT_THAT(IsTrue(UE::UAF::FAssetDataFactory::GetRegisteredObjectClasses<FUAFGraphFactoryAsset>().Contains(UUAFAssetDataTestObject::StaticClass())))
			})
		.Then( TEXT("Create Asset Data From Factory"), [&]()
			{
				int TestNumber = 1234;
				TestObject = NewObject<UUAFAssetDataTestObject>();
				TestObject->TestData = TestNumber;

				// Try to create a test asset from the object
				UE::UAF::FAssetHandle TestInstanceStruct = UE::UAF::FAssetDataFactory::CreateUAFAssetDataFromObject(TestObject);

				ASSERT_THAT(IsTrue(TestInstanceStruct.IsValid()));
				ASSERT_THAT(IsNotNull(TestInstanceStruct.GetPtr<FUAFGraphFactoryAsset_Test>()));
				ASSERT_THAT(AreEqual(TestNumber, TestInstanceStruct.Get<FUAFGraphFactoryAsset_Test>().TestData));

				// Verify that the templated version works for the same type
				TInstancedStruct<FUAFGraphFactoryAsset_Test> TestInstanceStruct2 = UE::UAF::FAssetDataFactory::CreateUAFAssetDataFromObject<FUAFGraphFactoryAsset_Test>(TestObject);
				ASSERT_THAT(IsTrue(TestInstanceStruct2.IsValid()));
				ASSERT_THAT(AreEqual(TestNumber, TestInstanceStruct2.Get().TestData));

				// Verify that the templated version works for base type
				TInstancedStruct<FUAFGraphFactoryAsset> GraphFactoryAsset = UE::UAF::FAssetDataFactory::CreateUAFAssetDataFromObject<FUAFGraphFactoryAsset>(TestObject);
				ASSERT_THAT(IsTrue(GraphFactoryAsset.IsValid()));
				ASSERT_THAT(AreEqual(TestNumber, GraphFactoryAsset.Get<FUAFGraphFactoryAsset_Test>().TestData));

				// Verify that if we ask for an incompatible derived data type, it wont produce a asset data
				TInstancedStruct<FUAFGraphFactoryAsset_Animation> TestInstanceStruct3 = UE::UAF::FAssetDataFactory::CreateUAFAssetDataFromObject<FUAFGraphFactoryAsset_Animation>(TestObject);
				ASSERT_THAT(IsFalse(TestInstanceStruct3.IsValid()));
			})
		.OnTearDown(TEXT("Unregister Test Asset Data"), [&]()
		{
			UE::UAF::FAssetDataFactory::UnregisterAsset(UAFAssetDataTestObjectClassPath);
			TestObject = nullptr;
		});
	}
	
	TObjectPtr<UAnimSequence> Sequence;
	TInstancedStruct<FUAFGraphFactoryAsset_Animation> AnimAsset;

	UE::UAF::EAnimAssetLoopMode TestLoopMode = UE::UAF::EAnimAssetLoopMode::ForceLoop;
	TEST_METHOD(Verify_Core_Asset_Types_Anim_Sequence)
	{
	
		TestCommandBuilder.Do(TEXT("Setup"), [&]()
			{
				Sequence = CreateAnimSequence();
				ASSERT_THAT(IsNotNull(Sequence));

				ASSERT_THAT(IsNear(TestAnimLength, Sequence->GetPlayLength(), UE_SMALL_NUMBER));
			})
		.Then(TEXT("Generate Asset Data"), [&]()
			{
				AnimAsset = UE::UAF::FAssetDataFactory::CreateUAFAssetDataFromObject<FUAFGraphFactoryAsset_Animation>(Sequence);
				ASSERT_THAT(IsTrue(AnimAsset.IsValid()));
				ASSERT_THAT(AreEqual(Sequence, AnimAsset.Get().AnimationSequence));
				AnimAsset.GetMutable().PlayRate = TestRateScale;
				AnimAsset.GetMutable().LoopMode = TestLoopMode;
				
			})
		.Then(TEXT("Generate Factory Params From Asset"), [&]()
			{
				FAnimNextFactoryParams Params = UE::UAF::FAnimGraphFactory::GetDefaultParamsForAsset(AnimAsset);
				FAnimNextSimpleAnimGraphBuilder& GraphBuilder = ((FAnimNextSimpleAnimGraphBuilder&)Params.GetBuilder());
				ASSERT_THAT(IsTrue(GraphBuilder.Stacks.Num() > 0));
				ASSERT_THAT(IsTrue(GraphBuilder.Stacks[0].TraitDescs.Num() > 0));
				const FAnimNextSimpleAnimGraphBuilderTraitDesc* SequencePlayerTraitDesc = GraphBuilder.Stacks[0].TraitDescs.FindByPredicate(
					[](const FAnimNextSimpleAnimGraphBuilderTraitDesc& Item)
					{
						return Item.TraitData.GetPtr<UE::UAF::FSequencePlayerData>() != nullptr;
					});
				ASSERT_THAT(IsNotNull(SequencePlayerTraitDesc));
				ASSERT_THAT(IsNotNull(SequencePlayerTraitDesc->TraitData.GetMemory()));
				
				const UE::UAF::FSequencePlayerData* TypedSequencePlayerData = SequencePlayerTraitDesc->TraitData.GetPtr<UE::UAF::FSequencePlayerData>();
				ASSERT_THAT(IsNotNull(TypedSequencePlayerData));
				ASSERT_THAT(IsNear(TestRateScale, TypedSequencePlayerData->PlayRate, UE_SMALL_NUMBER));
				ASSERT_THAT(AreEqual(TestLoopMode, TypedSequencePlayerData->LoopMode));
				
			})
		.OnTearDown(TEXT("Tear Down"), [&]()
		{
			AnimAsset.Reset();
			Sequence = nullptr;
		});
	}

	float TestStartTimeRatio = 0.75f;
	TEST_METHOD(Verify_Core_Asset_Types_Anim_Sequence_StartTime_Ratio)
	{
		TestCommandBuilder.Do(TEXT("Setup"), [&]()
			{
				Sequence = CreateAnimSequence();
				ASSERT_THAT(IsNotNull(Sequence));

				ASSERT_THAT(IsNear(TestAnimLength, Sequence->GetPlayLength(), UE_SMALL_NUMBER));
			})
		.Then(TEXT("Generate Asset Data"), [&]()
			{
				AnimAsset = UE::UAF::FAssetDataFactory::CreateUAFAssetDataFromObject<FUAFGraphFactoryAsset_Animation>(Sequence);
				ASSERT_THAT(IsTrue(AnimAsset.IsValid()));
				ASSERT_THAT(AreEqual(Sequence, AnimAsset.Get().AnimationSequence));
				AnimAsset.GetMutable().StartTimeType = EAnimationStartTimeType::TimePercent;
				AnimAsset.GetMutable().StartTimePercent = TestStartTimeRatio * 100.0f;
				
			})
		.Then(TEXT("Generate Factory Params From Asset"), [&]()
			{
				FAnimNextFactoryParams Params = UE::UAF::FAnimGraphFactory::GetDefaultParamsForAsset(AnimAsset);
				FAnimNextSimpleAnimGraphBuilder& GraphBuilder = ((FAnimNextSimpleAnimGraphBuilder&)Params.GetBuilder());
				ASSERT_THAT(IsTrue(GraphBuilder.Stacks.Num() > 0));
				ASSERT_THAT(IsTrue(GraphBuilder.Stacks[0].TraitDescs.Num() > 0));
				const FAnimNextSimpleAnimGraphBuilderTraitDesc* SequencePlayerTraitData = GraphBuilder.Stacks[0].TraitDescs.FindByPredicate(
					[](const FAnimNextSimpleAnimGraphBuilderTraitDesc& Item)
					{
						return Item.TraitData.GetPtr<UE::UAF::FSequencePlayerData>() != nullptr;
					});
				ASSERT_THAT(IsNotNull(SequencePlayerTraitData));

				const UE::UAF::FSequencePlayerData* TypedSequencePlayerData = SequencePlayerTraitData->TraitData.GetPtr<UE::UAF::FSequencePlayerData>();
				ASSERT_THAT(IsNotNull(TypedSequencePlayerData));
				ASSERT_THAT(IsNear(TestStartTimeRatio * TestAnimLength, TypedSequencePlayerData->StartPosition, UE_SMALL_NUMBER));
				
			})
		.OnTearDown(TEXT("Tear Down"), [&]()
		{
			AnimAsset.Reset();
			Sequence = nullptr;
		});
	}

	TEST_METHOD(Verify_Core_Asset_Types_Anim_Sequence_StartTime_Clamp_Time)
	{
	
		TestCommandBuilder.Do(TEXT("Setup"), [&]()
			{
				Sequence = CreateAnimSequence();
				ASSERT_THAT(IsNotNull(Sequence));

				ASSERT_THAT(IsNear(TestAnimLength, Sequence->GetPlayLength(), UE_SMALL_NUMBER));
			})
		.Then(TEXT("Generate Asset Data"), [&]()
			{
				AnimAsset = UE::UAF::FAssetDataFactory::CreateUAFAssetDataFromObject<FUAFGraphFactoryAsset_Animation>(Sequence);
				ASSERT_THAT(IsTrue(AnimAsset.IsValid()));
				ASSERT_THAT(AreEqual(Sequence, AnimAsset.Get().AnimationSequence));
				AnimAsset.GetMutable().StartTimeType = EAnimationStartTimeType::TimeInSeconds;
				AnimAsset.GetMutable().StartTimeSeconds = 9999.f;
				AnimAsset.GetMutable().LoopMode = UE::UAF::EAnimAssetLoopMode::ForceNonLoop;
				
			})
		.Then(TEXT("Generate Factory Params From Asset"), [&]()
			{
				FAnimNextFactoryParams Params = UE::UAF::FAnimGraphFactory::GetDefaultParamsForAsset(AnimAsset);
				FAnimNextSimpleAnimGraphBuilder& GraphBuilder = ((FAnimNextSimpleAnimGraphBuilder&)Params.GetBuilder());
				ASSERT_THAT(IsTrue(GraphBuilder.Stacks.Num() > 0));
				ASSERT_THAT(IsTrue(GraphBuilder.Stacks[0].TraitDescs.Num() > 0));
				const FAnimNextSimpleAnimGraphBuilderTraitDesc* SequencePlayerTraitData = GraphBuilder.Stacks[0].TraitDescs.FindByPredicate(
					[](const FAnimNextSimpleAnimGraphBuilderTraitDesc& Item)
					{
						return Item.TraitData.GetPtr<UE::UAF::FSequencePlayerData>() != nullptr;
					});
				ASSERT_THAT(IsNotNull(SequencePlayerTraitData));

				const UE::UAF::FSequencePlayerData* TypedSequencePlayerData = SequencePlayerTraitData->TraitData.GetPtr<UE::UAF::FSequencePlayerData>();
				ASSERT_THAT(IsNotNull(TypedSequencePlayerData));
				// Should be clamped to duration
				ASSERT_THAT(IsNear(TestAnimLength, TypedSequencePlayerData->StartPosition, UE_SMALL_NUMBER));
				
			})
		.OnTearDown(TEXT("Tear Down"), [&]()
		{
			AnimAsset.Reset();
			Sequence = nullptr;
		});
	}

	TEST_METHOD(Verify_Core_Asset_Types_Anim_Sequence_StartTime_Clamp_Ratio)
	{
	
		TestCommandBuilder.Do(TEXT("Setup"), [&]()
			{
				Sequence = CreateAnimSequence();
				ASSERT_THAT(IsNotNull(Sequence));

				ASSERT_THAT(IsNear(TestAnimLength, Sequence->GetPlayLength(), UE_SMALL_NUMBER));
			})
		.Then(TEXT("Generate Asset Data"), [&]()
			{
				AnimAsset = UE::UAF::FAssetDataFactory::CreateUAFAssetDataFromObject<FUAFGraphFactoryAsset_Animation>(Sequence);
				ASSERT_THAT(IsTrue(AnimAsset.IsValid()));
				ASSERT_THAT(AreEqual(Sequence, AnimAsset.Get().AnimationSequence));
				AnimAsset.GetMutable().StartTimeType = EAnimationStartTimeType::TimePercent;
				AnimAsset.GetMutable().StartTimePercent = 200.0f;	// Set at 200%
				AnimAsset.GetMutable().LoopMode = UE::UAF::EAnimAssetLoopMode::ForceNonLoop;
				
			})
		.Then(TEXT("Generate Factory Params From Asset"), [&]()
			{
				FAnimNextFactoryParams Params = UE::UAF::FAnimGraphFactory::GetDefaultParamsForAsset(AnimAsset);
				FAnimNextSimpleAnimGraphBuilder& GraphBuilder = ((FAnimNextSimpleAnimGraphBuilder&)Params.GetBuilder());
				ASSERT_THAT(IsTrue(GraphBuilder.Stacks.Num() > 0));
				ASSERT_THAT(IsTrue(GraphBuilder.Stacks[0].TraitDescs.Num() > 0));
				const FAnimNextSimpleAnimGraphBuilderTraitDesc* SequencePlayerTraitData = GraphBuilder.Stacks[0].TraitDescs.FindByPredicate(
					[](const FAnimNextSimpleAnimGraphBuilderTraitDesc& Item)
					{
						return Item.TraitData.GetPtr<UE::UAF::FSequencePlayerData>() != nullptr;
					});
				ASSERT_THAT(IsNotNull(SequencePlayerTraitData));

				const UE::UAF::FSequencePlayerData* TypedSequencePlayerData = SequencePlayerTraitData->TraitData.GetPtr<UE::UAF::FSequencePlayerData>();
				ASSERT_THAT(IsNotNull(TypedSequencePlayerData));
				// Should be clamped to duration
				ASSERT_THAT(IsNear(TestAnimLength, TypedSequencePlayerData->StartPosition, UE_SMALL_NUMBER));
				
			})
		.OnTearDown(TEXT("Tear Down"), [&]()
		{
			AnimAsset.Reset();
			Sequence = nullptr;
		});
	}

	float TestStartTimeSeconds = 1.15f;
	TEST_METHOD(Verify_Core_Asset_Types_Anim_Sequence_StartTime_Seconds)
	{
	
		TestCommandBuilder.Do(TEXT("Setup"), [&]()
			{
				Sequence = CreateAnimSequence();
				ASSERT_THAT(IsNotNull(Sequence));

				ASSERT_THAT(IsNear(TestAnimLength, Sequence->GetPlayLength(), UE_SMALL_NUMBER));
			})
		.Then(TEXT("Generate Asset Data"), [&]()
			{
				AnimAsset = UE::UAF::FAssetDataFactory::CreateUAFAssetDataFromObject<FUAFGraphFactoryAsset_Animation>(Sequence);
				ASSERT_THAT(IsTrue(AnimAsset.IsValid()));
				ASSERT_THAT(AreEqual(Sequence, AnimAsset.Get().AnimationSequence));
				AnimAsset.GetMutable().StartTimeType = EAnimationStartTimeType::TimeInSeconds;
				AnimAsset.GetMutable().StartTimeSeconds = TestStartTimeSeconds;
				
			})
		.Then(TEXT("Generate Factory Params From Asset"), [&]()
			{
				FAnimNextFactoryParams Params = UE::UAF::FAnimGraphFactory::GetDefaultParamsForAsset(AnimAsset);
				FAnimNextSimpleAnimGraphBuilder& GraphBuilder = ((FAnimNextSimpleAnimGraphBuilder&)Params.GetBuilder());
				ASSERT_THAT(IsTrue(GraphBuilder.Stacks.Num() > 0));
				ASSERT_THAT(IsTrue(GraphBuilder.Stacks[0].TraitDescs.Num() > 0));
				const FAnimNextSimpleAnimGraphBuilderTraitDesc* SequencePlayerTraitData = GraphBuilder.Stacks[0].TraitDescs.FindByPredicate(
					[](const FAnimNextSimpleAnimGraphBuilderTraitDesc& Item)
					{
						return Item.TraitData.GetPtr<UE::UAF::FSequencePlayerData>() != nullptr;
					});
				ASSERT_THAT(IsNotNull(SequencePlayerTraitData));

				const UE::UAF::FSequencePlayerData* TypedSequencePlayerData = SequencePlayerTraitData->TraitData.GetPtr<UE::UAF::FSequencePlayerData>();
				ASSERT_THAT(IsNotNull(TypedSequencePlayerData));
				ASSERT_THAT(IsNear(TestStartTimeSeconds, TypedSequencePlayerData->StartPosition, UE_SMALL_NUMBER));
				
			})
		.OnTearDown(TEXT("Tear Down"), [&]()
		{
			AnimAsset.Reset();
			Sequence = nullptr;
		});
	}
	
	TEST_METHOD(Verify_Core_Asset_Types_Anim_Sequence_StartTime_Negative_TimeRatio_Is_Clamped)
	{
	
		TestCommandBuilder.Do(TEXT("Setup"), [&]()
			{
				Sequence = CreateAnimSequence();
				ASSERT_THAT(IsNotNull(Sequence));

				ASSERT_THAT(IsNear(TestAnimLength, Sequence->GetPlayLength(), UE_SMALL_NUMBER));
			})
		.Then(TEXT("Generate Asset Data"), [&]()
			{
				AnimAsset = UE::UAF::FAssetDataFactory::CreateUAFAssetDataFromObject<FUAFGraphFactoryAsset_Animation>(Sequence);
				ASSERT_THAT(IsTrue(AnimAsset.IsValid()));
				ASSERT_THAT(AreEqual(Sequence, AnimAsset.Get().AnimationSequence));
				AnimAsset.GetMutable().StartTimeType = EAnimationStartTimeType::TimePercent;
				AnimAsset.GetMutable().StartTimePercent = -1.0f;
				
			})
		.Then(TEXT("Generate Factory Params From Asset"), [&]()
			{
				FAnimNextFactoryParams Params = UE::UAF::FAnimGraphFactory::GetDefaultParamsForAsset(AnimAsset);
				FAnimNextSimpleAnimGraphBuilder& GraphBuilder = ((FAnimNextSimpleAnimGraphBuilder&)Params.GetBuilder());
				ASSERT_THAT(IsTrue(GraphBuilder.Stacks.Num() > 0));
				ASSERT_THAT(IsTrue(GraphBuilder.Stacks[0].TraitDescs.Num() > 0));
				const FAnimNextSimpleAnimGraphBuilderTraitDesc* SequencePlayerTraitData = GraphBuilder.Stacks[0].TraitDescs.FindByPredicate(
					[](const FAnimNextSimpleAnimGraphBuilderTraitDesc& Item)
					{
						return Item.TraitData.GetPtr<UE::UAF::FSequencePlayerData>() != nullptr;
					});
				ASSERT_THAT(IsNotNull(SequencePlayerTraitData));

				const UE::UAF::FSequencePlayerData* TypedSequencePlayerData = SequencePlayerTraitData->TraitData.GetPtr<UE::UAF::FSequencePlayerData>();
				ASSERT_THAT(IsNotNull(TypedSequencePlayerData));
				ASSERT_THAT(IsNear(0.0f, TypedSequencePlayerData->StartPosition, UE_SMALL_NUMBER));
				
			})
		.OnTearDown(TEXT("Tear Down"), [&]()
		{
			AnimAsset.Reset();
			Sequence = nullptr;
		});
	}

	TObjectPtr<UBlendSpace> BlendSpace;
	TInstancedStruct<FUAFGraphFactoryAsset_BlendSpace> BlendSpace_Asset;
	float TestX = 1.3f;
	float TestY = 2.3f;
	TEST_METHOD(Verify_Core_Asset_Types_BlendSpace)
	{
		TestCommandBuilder.Do(TEXT("Setup"), [&]()
			{
				BlendSpace = NewObject<UBlendSpace>();
				ASSERT_THAT(IsNotNull(BlendSpace));
			})
		.Then(TEXT("Generate Asset Data"), [&]()
			{
				BlendSpace_Asset = UE::UAF::FAssetDataFactory::CreateUAFAssetDataFromObject<FUAFGraphFactoryAsset_BlendSpace>(BlendSpace);
				ASSERT_THAT(IsTrue(BlendSpace_Asset.IsValid()));
				ASSERT_THAT(AreEqual(BlendSpace, BlendSpace_Asset.Get().BlendSpace));
				BlendSpace_Asset.GetMutable().PlayRate = TestRateScale;
				BlendSpace_Asset.GetMutable().XAxisSamplePoint = TestX;
				BlendSpace_Asset.GetMutable().YAxisSamplePoint = TestY;
			})
		.Then(TEXT("Generate Factory Params From Asset"), [&]()
			{
				FAnimNextFactoryParams Params = UE::UAF::FAnimGraphFactory::GetDefaultParamsForAsset(BlendSpace_Asset);
				FAnimNextSimpleAnimGraphBuilder& GraphBuilder = ((FAnimNextSimpleAnimGraphBuilder&)Params.GetBuilder());
				ASSERT_THAT(IsTrue(GraphBuilder.Stacks.Num() > 0));
				ASSERT_THAT(IsTrue(GraphBuilder.Stacks[0].TraitDescs.Num() > 0));
				const FAnimNextSimpleAnimGraphBuilderTraitDesc* BlendSpacePlayerTraitDesc = GraphBuilder.Stacks[0].TraitDescs.FindByPredicate(
					[](const FAnimNextSimpleAnimGraphBuilderTraitDesc& Item)
					{
						return Item.TraitData.GetPtr<UE::UAF::FBlendSpacePlayerData>() != nullptr;
					});
				ASSERT_THAT(IsNotNull(BlendSpacePlayerTraitDesc));
				ASSERT_THAT(IsNotNull(BlendSpacePlayerTraitDesc->TraitData.GetMemory()));

				const UE::UAF::FBlendSpacePlayerData* TypedBlendSpacePlayerData = BlendSpacePlayerTraitDesc->TraitData.GetPtr<UE::UAF::FBlendSpacePlayerData>();
				ASSERT_THAT(IsNotNull(TypedBlendSpacePlayerData));
				ASSERT_THAT(IsNear(TestRateScale, TypedBlendSpacePlayerData->PlayRate, UE_SMALL_NUMBER));
				ASSERT_THAT(IsNear(TestX, TypedBlendSpacePlayerData->XAxisSamplePoint, UE_SMALL_NUMBER));
				ASSERT_THAT(IsNear(TestY, TypedBlendSpacePlayerData->YAxisSamplePoint, UE_SMALL_NUMBER));
			})
		.OnTearDown(TEXT("Tear Down"), [&]()
		{
			BlendSpace_Asset.Reset();
			BlendSpace = nullptr;
		});
	}
	
	TObjectPtr<UUAFAnimGraph> Graph;
    TInstancedStruct<FUAFGraphFactoryAsset_Graph> Graph_Asset;
    TEST_METHOD(Verify_Core_Asset_Types_Graph)
    {
    	TestCommandBuilder.Do(TEXT("Setup"), [&]()
    		{
    			Graph = NewObject<UUAFAnimGraph>();
    			ASSERT_THAT(IsNotNull(Graph));
    		})
    	.Then(TEXT("Generate Asset Data"), [&]()
    		{
    			Graph_Asset = UE::UAF::FAssetDataFactory::CreateUAFAssetDataFromObject<FUAFGraphFactoryAsset_Graph>(Graph);
    			ASSERT_THAT(IsTrue(Graph_Asset.IsValid()));
    			ASSERT_THAT(AreEqual(Graph, Graph_Asset.Get().AnimationGraph));
    		})
    	.Then(TEXT("Generate Factory Params From Asset"), [&]()
    		{
    			FAnimNextFactoryParams Params = UE::UAF::FAnimGraphFactory::GetDefaultParamsForAsset(Graph_Asset);
    			FAnimNextSimpleAnimGraphBuilder& GraphBuilder = ((FAnimNextSimpleAnimGraphBuilder&)Params.GetBuilder());
    			ASSERT_THAT(IsTrue(GraphBuilder.Stacks.Num() > 0));
    			ASSERT_THAT(IsTrue(GraphBuilder.Stacks[0].TraitDescs.Num() > 0));
    			const FAnimNextSimpleAnimGraphBuilderTraitDesc* InjectionSiteTraitDesc = GraphBuilder.Stacks[0].TraitDescs.FindByPredicate(
    				[](const FAnimNextSimpleAnimGraphBuilderTraitDesc& Item)
    				{
    					return Item.TraitData.GetPtr<UE::UAF::FInjectionSiteTraitData>() != nullptr;
    				});
    			ASSERT_THAT(IsNotNull(InjectionSiteTraitDesc));
    			ASSERT_THAT(IsNotNull(InjectionSiteTraitDesc->TraitData.GetMemory()));

    			const UE::UAF::FInjectionSiteTraitData* InjectionSiteData = InjectionSiteTraitDesc->TraitData.GetPtr<UE::UAF::FInjectionSiteTraitData>();
    			ASSERT_THAT(IsNotNull(InjectionSiteData));
    			ASSERT_THAT(AreEqual(InjectionSiteData->Graph.Asset, Graph));
    		})
    	.OnTearDown(TEXT("Tear Down"), [&]()
    	{
    		Graph_Asset.Reset();
			Graph = nullptr;
    	});
    }
	
	FTopLevelAssetPath TestObjectClassPath;
	FTopLevelAssetPath ChildASubobjectClassPath;
	TEST_METHOD(Verify_GetRegisteredObjectClasses_By_Base_Asset_Type)
	{
		TestCommandBuilder
		.Do(TEXT("Register test types"), [&]()
			{
				/** 
				 * Class Hierarchy for Test
				 * UUAFAssetDataTestObject (registered) -> UUAFAssetDataTestObject_ChildA (registered) -> UUAFAssetDataTestObject_GrandChildA
				 *										-> UUAFAssetDataTestObject_ChildB -> UUAFAssetDataTestObject_GrandChildB
				*/
				
				// Register UUAFAssetDataTestObject with FUAFGraphFactoryAsset_Test
				TestObjectClassPath = UE::UAF::FAssetDataFactory::RegisterAsset<FUAFGraphFactoryAsset_Test, UUAFAssetDataTestObject>(
					[](const UUAFAssetDataTestObject*) { return FUAFGraphFactoryAsset_Test(); });

				// Register UUAFAssetDataTestObject_ChildA with its own factory FUAFGraphFactoryAsset_TestChildA
				ChildASubobjectClassPath = UE::UAF::FAssetDataFactory::RegisterAsset<FUAFGraphFactoryAsset_TestChildA, UUAFAssetDataTestObject_ChildA>(
					[](const UUAFAssetDataTestObject_ChildA*) { return FUAFGraphFactoryAsset_TestChildA(); });
			})
		.Then(TEXT("Verify classes returned with GetRegisteredObjectClasses for FUAFGraphFactoryAsset_Test"), [&]()
			{
				TArray<UClass*> Classes = UE::UAF::FAssetDataFactory::GetRegisteredObjectClasses<FUAFGraphFactoryAsset_Test>();

				// The directly registered class must be present
				ASSERT_THAT(IsTrue(Classes.Contains(UUAFAssetDataTestObject::StaticClass())))

				// Any unregistered subclass is gathered via GatherDerivedObjectClasses
				ASSERT_THAT(IsTrue(Classes.Contains(UUAFAssetDataTestObject_ChildB::StaticClass())))
				ASSERT_THAT(IsTrue(Classes.Contains(UUAFAssetDataTestObject_GrandChildB::StaticClass())))

				// A subclass that has its own separate registration must NOT be gathered —
				// GatherDerivedObjectClasses stops at classes that have their own initializer
				ASSERT_THAT(IsFalse(Classes.Contains(UUAFAssetDataTestObject_ChildA::StaticClass())))
				
				// A subclass whose parent has its own initializer must not be gathered
				ASSERT_THAT(IsFalse(Classes.Contains(UUAFAssetDataTestObject_GrandChildA::StaticClass())))
			})
		.Then(TEXT("Verify classes returned for FUAFGraphFactoryAsset_TestChildA"), [&]()
			{
				TArray<UClass*> Classes = UE::UAF::FAssetDataFactory::GetRegisteredObjectClasses<FUAFGraphFactoryAsset_TestChildA>();

				// The class registered under this asset type should appear
				ASSERT_THAT(IsTrue(Classes.Contains(UUAFAssetDataTestObject_ChildA::StaticClass())))
				
				// An unregistered subclass is gathered via GatherDerivedObjectClasses
				ASSERT_THAT(IsTrue(Classes.Contains(UUAFAssetDataTestObject_GrandChildA::StaticClass())))

				// The parent class and its unregistered subclass belong to a different asset type
				ASSERT_THAT(IsFalse(Classes.Contains(UUAFAssetDataTestObject::StaticClass())))
				ASSERT_THAT(IsFalse(Classes.Contains(UUAFAssetDataTestObject_ChildB::StaticClass())))
				
			})
		.Then(TEXT("Verify classes returned for FUAFGraphFactoryAsset (common base)"), [&]()
			{
				TArray<UClass*> Classes = UE::UAF::FAssetDataFactory::GetRegisteredObjectClasses<FUAFGraphFactoryAsset>();

				// All subclasses should appear when querying the shared base asset type
				ASSERT_THAT(IsTrue(Classes.Contains(UUAFAssetDataTestObject::StaticClass())))
				ASSERT_THAT(IsTrue(Classes.Contains(UUAFAssetDataTestObject_ChildA::StaticClass())))
				ASSERT_THAT(IsTrue(Classes.Contains(UUAFAssetDataTestObject_ChildB::StaticClass())))
				ASSERT_THAT(IsTrue(Classes.Contains(UUAFAssetDataTestObject_GrandChildA::StaticClass())))
				ASSERT_THAT(IsTrue(Classes.Contains(UUAFAssetDataTestObject_GrandChildB::StaticClass())))
			})
		.OnTearDown(TEXT("Unregister test types"), [&]()
			{
				UE::UAF::FAssetDataFactory::UnregisterAsset(TestObjectClassPath);
				UE::UAF::FAssetDataFactory::UnregisterAsset(ChildASubobjectClassPath);
			});
	}

};

TObjectPtr<USkeletalMesh> FUAFAssetDataTests::GSkeletalMesh = nullptr;
TObjectPtr<USkeleton> FUAFAssetDataTests::GSkeleton = nullptr;

#undef LOCTEXT_NAMESPACE

#endif// WITH_EDITOR
#endif // WITH_DEV_AUTOMATION_TESTS