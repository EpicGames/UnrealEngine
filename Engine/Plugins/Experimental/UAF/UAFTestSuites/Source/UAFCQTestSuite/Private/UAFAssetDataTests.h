// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/UAFAssetData.h"

#include "UAFAssetDataTests.generated.h"

USTRUCT(meta=(Hidden))
struct FUAFGraphFactoryAsset_Test : public FUAFGraphFactoryAsset
{
	GENERATED_BODY()
public:
	int TestData = 0;
};

UCLASS()
class UUAFAssetDataTestObject : public UObject
{
	GENERATED_BODY()

public:
	int TestData = 0;
};


// Subclass of UUAFAssetDataTestObject that IS registered with its own asset type (FUAFGraphFactoryAsset_TestChildA).
// GetRegisteredObjectClasses with the parent's asset type should NOT include it because it stops recursing at classes that have their own registered initializer.
UCLASS()
class UUAFAssetDataTestObject_ChildA : public UUAFAssetDataTestObject
{
	GENERATED_BODY()
};

// Factory Asset to register the asset type of UUAFAssetDataTestObject_ChildA 
USTRUCT(meta=(Hidden))
struct FUAFGraphFactoryAsset_TestChildA : public FUAFGraphFactoryAsset
{
	GENERATED_BODY()
};

// Subclass of UUAFAssetDataTestObject that is NOT registered with FAssetDataFactory.
// GetRegisteredObjectClasses with the parent's asset type should include it
UCLASS()
class UUAFAssetDataTestObject_ChildB : public UUAFAssetDataTestObject
{
	GENERATED_BODY()
};

// Subclass of UUAFAssetDataTestObject_ChildA
// GetRegisteredObjectClasses with UUAFAssetDataTestObject_ChildA as base type should include this 
// GetRegisteredObjectClasses with UUAFAssetDataTestObject should not include this 
UCLASS()
class UUAFAssetDataTestObject_GrandChildA : public UUAFAssetDataTestObject_ChildA
{
	GENERATED_BODY()
};
 
// Subclass of UUAFAssetDataTestObject_ChildB
// GetRegisteredObjectClasses with UUAFAssetDataTestObject should include this and its parent 
UCLASS()
class UUAFAssetDataTestObject_GrandChildB : public UUAFAssetDataTestObject_ChildB
{
	GENERATED_BODY()
};
 


