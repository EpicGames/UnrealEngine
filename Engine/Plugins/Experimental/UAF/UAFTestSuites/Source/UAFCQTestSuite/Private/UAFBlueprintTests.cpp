// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS
#if WITH_EDITOR

#include "UAFTestsUtilities.h"
#include "CQTest.h"
#include "Editor.h"
#include "Component/AnimNextComponent.h"
#include "Components/ActorComponent.h"
#include "Factories/BlueprintFactory.h"
#include "GameFramework/Character.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Module/AnimNextModule.h"
#include "Module/AnimNextModuleFactory.h"
#include "Module/UAFSystemAssetData.h"

TEST_CLASS(FAssetBlueprintTests, "Animation.UAF.Functional")
{
	UBlueprint* Blueprint;
	UObject* UAFAsset;
	UUAFComponent* AnimNextComponent;
	UAssetEditorSubsystem* AssetEditorSubsystem;

	BEFORE_EACH()
	{
		AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		ASSERT_THAT(IsNotNull(AssetEditorSubsystem));
	}

	// QMetry Test Case:  UE-TC-10710 
	TEST_METHOD(Verify_Blueprint_Add_AnimNext_Component)
	{
		TestCommandBuilder
			.Do(TEXT("Create Character Blueprint Asset"), [&]()
			{
				UBlueprintFactory* Factory = NewObject<UBlueprintFactory>();
				Factory->ParentClass = ACharacter::StaticClass();
				
				const FString BlueprintName = "NewBlueprint";
				UObject* factoryObject = UAFTestsUtilities::CreateFactoryObject(Factory, UBlueprint::StaticClass(), BlueprintName);
				Blueprint = CastChecked<UBlueprint>(factoryObject);
				ASSERT_THAT(IsNotNull(Blueprint));
				ASSERT_THAT(AreEqual(BlueprintName, Blueprint->GetFName()));
			})
			.Then(TEXT("Create AnimNext Component"), [&]()
			{
				const FName ComponentName = FName(TEXT("AnimNext"));
				AnimNextComponent = NewObject<UUAFComponent>(GetTransientPackage(), UUAFComponent::StaticClass(), ComponentName);
				ASSERT_THAT(IsNotNull(AnimNextComponent));
				ASSERT_THAT(AreEqual(ComponentName, AnimNextComponent->GetFName()));
			})
			.Then(TEXT("Add the AnimNext Component to the Character Blueprint and Open in Editor"), [&]()
			{
				TArray<UActorComponent*> Components;
				Components.Add(AnimNextComponent);
				FKismetEditorUtilities::AddComponentsToBlueprint(Blueprint, Components);
				ASSERT_THAT(IsTrue(AssetEditorSubsystem->OpenEditorForAsset(Blueprint)));
			});
	}
	
	// QMetry Test Case:  UE-TC-10711
	TEST_METHOD(Verify_Assign_Module_To_Blueprint)
	{
		TestCommandBuilder
			.Do(TEXT("Create UAF Module Asset"), [&]()
			{
				const FString UAFModuleName = "NewAnimNextModule";
				const TSubclassOf<UUAFSystemFactory> ModuleFactoryClass = UUAFSystemFactory::StaticClass();
				UObject* factoryObject = UAFTestsUtilities::CreateFactoryObject(NewObject<UUAFSystemFactory>(), UUAFSystem::StaticClass(), UAFModuleName);				
				UAFAsset = CastChecked<UUAFSystem>(factoryObject);			
				ASSERT_THAT(IsNotNull(UAFAsset));
				ASSERT_THAT(AreEqual(UAFModuleName, UAFAsset->GetFName()));
			})			
			.Then(TEXT("Create Character Blueprint Asset"), [&]()
			{
				UBlueprintFactory* Factory = NewObject<UBlueprintFactory>();
				Factory->ParentClass = ACharacter::StaticClass();
				
				const FString BlueprintName = "NewBlueprint";
				UObject* factoryObject = UAFTestsUtilities::CreateFactoryObject(Factory, UBlueprint::StaticClass(), BlueprintName);
				Blueprint = CastChecked<UBlueprint>(factoryObject);
				ASSERT_THAT(IsNotNull(Blueprint));
				ASSERT_THAT(AreEqual(BlueprintName, Blueprint->GetFName()));
			})	
			.Then(TEXT("Create AnimNext Component"), [&]()
			{
				const FName ComponentName = FName(TEXT("AnimNext"));
				AnimNextComponent = NewObject<UUAFComponent>(UAFAsset->GetPackage(), UUAFComponent::StaticClass(), ComponentName);
				ASSERT_THAT(IsNotNull(AnimNextComponent));
				ASSERT_THAT(AreEqual(ComponentName, AnimNextComponent->GetFName()));
			})
			.Then(TEXT("Set the UAF Module and add the AnimNext Component to the Character Blueprint and Open in Editor"), [&]()
			{
				TArray<UActorComponent*> Components;
				AnimNextComponent->SetAssetInternal(FUAFSystemFactoryAsset_System(Cast<UUAFSystem>(UAFAsset)));
				Components.Add(AnimNextComponent);
				FKismetEditorUtilities::AddComponentsToBlueprint(Blueprint, Components);
				ASSERT_THAT(IsTrue(AssetEditorSubsystem->OpenEditorForAsset(Blueprint)));
			});
	}
};

#endif // WITH_EDITOR
#endif // WITH_DEV_AUTOMATION_TESTS
