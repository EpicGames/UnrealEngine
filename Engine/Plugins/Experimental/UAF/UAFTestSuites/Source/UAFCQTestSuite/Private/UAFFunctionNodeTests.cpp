// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS
#if WITH_EDITOR

#include "UAFTestsUtilities.h"
#include "CQTest.h"
#include "Editor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextAnimationGraphFactory.h"
#include "Module/AnimNextModule.h"
#include "Module/AnimNextModuleFactory.h"
#include "AnimNextStateTree.h"
#include "AnimNextStateTreeFactory.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "RigVMModel/Nodes/RigVMLibraryNode.h"

TEST_CLASS(FAssetFunctionNodeTests, "Animation.UAF.Functional")
{
	UUAFRigVMAsset* AnimNextAsset;
	UAssetEditorSubsystem* AssetEditorSubsystem;
	UUAFRigVMAssetEditorData* EditorData;
	URigVMLibraryNode* LibraryNode;
	const FString FunctionName = "NewFunction";

	BEFORE_EACH()
	{
		AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		ASSERT_THAT(IsNotNull(AssetEditorSubsystem));
	}

	// QMetry Test Case:  UE-TC-18834
	TEST_METHOD(AnimNextAnimationGraph_Create_Function_Node_Pins)
	{		
		TestCommandBuilder
			.Do(TEXT("Create AnimNext Animation Graph Asset"), [&]()
			{
				const TSubclassOf<UUAFAnimGraphFactory> AnimNextAnimGraphFactoryClass = UUAFAnimGraphFactory::StaticClass();
				UObject* factoryObject = UAFTestsUtilities::CreateFactoryObject(NewObject<UUAFAnimGraphFactory>(GetTransientPackage(), AnimNextAnimGraphFactoryClass.Get()), UUAFAnimGraph::StaticClass(), "NewAnimNextAnimationGraph");
				AnimNextAsset = CastChecked<UUAFAnimGraph>(factoryObject);				
				ASSERT_THAT(IsTrue(AssetEditorSubsystem->OpenEditorForAsset(AnimNextAsset)));
			})
			.Then(TEXT("Create Function Node"), [&]()
			{
				LibraryNode = UAFTestsUtilities::AddFunctionNode(AnimNextAsset);
				ASSERT_THAT(IsNotNull(LibraryNode));
				ASSERT_THAT(AreEqual(FunctionName, LibraryNode->GetNodeTitle()));
			})
			.Then(TEXT("Assert Function Node Creation"), [&]()
			{
				EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(AnimNextAsset);	
				URigVMController* Controller = CastChecked<IRigVMClientHost>(EditorData)->GetControllerByName(FunctionName);
				ASSERT_THAT(IsNotNull(Controller));
				ASSERT_THAT(IsNotNull(Controller->GetGraph()));				
			})		
			.Then(TEXT("Add Function Node Pins"), [&]()
			{
				const FString PinName = "Argument";
				URigVMPin* InputPin = UAFTestsUtilities::AddPin(AnimNextAsset, LibraryNode, ERigVMPinDirection::Input, PinName);
				URigVMPin* OutputPin =  UAFTestsUtilities::AddPin(AnimNextAsset, LibraryNode, ERigVMPinDirection::Output, PinName);
				ASSERT_THAT(IsNotNull(InputPin));
				ASSERT_THAT(IsNotNull(OutputPin));
			});
	}
	
	// QMetry Test Case:  UE-TC-18831
	TEST_METHOD(Undo_Redo_AnimNextModule_Create_Function_Node)
	{		
		TestCommandBuilder
			.Do(TEXT("Create AnimNext Module Asset"), [&]()
			{
				const TSubclassOf<UUAFSystemFactory> AnimNextModuleFactoryClass = UUAFSystemFactory::StaticClass();
				UObject* factoryObject = UAFTestsUtilities::CreateFactoryObject(NewObject<UUAFSystemFactory>(GetTransientPackage(), AnimNextModuleFactoryClass.Get()), UUAFSystem::StaticClass(), "NewAnimNextModule");
				AnimNextAsset = CastChecked<UUAFSystem>(factoryObject);				
				ASSERT_THAT(IsTrue(AssetEditorSubsystem->OpenEditorForAsset(AnimNextAsset)));
			})
			.Then(TEXT("Create Function Node"), [&]()
			{
				LibraryNode = UAFTestsUtilities::AddFunctionNode(AnimNextAsset);
				ASSERT_THAT(IsNotNull(LibraryNode));
				ASSERT_THAT(AreEqual(FunctionName, LibraryNode->GetNodeTitle()));
			})
			.Then(TEXT("Assert Function Node Creation"), [&]()
			{
				EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(AnimNextAsset);	
				URigVMController* Controller = CastChecked<IRigVMClientHost>(EditorData)->GetControllerByName(FunctionName);
				ASSERT_THAT(IsNotNull(Controller));
				ASSERT_THAT(IsNotNull(Controller->GetGraph()));				
			})
			.Until(TEXT("Execute Undo"), [&]()
			{
				return GEditor->UndoTransaction();
			})	
			.Then(TEXT("Assert Undo Function Node"), [&]()
			{
				URigVMController* Controller = CastChecked<IRigVMClientHost>(EditorData)->GetControllerByName(FunctionName);	
				ASSERT_THAT(IsNull(Controller));
			})
			.Until(TEXT("Execute Redo"), [&]()
			{
				return GEditor->RedoTransaction();
			})
			.Then(TEXT("Assert Redo Function Node"), [&]()
			{
				URigVMController* Controller = CastChecked<IRigVMClientHost>(EditorData)->GetControllerByName(FunctionName);	
				ASSERT_THAT(IsNotNull(Controller));
				ASSERT_THAT(IsNotNull(Controller->GetGraph()));
			});
	}
	
	// QMetry Test Case:  UE-TC-18829
	TEST_METHOD(Undo_Redo_AnimNextAnimationGraph_Create_Function_Node)
	{		
		TestCommandBuilder
			.Do(TEXT("Create AnimNext Animation Graph Asset"), [&]()
			{
				const TSubclassOf<UUAFAnimGraphFactory> AnimNextAnimGraphFactoryClass = UUAFAnimGraphFactory::StaticClass();
				UObject* factoryObject = UAFTestsUtilities::CreateFactoryObject(NewObject<UUAFAnimGraphFactory>(GetTransientPackage(), AnimNextAnimGraphFactoryClass.Get()), UUAFAnimGraph::StaticClass(), "NewAnimNextAnimationGraph");
				AnimNextAsset = CastChecked<UUAFAnimGraph>(factoryObject);				
				ASSERT_THAT(IsTrue(AssetEditorSubsystem->OpenEditorForAsset(AnimNextAsset)));
			})
			.Then(TEXT("Create Function Node"), [&]()
			{
				LibraryNode = UAFTestsUtilities::AddFunctionNode(AnimNextAsset);
				ASSERT_THAT(IsNotNull(LibraryNode));
				ASSERT_THAT(AreEqual(FunctionName, LibraryNode->GetNodeTitle()));
			})
			.Then(TEXT("Assert Function Node Creation"), [&]()
			{
				EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(AnimNextAsset);	
				URigVMController* Controller = CastChecked<IRigVMClientHost>(EditorData)->GetControllerByName(FunctionName);
				ASSERT_THAT(IsNotNull(Controller));
				ASSERT_THAT(IsNotNull(Controller->GetGraph()));				
			})
			.Until(TEXT("Execute Undo"), [&]()
			{
				return GEditor->UndoTransaction();
			})	
			.Then(TEXT("Assert Undo Function Node"), [&]()
			{
				URigVMController* Controller = CastChecked<IRigVMClientHost>(EditorData)->GetControllerByName(FunctionName);	
				ASSERT_THAT(IsNull(Controller));
			})
			.Until(TEXT("Execute Redo"), [&]()
			{
				return GEditor->RedoTransaction();
			})
			.Then(TEXT("Assert Redo Function Node"), [&]()
			{
				URigVMController* Controller = CastChecked<IRigVMClientHost>(EditorData)->GetControllerByName(FunctionName);	
				ASSERT_THAT(IsNotNull(Controller));
				ASSERT_THAT(IsNotNull(Controller->GetGraph()));
			});
	}	
};

#endif // WITH_EDITOR
#endif // WITH_DEV_AUTOMATION_TESTS
