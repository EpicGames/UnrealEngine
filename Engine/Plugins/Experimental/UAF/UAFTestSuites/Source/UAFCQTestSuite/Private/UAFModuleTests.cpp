// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS
#if WITH_EDITOR

#include "UAFTestsUtilities.h"
#include "CQTest.h"
#include "Editor.h"
#include "ScopedTransaction.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "AnimNextTest.h"
#include "Module/SystemReference.h"
#include "Module/RigUnit_AnimNextModuleEvents.h"
#include "Module/AnimNextModuleInitMethod.h"
#include "Module/AnimNextModule.h"
#include "Module/AnimNextModuleFactory.h"
#include "Graph/RigUnit_AnimNextRunAnimationGraph_v2.h"
#include "Graph/RigUnit_AnimNextWriteSkeletalMeshComponentPose.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "Math/Vector2D.h"
#include "AssetToolsModule.h"
#include "BlueprintEditor.h"
#include "BlueprintEditorModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ObjectTools.h"
#include "Animation/AnimSequence.h"
#include "Asset/UAFAnimGraphAssetData.h"
#include "Component/AnimNextComponent.h"
#include "Engine/SCS_Node.h"
#include "Engine/SimpleConstructionScript.h"
#include "Engine/SkeletalMesh.h"
#include "Factories/BlueprintFactory.h"
#include "GameFramework/Character.h"
#include "Graph/RigUnit_UAFRunAsset.h"
#include "Graph/RigUnit_UAFWriteSystemOutput.h"
#include "Graph/UAFSystemOutputComponent.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Module/AnimNextModule_EditorData.h"
#include "Module/UAFSystemAssetData.h"
#include "Subsystems/EditorActorSubsystem.h"
#include "UObject/SavePackage.h"

TEST_CLASS(FAnimNextModuleTests, "Animation.UAF.Functional")
{
	UUAFSystem* AnimNextModule = nullptr;
	UPackage* AnimNextModulePackage = nullptr;
	FName AnimNextModulePackageName;
	UAssetEditorSubsystem* AssetEditorSubsystem = nullptr;
	UUAFRigVMAssetEditorData* EditorData = nullptr;
	URigVMGraph* ControllerGraph = nullptr;
	UEdGraph* ParentGraph = nullptr;
	UEdGraphNode* RunGraphNode = nullptr;
	UEdGraphNode* WritePoseNode = nullptr;
	const FString InitializeNodeName = "AnimNextInitializeEvent";
	const FString RunGraphNodeName = "AnimNextRunAnimationGraph_v2";
	const FString WritePoseNodeName = "AnimNextWriteSkeletalMeshComponentPose";
	const FString CollapseNodeName = "CollapseNode";
	const FString FunctionNodeName = "New Function";
	const FString VariableName = "NewVariable";
	const FString GetVariableNodeName = "VariableNode";
	const FString SetVariableNodeName = "VariableNode_1";

	BEFORE_EACH()
	{
		AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		ASSERT_THAT(IsNotNull(AssetEditorSubsystem));
	}
	
	AFTER_EACH()
	{
		if (AnimNextModule)
		{
			ObjectTools::DeleteSingleObject(AnimNextModule);
			AnimNextModule = nullptr;
		}
	}
		
	// QMetry Test Case:  UE-TC-18778
	TEST_METHOD(Undo_Redo_In_AnimNext_Module)
	{
		TestCommandBuilder
			.Do(TEXT("Create AnimNext Module Asset"), [&]()
			{
				CreateAnimNextModule();
			})
			.Then(TEXT("Create a Write Pose node"), [&]()
			{
				EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(AnimNextModule);
				ASSERT_THAT(IsNotNull(EditorData));
				ControllerGraph = EditorData->GetControllerByName("RigVMGraph")->GetGraph();
				ASSERT_THAT(IsNotNull(ControllerGraph));				
				ParentGraph = Cast<UEdGraph>(EditorData->GetEditorObjectForRigVMGraph(ControllerGraph));
				ASSERT_THAT(IsNotNull(ParentGraph));
				
				TArray<UEdGraphPin*> FromPins;
				WritePoseNode = UAFTestsUtilities::AddUnitNode(ParentGraph, FRigUnit_AnimNextWriteSkeletalMeshComponentPose::StaticStruct()->GetPathName(), FromPins, FVector2f(250.0f, 135.0f));
				ASSERT_THAT(IsNotNull(WritePoseNode));
			})
			.Then(TEXT("Create a bool Variable"), [&]()
			{
				FScopedTransaction Transaction(NSLOCTEXT("Add_Variables", "AddVariables", "Add Variable(s)"));

				UAnimNextVariableEntry* VariableEntry = UAFTestsUtilities::AddVariable(AnimNextModule, FAnimNextParamType::GetType<bool>(), VariableName, "false");
				ASSERT_THAT(IsNotNull(VariableEntry));
			})
			.Then(TEXT("Add Get Variable Node to Graph"), [&]()
			{
				TArray<UEdGraphPin*> FromPins;
				UEdGraphNode* GetVariableNode = UAFTestsUtilities::AddVariableNode(ParentGraph, AnimNextModule, VariableName, FAnimNextParamType::GetType<bool>(), FAnimNextSchemaAction_Variable::EVariableAccessorChoice::Get, FromPins, FVector2f(-250.0f, 135.0f));
				ASSERT_THAT(IsNotNull(GetVariableNode));
			})
			.Then(TEXT("Add Set Variable Node to Graph"), [&]()
			{
				TArray<UEdGraphPin*> FromPins;
				UEdGraphNode* SetVariableNode = UAFTestsUtilities::AddVariableNode(ParentGraph, AnimNextModule, VariableName, FAnimNextParamType::GetType<bool>(), FAnimNextSchemaAction_Variable::EVariableAccessorChoice::Set, FromPins, FVector2f(-250.0f, 235.0f));
				ASSERT_THAT(IsNotNull(SetVariableNode));
			})
			.Until(TEXT("Execute Undo Set Variable Node"), [&]()
			{
				return GEditor->UndoTransaction();
			})
			.Until(TEXT("Execute Undo Get Variable Node"), [&]()
			{
				return GEditor->UndoTransaction();
			})
			.Until(TEXT("Execute Undo Create bool Variable"), [&]()
			{
				return GEditor->UndoTransaction();
			})		
			.Until(TEXT("Execute Undo Create Write Pose Node"), [&]()
			{
				return GEditor->UndoTransaction();
			})
			.Then(TEXT("Assert Undos"), [&]()
			{
				ASSERT_THAT(IsNull(ControllerGraph->FindNodeByName(FName(SetVariableNodeName))));
				ASSERT_THAT(IsNull(ControllerGraph->FindNodeByName(FName(GetVariableNodeName))));
				ASSERT_THAT(IsNull(ControllerGraph->FindNodeByName(FName(WritePoseNodeName))));
				ASSERT_THAT(IsNull(EditorData->FindEntry(FName(VariableName))));
			})
			.Until(TEXT("Execute Redo Create Write Pose Node"), [&]()
			{
				return GEditor->RedoTransaction();
			})
			.Until(TEXT("Execute Redo Create bool Variable"), [&]()
			{
				return GEditor->RedoTransaction();
			})
			.Until(TEXT("Execute Redo Get Variable Node"), [&]()
			{
				return GEditor->RedoTransaction();
			})
			.Until(TEXT("Execute Redo Set Variable Node"), [&]()
			{
				return GEditor->RedoTransaction();
			})
			.Then(TEXT("Assert Redos"), [&]()
			{
				ASSERT_THAT(IsNotNull(ControllerGraph->FindNodeByName(FName(SetVariableNodeName))));
				ASSERT_THAT(IsNotNull(ControllerGraph->FindNodeByName(FName(GetVariableNodeName))));
				ASSERT_THAT(IsNotNull(ControllerGraph->FindNodeByName(FName(WritePoseNodeName))));
			});
	}

	// QMetry Test Case:  UE-TC-18832
	TEST_METHOD(Collapse_Node_To_Function_In_AnimNext_Module)
	{
		TestCommandBuilder
			.Do(TEXT("Create AnimNext Module Asset"), [&]()
			{
				CreateAnimNextModule();
			})
			.Then(TEXT("Create a Run Graph node"), [&]()
			{
				EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(AnimNextModule);
				ASSERT_THAT(IsNotNull(EditorData));
				ControllerGraph = EditorData->GetControllerByName("RigVMGraph")->GetGraph();
				ASSERT_THAT(IsNotNull(ControllerGraph));				
				ParentGraph = Cast<UEdGraph>(EditorData->GetEditorObjectForRigVMGraph(ControllerGraph));
				ASSERT_THAT(IsNotNull(ParentGraph));	
				
				TArray<UEdGraphPin*> FromPins;
				RunGraphNode = UAFTestsUtilities::AddUnitNode(ParentGraph, FRigUnit_AnimNextRunAnimationGraph_v2::StaticStruct()->GetPathName(), FromPins, FVector2f(64.0f, 16.0f));
				ASSERT_THAT(IsNotNull(RunGraphNode));
			})
			.Then(TEXT("Create a Write Pose node"), [&]()
			{
				TArray<UEdGraphPin*> FromPins;
				WritePoseNode = UAFTestsUtilities::AddUnitNode(ParentGraph, FRigUnit_AnimNextWriteSkeletalMeshComponentPose::StaticStruct()->GetPathName(), FromPins, FVector2f(336.0f, 0.0f));
				ASSERT_THAT(IsNotNull(WritePoseNode));
			})
			.Then(TEXT("Add link from PrePhysics node Execute pin to Run Graph node Execute pin"), [&]()
			{
				ASSERT_THAT(IsTrue(UAFTestsUtilities::AddLink(AnimNextModule, "RigUnit_AnimNextPrePhysicsEvent.ExecuteContext", "AnimNextRunAnimationGraph_v2.ExecuteContext")));
			})
			.Then(TEXT("Add link from Run Graph node output pin to Write Pose node Execute pin"), [&]()
			{				
				ASSERT_THAT(IsTrue(UAFTestsUtilities::AddLink(AnimNextModule, "AnimNextRunAnimationGraph_v2.ExecuteContext", "AnimNextWriteSkeletalMeshComponentPose.ExecuteContext")));
			})
			.Then(TEXT("Collpase Run Graph and Write Pose nodes to a Function"), [&]()
			{
				TArray<FName> Nodes;
				Nodes.Add(FName(RunGraphNodeName));
				Nodes.Add(FName(WritePoseNodeName));
				URigVMCollapseNode* CollapseNode = UAFTestsUtilities::CollapseNodes(AnimNextModule, Nodes, FunctionNodeName, true);
				ASSERT_THAT(IsNotNull(CollapseNode));
			})		
			.Until(TEXT("Execute Undo Collpase Nodes"), [&]()
			{
				return GEditor->UndoTransaction();
			})
			.Then(TEXT("Assert Undo Collapse Nodes"), [&]()
			{
				ASSERT_THAT(IsNotNull(ControllerGraph->FindNodeByName(FName(RunGraphNodeName))));
				ASSERT_THAT(IsNotNull(ControllerGraph->FindNodeByName(FName(WritePoseNodeName))));
				ASSERT_THAT(IsNull(ControllerGraph->FindNodeByName(FName(FunctionNodeName))));
			})
			.Until(TEXT("Execute Redo Collapse Nodes"), [&]()
			{
				return GEditor->RedoTransaction();
			})
			.Then(TEXT("Assert Redo Collapse Nodes"), [&]()
			{
				ASSERT_THAT(IsNull(ControllerGraph->FindNodeByName(FName(RunGraphNodeName))));
				ASSERT_THAT(IsNull(ControllerGraph->FindNodeByName(FName(WritePoseNodeName))));
				ASSERT_THAT(IsNotNull(ControllerGraph->FindNodeByName(FName(FunctionNodeName))));
			});		
	}

	// QMetry Test Case:  UE-TC-18833
	TEST_METHOD(Collapse_Node_To_Node_In_AnimNext_Module)
	{
		TestCommandBuilder
			.Do(TEXT("Create AnimNext Module Asset"), [&]()
			{
				CreateAnimNextModule();
			})
			.Then(TEXT("Create a Run Graph node"), [&]()
			{
				EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(AnimNextModule);
				ASSERT_THAT(IsNotNull(EditorData));
				ControllerGraph = EditorData->GetControllerByName("RigVMGraph")->GetGraph();
				ASSERT_THAT(IsNotNull(ControllerGraph));				
				ParentGraph = Cast<UEdGraph>(EditorData->GetEditorObjectForRigVMGraph(ControllerGraph));
				ASSERT_THAT(IsNotNull(ParentGraph));	
				
				TArray<UEdGraphPin*> FromPins;
				RunGraphNode = UAFTestsUtilities::AddUnitNode(ParentGraph, FRigUnit_AnimNextRunAnimationGraph_v2::StaticStruct()->GetPathName(), FromPins, FVector2f(64.0f, 16.0f));
				ASSERT_THAT(IsNotNull(RunGraphNode));
			})
			.Then(TEXT("Create a Write Pose node"), [&]()
			{
				TArray<UEdGraphPin*> FromPins;
				WritePoseNode = UAFTestsUtilities::AddUnitNode(ParentGraph, FRigUnit_AnimNextWriteSkeletalMeshComponentPose::StaticStruct()->GetPathName(), FromPins, FVector2f(336.0f, 0.0f));
				ASSERT_THAT(IsNotNull(WritePoseNode));
			})
			.Then(TEXT("Add link from PrePhysics node Execute pin to Run Graph node Execute pin"), [&]()
			{
				ASSERT_THAT(IsTrue(UAFTestsUtilities::AddLink(AnimNextModule, "RigUnit_AnimNextPrePhysicsEvent.ExecuteContext", "AnimNextRunAnimationGraph_v2.ExecuteContext")));
			})
			.Then(TEXT("Add link from Run Graph node output pin to Write Pose node Execute pin"), [&]()
			{				
				ASSERT_THAT(IsTrue(UAFTestsUtilities::AddLink(AnimNextModule, "AnimNextRunAnimationGraph_v2.ExecuteContext", "AnimNextWriteSkeletalMeshComponentPose.ExecuteContext")));
			})
			.Then(TEXT("Collpase Run Graph and Write Pose nodes"), [&]()
			{
				TArray<FName> Nodes;
				Nodes.Add(FName(RunGraphNodeName));
				Nodes.Add(FName(WritePoseNodeName));
				URigVMCollapseNode* CollapseNode = UAFTestsUtilities::CollapseNodes(AnimNextModule, Nodes);
				ASSERT_THAT(IsNotNull(CollapseNode));
			})
			.Until(TEXT("Execute Undo Collpase Nodes"), [&]()
			{
				return GEditor->UndoTransaction();
			})
			.Then(TEXT("Assert Undo Collapse Nodes"), [&]()
			{
				ASSERT_THAT(IsNotNull(ControllerGraph->FindNodeByName(FName(RunGraphNodeName))));
				ASSERT_THAT(IsNotNull(ControllerGraph->FindNodeByName(FName(WritePoseNodeName))));
				ASSERT_THAT(IsNull(ControllerGraph->FindNodeByName(FName(CollapseNodeName))));
			})
			.Until(TEXT("Execute Redo Collapse Nodes"), [&]()
			{
				return GEditor->RedoTransaction();
			})
			.Then(TEXT("Assert Redo Collapse Nodes"), [&]()
			{
				ASSERT_THAT(IsNull(ControllerGraph->FindNodeByName(FName(RunGraphNodeName))));
				ASSERT_THAT(IsNull(ControllerGraph->FindNodeByName(FName(WritePoseNodeName))));
				ASSERT_THAT(IsNotNull(ControllerGraph->FindNodeByName(FName(CollapseNodeName))));
			});
	}
	
	TEST_METHOD(System_Input_Output)
	{
		TestCommandBuilder
			.Do(TEXT("Create UAF System Asset"), [&]()
			{
				CreateAnimNextModule();
			})
			.Then(TEXT("Create a Run Graph node"), [&]()
			{
				EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(AnimNextModule);
				ASSERT_THAT(IsNotNull(EditorData));
				ControllerGraph = EditorData->GetControllerByName("RigVMGraph")->GetGraph();
				ASSERT_THAT(IsNotNull(ControllerGraph));				
				ParentGraph = Cast<UEdGraph>(EditorData->GetEditorObjectForRigVMGraph(ControllerGraph));
				ASSERT_THAT(IsNotNull(ParentGraph));	
				
				TArray<UEdGraphPin*> FromPins;
				RunGraphNode = UAFTestsUtilities::AddUnitNode(ParentGraph, FRigUnit_AnimNextRunAnimationGraph_v2::StaticStruct()->GetPathName(), FromPins, FVector2f(64.0f, 16.0f));
				ASSERT_THAT(IsNotNull(RunGraphNode));
			})
			.Then(TEXT("Create a Write Pose node"), [&]()
			{
				TArray<UEdGraphPin*> FromPins;
				WritePoseNode = UAFTestsUtilities::AddUnitNode(ParentGraph, FRigUnit_AnimNextWriteSkeletalMeshComponentPose::StaticStruct()->GetPathName(), FromPins, FVector2f(336.0f, 0.0f));
				ASSERT_THAT(IsNotNull(WritePoseNode));
			})
			.Then(TEXT("Add link from PrePhysics node Execute pin to Run Graph node Execute pin"), [&]()
			{
				ASSERT_THAT(IsTrue(UAFTestsUtilities::AddLink(AnimNextModule, "RigUnit_AnimNextPrePhysicsEvent.ExecuteContext", "AnimNextRunAnimationGraph_v2.ExecuteContext")));
			})
			.Then(TEXT("Add link from Run Graph node output pin to Write Pose node Execute pin"), [&]()
			{				
				ASSERT_THAT(IsTrue(UAFTestsUtilities::AddLink(AnimNextModule, "AnimNextRunAnimationGraph_v2.ExecuteContext", "AnimNextWriteSkeletalMeshComponentPose.ExecuteContext")));
			})
			.Then(TEXT("Collpase Run Graph and Write Pose nodes"), [&]()
			{
				TArray<FName> Nodes;
				Nodes.Add(FName(RunGraphNodeName));
				Nodes.Add(FName(WritePoseNodeName));
				URigVMCollapseNode* CollapseNode = UAFTestsUtilities::CollapseNodes(AnimNextModule, Nodes);
				ASSERT_THAT(IsNotNull(CollapseNode));
			})
			.Until(TEXT("Execute Undo Collpase Nodes"), [&]()
			{
				return GEditor->UndoTransaction();
			})
			.Then(TEXT("Assert Undo Collapse Nodes"), [&]()
			{
				ASSERT_THAT(IsNotNull(ControllerGraph->FindNodeByName(FName(RunGraphNodeName))));
				ASSERT_THAT(IsNotNull(ControllerGraph->FindNodeByName(FName(WritePoseNodeName))));
				ASSERT_THAT(IsNull(ControllerGraph->FindNodeByName(FName(CollapseNodeName))));
			})
			.Until(TEXT("Execute Redo Collapse Nodes"), [&]()
			{
				return GEditor->RedoTransaction();
			})
			.Then(TEXT("Assert Redo Collapse Nodes"), [&]()
			{
				ASSERT_THAT(IsNull(ControllerGraph->FindNodeByName(FName(RunGraphNodeName))));
				ASSERT_THAT(IsNull(ControllerGraph->FindNodeByName(FName(WritePoseNodeName))));
				ASSERT_THAT(IsNotNull(ControllerGraph->FindNodeByName(FName(CollapseNodeName))));
			});
	}
	
protected:
	void CreateAnimNextModule()
	{
		UFactory* Factory = NewObject<UUAFSystemFactory>();
		AnimNextModulePackage = CreateUniquePackage("NewAnimNextModule");
		AnimNextModulePackageName = *FPaths::GetBaseFilename(AnimNextModulePackage->GetName());		
		UObject* FactoryObject = Factory->FactoryCreateNew(UUAFSystem::StaticClass(), AnimNextModulePackage, AnimNextModulePackageName, RF_Public | RF_Standalone, NULL, GWarn);
		FAssetRegistryModule::AssetCreated(FactoryObject);
		AnimNextModule = CastChecked<UUAFSystem>(FactoryObject);
	}
	
	UPackage* CreateUniquePackage(FString InAssetName)
	{
		// Create a unique package name and path
		const FString BasePackageName = "/Game/";    // aka “/All/Content/‘
		FString UniquePackageName;
		FString AssetName;
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		AssetTools.CreateUniqueAssetName(BasePackageName, InAssetName, UniquePackageName, AssetName);
		return CreatePackage(*UniquePackageName);
	}
};

namespace UE::UAF::Tests
{

TEST_CLASS(FUAFSystemTests, "Animation.UAF.System")
{
	struct FSystemTestData
	{
		TStrongObjectPtr<UUAFSystem> System;
		UUAFSystem_EditorData* EditorData = nullptr;
		URigVMGraph* ControllerGraph = nullptr;
		UEdGraph* ParentGraph = nullptr;
		TStrongObjectPtr<UUAFComponent> Component;
	};

	TArray<FSystemTestData> SystemData;
	UEditorActorSubsystem* EditorActorSubsystem = nullptr;
	TStrongObjectPtr<USkeleton> Skeleton;
	TStrongObjectPtr<USkeletalMesh> SkeletalMesh;
	TStrongObjectPtr<UAnimSequence> AnimSequence;
	TStrongObjectPtr<UBlueprint> Blueprint;
	AActor* Actor = nullptr;

	BEFORE_EACH()
	{
		EditorActorSubsystem = GEditor->GetEditorSubsystem<UEditorActorSubsystem>();
		ASSERT_THAT(IsNotNull(EditorActorSubsystem));

		UPackage* SkeletonPackage = CreateUniquePackage("NewSkeletalMesh");
		Skeleton = TStrongObjectPtr(NewObject<USkeleton>(SkeletonPackage, FName(), RF_Public | RF_Standalone | RF_Transient));
		{
			FReferenceSkeletonModifier SkeletonModifier(Skeleton.Get());

			// Add a root bone
			SkeletonModifier.Add(FMeshBoneInfo(NAME_Root, FString(TEXT("Root")), -1), FTransform::Identity);
		}

		UPackage* SkeletalMeshPackage = CreateUniquePackage("NewSkeletalMesh");
		SkeletalMesh = TStrongObjectPtr(NewObject<USkeletalMesh>(SkeletalMeshPackage, FName(), RF_Public | RF_Standalone | RF_Transient));
		SkeletalMesh->SetSkeleton(Skeleton.Get());

		UPackage* AnimSequencePackage = CreateUniquePackage("NewAnimSequence");
		AnimSequence = TStrongObjectPtr(NewObject<UAnimSequence>(AnimSequencePackage, FName(), RF_Public | RF_Standalone | RF_Transient));
		AnimSequence->SetSkeleton(Skeleton.Get());

		const int32 TestNumFrames = 30;
		const FFrameRate TestFps = FFrameRate(30, 1);

		TScriptInterface<IAnimationDataController> Controller = AnimSequence->GetDataModel()->GetController();
		Controller->OpenBracket(NSLOCTEXT("UAFSystemTests", "CreateAnimSequence", "Creating Animation Sequence"), false);
		Controller->InitializeModel();
		Controller->SetFrameRate(TestFps, false);
		Controller->SetNumberOfFrames(TestNumFrames, false);
		Controller->CloseBracket(false);
		AnimSequence->WaitOnExistingCompression();
	}

	AFTER_EACH()
	{
		for(const FSystemTestData& Data : SystemData)
		{
			if (Data.System)
			{
				ObjectTools::DeleteSingleObject(Data.System.Get());
			}
		}
		SystemData.Empty();
		EditorActorSubsystem->DestroyActor(Actor);
		EditorActorSubsystem = nullptr;
		Skeleton = nullptr;
		SkeletalMesh = nullptr;
		Blueprint = nullptr;
		Actor = nullptr;
		
		FUtils::CleanupAfterTests();
	}

	TEST_METHOD(System_Input_Output)
	{
		TestCommandBuilder
			.Do(TEXT("Create UAF System Assets"), [&]()
			{
				// First system uses an asset, so we dont need to m,ess with its graph
				{
					FSystemTestData& System0 = SystemData.AddDefaulted_GetRef();
				}

				// Second system writes input to output
				{
					FSystemTestData& System1 = CreateSystem();
					TArray<UEdGraphPin*> FromPins;

					UAnimNextVariableEntry* VariableEntry = UAFTestsUtilities::AddVariable(System1.System.Get(), FAnimNextParamType::GetType<FUAFValueBundle>(), TEXT("InputValues"), TEXT(""));
					ASSERT_THAT(IsNotNull(VariableEntry));

					UEdGraphNode* VariableNode = UAFTestsUtilities::AddVariableNode(System1.ParentGraph, System1.System.Get(), TEXT("InputValues"), FAnimNextParamType::GetType<FUAFValueBundle>(), FAnimNextSchemaAction_Variable::EVariableAccessorChoice::Get, FromPins, FVector2f(64.0f, 16.0f));
					ASSERT_THAT(IsNotNull(VariableNode));

					UEdGraphNode* WritePoseNode = UAFTestsUtilities::AddUnitNode(System1.ParentGraph, FRigUnit_UAFWriteSystemOutput::StaticStruct()->GetPathName(), FromPins, FVector2f(336.0f, 0.0f));
					ASSERT_THAT(IsNotNull(WritePoseNode));

					// Link PrePhysics to Output
					ASSERT_THAT(IsTrue(UAFTestsUtilities::AddLink(System1.System.Get(), "RigUnit_AnimNextPrePhysicsEvent.ExecuteContext", "UAFWriteSystemOutput.ExecuteContext")));

					// Link input var to output
					ASSERT_THAT(IsTrue(UAFTestsUtilities::AddLink(System1.System.Get(), "VariableNode.Value", "UAFWriteSystemOutput.Value")));
				}
			})
			.Do(TEXT("Create Character Blueprint Asset"), [&]()
			{
				UBlueprintFactory* Factory = NewObject<UBlueprintFactory>();
				Factory->ParentClass = ACharacter::StaticClass();
		
				const FString BlueprintName = "NewBlueprint";
				UObject* FactoryObject = UAFTestsUtilities::CreateFactoryObject(Factory, UBlueprint::StaticClass(), BlueprintName);
				Blueprint = TStrongObjectPtr(CastChecked<UBlueprint>(FactoryObject));
				ASSERT_THAT(IsNotNull(Blueprint));
				ASSERT_THAT(AreEqual(BlueprintName, Blueprint->GetFName()));
			})
			.Then(TEXT("Create Blueprint"), [&]()
			{
				TArray<UActorComponent*> Components;
				{
					FSystemTestData& System0 = SystemData[0];
					const FName ComponentName = FName(TEXT("UAF0"));
					System0.Component = TStrongObjectPtr(NewObject<UUAFComponent>(GetTransientPackage(), UUAFComponent::StaticClass(), ComponentName));
					ASSERT_THAT(IsNotNull(System0.Component));
					ASSERT_THAT(AreEqual(ComponentName, System0.Component->GetFName()));

					// First component just plays a simple animation
					System0.Component->AssetData = TInstancedStruct<FUAFGraphFactoryAsset_Animation>::Make(AnimSequence.Get());

					Components.Add(System0.Component.Get());
				}

				{
					FSystemTestData& System1 = SystemData[1];
					const FName ComponentName = FName(TEXT("UAF1"));
					System1.Component = TStrongObjectPtr(NewObject<UUAFComponent>(GetTransientPackage(), UUAFComponent::StaticClass(), ComponentName));
					ASSERT_THAT(IsNotNull(System1.Component));
					ASSERT_THAT(AreEqual(ComponentName, System1.Component->GetFName()));

					// Tag component so we can find it later
					System1.Component->ComponentTags.Add(TEXT("Output"));

					// Second component uses our system
					System1.Component->AssetData = TInstancedStruct<FUAFSystemFactoryAsset_System>::Make(System1.System.Get());

					// Set UAF0 as an input to this component
					FUAFComponentInputDesc InputDesc;
					InputDesc.Component.PathToComponent = TEXT("UAF0");
					InputDesc.Input = FAnimNextVariableReference::FromName(TEXT("InputValues"), System1.System.Get());
					System1.Component->Inputs.Add(InputDesc);

					// Set output to the character's mesh component
					System1.Component->OutputComponent.PathToComponent = TEXT("Mesh");
					Components.Add(System1.Component.Get());
				}

				FKismetEditorUtilities::AddComponentsToBlueprint(Blueprint.Get(), Components);
			})
			.Then(TEXT("Compile Blueprint"), [&]()
			{
				auto CompileBlueprint = [this]()
				{
					FCompilerResultsLog LogResults;
					LogResults.SetSourcePath(Blueprint->GetPathName());
					FKismetEditorUtilities::CompileBlueprint(Blueprint.Get(), EBlueprintCompileOptions::None, &LogResults);
				};

				// Perform initial compile to generate the BP CDO
				CompileBlueprint();
				ASSERT_THAT(IsTrue(Blueprint->Status == EBlueprintStatus::BS_UpToDate));
				ASSERT_THAT(IsNotNull(Blueprint->GeneratedClass));

				// Modify the mesh component to use the correct mesh and recompile
				ACharacter* CDO = Cast<ACharacter>(Blueprint->GeneratedClass->GetDefaultObject());
				ASSERT_THAT(IsNotNull(CDO));
				USkeletalMeshComponent* MeshComponent = CDO->GetMesh();
				ASSERT_THAT(IsNotNull(MeshComponent));
				MeshComponent->SetEnableAnimation(false);
				MeshComponent->SetSkeletalMesh(SkeletalMesh.Get(), false);
				CompileBlueprint();
				ASSERT_THAT(IsTrue(Blueprint->Status == EBlueprintStatus::BS_UpToDate));
			})
			.Then(TEXT("Spawn BP actor in editor level"), [&]()
			{
				// Our test mesh is not really valid for use, so suppress some warnings here
				TestRunner->AddExpectedError(TEXT("Could not write to skeletal mesh component"), EAutomationExpectedErrorFlags::Contains, 4);
				TestRunner->AddExpectedError(TEXT("Error generating CanonicalBoneSet for SkeletalMesh"));
				TestRunner->AddExpectedError(TEXT("Skeletal Mesh asset '")); // SkeletalMesh_0' has no render data

				Actor = EditorActorSubsystem->SpawnActorFromClass(Blueprint->GeneratedClass.Get(), FVector::ZeroVector, FRotator::ZeroRotator, true);
				ASSERT_THAT(IsNotNull(Actor));
				UUAFComponent* OutputComponent = Actor->FindComponentByTag<UUAFComponent>(TEXT("Output"));
				ASSERT_THAT(IsNotNull(OutputComponent));
				uint32 SerialNumber = 0;
				OutputComponent->GetSystemReference().ReadComponent<FUAFSystemOutputComponent>([&](TConstStructView<FUAFSystemOutputComponent> InSystemOutputComponent)
				{
					SerialNumber = InSystemOutputComponent.Get().SerialNumber;
				});
				ASSERT_THAT(IsTrue(SerialNumber != 0));
			});
	}

protected:
	FSystemTestData& CreateSystem()
	{
		UFactory* Factory = NewObject<UUAFSystemFactory>();
		UPackage* SystemPackage = CreateUniquePackage("NewUAFSystem");
		FName SystemPackageName = *FPaths::GetBaseFilename(SystemPackage->GetName());
		UObject* FactoryObject = Factory->FactoryCreateNew(UUAFSystem::StaticClass(), SystemPackage, SystemPackageName, RF_Public | RF_Standalone, NULL, GWarn);
		FAssetRegistryModule::AssetCreated(FactoryObject);

		FSystemTestData& TestData = SystemData.Emplace_GetRef();
		TestData.System = TStrongObjectPtr(CastChecked<UUAFSystem>(FactoryObject));
		TestData.EditorData = CastChecked<UUAFSystem_EditorData>(UE::UAF::UncookedOnly::FUtils::GetEditorData(TestData.System.Get()));
		TestData.ControllerGraph = TestData.EditorData->GetControllerByName("RigVMGraph")->GetGraph();
		TestData.ParentGraph = Cast<UEdGraph>(TestData.EditorData->GetEditorObjectForRigVMGraph(TestData.ControllerGraph));
		return TestData;
	}

	UPackage* CreateUniquePackage(FString InAssetName)
	{
		// Create a unique package name and path
		const FString BasePackageName = "/Game/";    // aka “/All/Content/‘
		FString UniquePackageName;
		FString AssetName;
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		AssetTools.CreateUniqueAssetName(BasePackageName, InAssetName, UniquePackageName, AssetName);
		return CreatePackage(*UniquePackageName);
	}
};

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSystemReference_Creation, "Animation.UAF.Runtime.SystemReference.Creation", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FSystemReference_Creation::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	ON_SCOPE_EXIT{ FUtils::CleanupAfterTests(); };

	UFactory* Factory = NewObject<UFactory>(GetTransientPackage(), UUAFSystemFactory::StaticClass());
	UUAFSystem* Asset = CastChecked<UUAFSystem>(Factory->FactoryCreateNew(UUAFSystem::StaticClass(), GetTransientPackage(), TEXT("TestAsset"), RF_Transient, nullptr, nullptr, NAME_None));
	UE_RETURN_ON_ERROR(Asset != nullptr, "FSystemReference_Creation -> Failed to create asset");

	TArray<FString> Messages;
	FRigVMRuntimeSettings RuntimeSettings;
	RuntimeSettings.SetLogFunction([&Messages](const FRigVMLogSettings& InLogSettings, const FRigVMExecuteContext* InContext, const FString& Message)
	{
		Messages.Add(Message);
	});
	Asset->GetRigVMExtendedExecuteContext().SetRuntimeSettings(RuntimeSettings);

	// Needs a world context object, otherwise tick function creation fails.
	FSystemReference SystemRef(FUAFSystemFactoryAsset_System(Asset), FUtils::GetWorld(), EAnimNextModuleInitMethod::None);
	UE_RETURN_ON_ERROR(SystemRef.IsValid(), "FSystemReference_Creation -> Failed to create System");

	const FTickFunction* TickFunction = SystemRef.FindTickFunction(FRigUnit_AnimNextPrePhysicsEvent::DefaultEventName);
	UE_RETURN_ON_ERROR(TickFunction != nullptr, "FSystemReference_Creation -> Failed to find tick functions.");

	SystemRef.RunEvent(FRigUnit_AnimNextPrePhysicsEvent::DefaultEventName, 0.0f);
	UE_RETURN_ON_ERROR(Messages.IsEmpty(), "FSystemReference_Creation -> Unexpected number of messages.");

	return true;
}

}

#endif // WITH_EDITOR
#endif // WITH_DEV_AUTOMATION_TESTS
