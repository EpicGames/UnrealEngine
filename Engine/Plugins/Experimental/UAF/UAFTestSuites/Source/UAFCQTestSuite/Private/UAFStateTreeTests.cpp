// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS
#if WITH_EDITOR

#include "UAFTestsUtilities.h"
#include "CQTest.h"
#include "Editor.h"
#include "ScopedTransaction.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "AnimNextStateTree.h"
#include "AnimNextStateTreeFactory.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "Math/Vector2D.h"

TEST_CLASS(FAnimNextStateTreeTests, "Animation.UAF.Functional")
{
	UAnimNextStateTree* AnimNextStateTree = nullptr;
	UAssetEditorSubsystem* AssetEditorSubsystem = nullptr;
	UUAFRigVMAssetEditorData* EditorData = nullptr;
	URigVMLibraryNode* LibraryNode = nullptr;
	const FString FunctionName = "NewFunction";
	
	BEFORE_EACH()
	{
		AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		ASSERT_THAT(IsNotNull(AssetEditorSubsystem));
	}
	
	// QMetry Test Case:  UE-TC-18830
	TEST_METHOD(Undo_Redo_AnimNextStateTree_Create_Function_Node)
	{		
		TestCommandBuilder
			.Do(TEXT("Create AnimNext State Tree Asset"), [&]()
			{
				CreateAnimNextStateTree();
			})
			.Then(TEXT("Create Function Node"), [&]()
			{
				LibraryNode = UAFTestsUtilities::AddFunctionNode(AnimNextStateTree);
				ASSERT_THAT(IsNotNull(LibraryNode));
				ASSERT_THAT(AreEqual(FunctionName, LibraryNode->GetNodeTitle()));		
			})
			.Then(TEXT("Assert Function Node Creation"), [&]()
			{
				EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(AnimNextStateTree);	
				URigVMController* Controller = CastChecked<IRigVMClientHost>(EditorData)->GetControllerByName(FunctionName);
				ASSERT_THAT(IsNotNull(Controller));
				ASSERT_THAT(IsNotNull(Controller->GetGraph()));				
			})
			.Until(TEXT("Execute Undo"), [&]()
			{
				return GEditor->UndoTransaction();
			})	
			.Then(TEXT("Assert Undo Function Node"), [&]()
			{
				URigVMController* Controller = CastChecked<IRigVMClientHost>(EditorData)->GetControllerByName(FunctionName);	
				ASSERT_THAT(IsNull(Controller));
			})
			.Until(TEXT("Execute Redo"), [&]()
			{
				return GEditor->RedoTransaction();
			})
			.Then(TEXT("Assert Redo Function Node"), [&]()
			{
				URigVMController* Controller = CastChecked<IRigVMClientHost>(EditorData)->GetControllerByName(FunctionName);	
				ASSERT_THAT(IsNotNull(Controller));
				ASSERT_THAT(IsNotNull(Controller->GetGraph()));
			});
	}	
	
protected:
	void CreateAnimNextStateTree()
	{
		const TSubclassOf<UAnimNextStateTreeFactory> StateTreeFactoryClass = UAnimNextStateTreeFactory::StaticClass();
		UObject* FactoryObject = UAFTestsUtilities::CreateFactoryObject(NewObject<UAnimNextStateTreeFactory>(GetTransientPackage(), StateTreeFactoryClass.Get()), UAnimNextStateTree::StaticClass(), "NewAnimNextStateTree");
		AnimNextStateTree = CastChecked<UAnimNextStateTree>(FactoryObject);
		ASSERT_THAT(IsNotNull(AnimNextStateTree));
		ASSERT_THAT(IsTrue(AssetEditorSubsystem->OpenEditorForAsset(AnimNextStateTree)));
	}	
};

#endif // WITH_EDITOR
#endif // WITH_DEV_AUTOMATION_TESTS
