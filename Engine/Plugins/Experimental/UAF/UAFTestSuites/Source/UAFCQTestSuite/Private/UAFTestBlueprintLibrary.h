// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "AnimNextRigVMFunctionData.h"
#include "AutomationTestModule.h"
#include "UAFCompilationScope.h"
#include "Factories/Factory.h"
#include "Module/AnimNextModule_EditorData.h"
#include "UObject/Class.h"
#include "UAFTestBlueprintLibrary.generated.h"

UCLASS()
class UUAFTestBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
	
public:
		
	UFUNCTION(BlueprintCallable, Category = "UAF|Test", meta=(ScriptMethod))
	static void RecompileVM(UUAFRigVMAsset* InAsset) 
	{	
		if (InAsset)
		{
			UE::UAF::UncookedOnly::Compilation::RequestAssetCompilation(InAsset);
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "Invalid Asset in UUAFTestBlueprintLibrary::RecompileVM")
		}
	}

	UFUNCTION(BlueprintCallable, Category = "UAF|Test", meta=(ScriptMethod))
	static URigVMGraph* GetModel(UUAFRigVMAsset* InAsset, const UEdGraph* InEdGraph = nullptr) 
	{
		if (InAsset)
		{
			return UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset)->GetModel(); 
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "Invalid Asset in UUAFTestBlueprintLibrary::GetModel")
		}
		
		return nullptr;
	}

	UFUNCTION(BlueprintCallable, Category = "UAF|Test", meta=(ScriptMethod))
	static URigVMGraph* GetDefaultModel(UUAFRigVMAsset* InAsset)
	{
		if (InAsset)
		{
			return UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset)->GetDefaultModel(); 
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "Invalid Asset in UUAFTestBlueprintLibrary::GetDefaultModel")
		}
		
		return nullptr;
	}

	UFUNCTION(BlueprintCallable, Category = "UAF|Test", meta=(ScriptMethod))
	static TArray<URigVMGraph*> GetAllModels(UUAFRigVMAsset* InAsset)
	{
		if (InAsset)
		{
			return UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset)->GetAllModels(); 
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "Invalid Asset in UUAFTestBlueprintLibrary::GetAllModels")
		}

		return TArray<URigVMGraph*>();
	}

	UFUNCTION(BlueprintCallable, Category = "UAF|Test", meta=(ScriptMethod))
	static URigVMFunctionLibrary* GetLocalFunctionLibrary(UUAFRigVMAsset* InAsset)
	{
		if (InAsset)
		{
			return UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset)->GetLocalFunctionLibrary(); 
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "Invalid Asset in UUAFTestBlueprintLibrary::GetLocalFunctionLibrary")
		}

		return nullptr;
	}

	UFUNCTION(BlueprintCallable, Category = "UAF|Test", meta=(ScriptMethod))
	static URigVMFunctionLibrary* GetOrCreateLocalFunctionLibrary(UUAFRigVMAsset* InAsset, bool bSetupUndoRedo = true)
	{
		if (InAsset)
		{
			return UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset)->GetOrCreateLocalFunctionLibrary(bSetupUndoRedo); 
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "Invalid Asset in UUAFTestBlueprintLibrary::GetOrCreateLocalFunctionLibrary")
		}

		return nullptr;
	}

	UFUNCTION(BlueprintCallable, Category = "UAF|Test", meta=(ScriptMethod))
	static URigVMGraph* AddModel(UUAFRigVMAsset* InAsset, FString InName = TEXT("Rig Graph"), bool bSetupUndoRedo = true, bool bPrintPythonCommand = true)
	{
		if (InAsset)
		{
			return UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset)->AddModel(InName, bSetupUndoRedo, bPrintPythonCommand); 
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "Invalid Asset in UUAFTestBlueprintLibrary::AddModel")
		}

		return nullptr;
	}

	UFUNCTION(BlueprintCallable, Category = "UAF|Test", meta=(ScriptMethod))
	static bool RemoveModel(UUAFRigVMAsset* InAsset, FString InName = TEXT("Rig Graph"), bool bSetupUndoRedo = true, bool bPrintPythonCommand = true)
	{
		if (InAsset)
		{
			return UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset)->RemoveModel(InName, bSetupUndoRedo, bPrintPythonCommand); 
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "Invalid Asset in UUAFTestBlueprintLibrary::RemoveModel")
		}

		return false;
	}

	UFUNCTION(BlueprintCallable, Category = "UAF|Test", meta=(ScriptMethod))
	static URigVMController* GetController(UUAFRigVMAsset* InAsset, const URigVMGraph* InGraph = nullptr)
	{
		if (InAsset)
		{
			return UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset)->GetController(InGraph); 
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "Invalid Asset in UUAFTestBlueprintLibrary::GetController")
		}

		return nullptr;
	}

	UFUNCTION(BlueprintCallable, Category = "UAF|Test", meta=(ScriptMethod))
	static URigVMController* GetControllerByName(UUAFRigVMAsset* InAsset, const FString InGraphName = TEXT(""))
	{
		if (InAsset)
		{
			return UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset)->GetControllerByName(InGraphName); 
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "Invalid Asset in UUAFTestBlueprintLibrary::GetControllerByName")
		}

		return nullptr;
	}

	UFUNCTION(BlueprintCallable, Category = "UAF|Test", meta=(ScriptMethod))
	static URigVMController* GetOrCreateController(UUAFRigVMAsset* InAsset, URigVMGraph* InGraph = nullptr)
	{
		if (InAsset)
		{
			return UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset)->GetController(InGraph); 
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "Invalid Asset in UUAFTestBlueprintLibrary::GetOrCreateController")
		}

		return nullptr;
	}
	
	UFUNCTION(BlueprintCallable, Category="UAFTest", meta=(ScriptMethod))
	static void ExecuteVM(UUAFSystem* InModule, const FName& InEventName, bool& bExecutionResult, TArray<FString>& OutMessages)
	{
		bExecutionResult = false;
		
		if (InModule == nullptr)
		{
			UE_LOGF(LogAnimation, Warning, "Invalid Asset in UUAFTestBlueprintLibrary::ExecuteVM")
			return;
		}
	
		if (!InModule->SupportsEvent(InEventName))
		{
			UE_LOGF(LogAnimation, Warning, "%ls does not support event named %ls", *InModule->GetName(), *InEventName.ToString());
			return;
		}
	
		if (InModule->GetVM() == nullptr)
		{
			UE_LOGF(LogAnimation, Warning, "%ls does not contain a valid VM", *InModule->GetName());
			return;
		}
	
		if (UUAFSystem_EditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFSystem_EditorData>(InModule))
		{
			if (EditorData->bVMRecompilationRequired)
			{
				UE_LOGF(LogAnimation, Warning, "%ls requires compilation - so invoking now", *InModule->GetName());
				EditorData->RecompileVM();
			}
		
			if (InModule->CompilationState == EAnimNextRigVMAssetState::CompiledWithErrors)
			{
				UE_LOGF(LogAnimation, Warning, "%ls compilation finished with errors", *InModule->GetName());
				return;
			}
		}
	
		FRigVMRuntimeSettings RuntimeSettings;
		RuntimeSettings.SetLogFunction([&OutMessages, ModuleName = InModule->GetName()](const FRigVMLogSettings& InLogSettings, const FRigVMExecuteContext* InContext, const FString& Message)
		{
			OutMessages.Add(Message);
		});
		InModule->GetRigVMExtendedExecuteContext().SetRuntimeSettings(RuntimeSettings);

		ERigVMExecuteResult Result = InModule->GetVM()->ExecuteVM(InModule->GetRigVMExtendedExecuteContext(), InEventName);
	
		InModule->GetRigVMExtendedExecuteContext().SetRuntimeSettings(FRigVMRuntimeSettings());

		bExecutionResult = Result == ERigVMExecuteResult::Succeeded;
	}
	
	UFUNCTION(BlueprintCallable, Category="UAFTest")
	static UUAFRigVMAsset* CreateAsset(TSubclassOf<UFactory> InAssetFactoryClass, const FName& InName)
	{
		if(InAssetFactoryClass.Get() == nullptr)
        {
			UE_LOGF(LogAnimation, Warning, "Invalid InAssetFactoryClass in UUAFTestBlueprintLibrary::CreateAsset");
			return nullptr;
		}

		UFactory* Factory = NewObject<UFactory>(GetTransientPackage(), InAssetFactoryClass.Get());
		if(Factory == nullptr)
        {
			UE_LOGF(LogAnimation, Warning, "Failed to create Factory of class %ls", *InAssetFactoryClass.Get()->GetName());
			return nullptr;
        }
		
		if (!Factory->GetSupportedClass()->IsChildOf(UUAFRigVMAsset::StaticClass()))
		{
			UE_LOGF(LogAnimation, Warning, "Factory %ls provided to UUAFTestBlueprintLibrary::CreateAsset does not support a UUAFRigVMAsset-based asset", *InAssetFactoryClass.Get()->GetName());
			return nullptr;
		}
		
		UUAFRigVMAsset* Asset = Cast<UUAFRigVMAsset>(Factory->FactoryCreateNew(Factory->GetSupportedClass(), GetTransientPackage(), InName, RF_Transient, nullptr, nullptr, NAME_None));
		
		return Asset;
	}
	
	// Test helper: mark a function as public so it gets compiled into FunctionData
	static void MarkFunctionAsPublic(UUAFRigVMAsset* InAsset, FName InFunctionName)
	{
		if (InAsset)
		{
			UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset);
			FRigVMGraphFunctionData* FuncData = EditorData->GraphFunctionStore.FindFunctionByName(InFunctionName);
			if (FuncData)
			{
				EditorData->GraphFunctionStore.MarkFunctionAsPublic(FuncData->Header.LibraryPointer, true);
			}
		}
	}

	// Test helper: get the FunctionGuid from a function's compiled FunctionData
	static FGuid GetFunctionGuid(const UUAFRigVMAsset* InAsset, FName InFunctionName)
	{
		if (InAsset)
		{
			FName EventName = FName(TEXT("__InternalCall_") + InFunctionName.ToString());
			for (const FAnimNextRigVMFunctionData& Data : InAsset->FunctionData)
			{
				if (Data.EventName == EventName)
				{
					return Data.FunctionGuid;
				}
			}
		}
		return FGuid();
	}

	// Test helper: register a function in the asset's FunctionData array so it can be called
	// via GetFunctionHandle/ExecuteParameterlessFunction. This is needed because
	// BuildFunctionWrapperEvents only processes programmatic function headers (from traits),
	// not user-created graph functions.
	static void AddTestFunctionData(UUAFRigVMAsset* InAsset, FName InFunctionName, FName InEventName, TArray<int32> InArgIndices)
	{
		if (InAsset)
		{
			FAnimNextRigVMFunctionData NewData;
			NewData.Name = InFunctionName;
			NewData.EventName = InEventName;
			NewData.ArgIndices = MoveTemp(InArgIndices);
			InAsset->FunctionData.Add(MoveTemp(NewData));
		}
	}

	UFUNCTION(BlueprintPure, Category="UAFTest", meta=(ScriptMethod))
	static EAnimNextRigVMAssetState GetCompilationState(const UUAFRigVMAsset* InAsset)
	{
#if WITH_EDITORONLY_DATA
		if (InAsset)
		{
			return InAsset->CompilationState;
		}
#endif // WITH_EDITORONLY_DATA
		
		return EAnimNextRigVMAssetState::Invalid;
	}
};
