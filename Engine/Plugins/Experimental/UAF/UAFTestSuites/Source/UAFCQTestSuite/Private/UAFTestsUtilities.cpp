// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS
#if WITH_EDITOR

#include "UAFTestsUtilities.h"

namespace UAFTestsUtilities
{
	UObject* CreateFactoryObject(UFactory* InFactory, UClass* InClass, const FString& InPackageName)
	{
		if (!InClass || !InFactory)
		{
			UE_LOGF(LogTemp, Error, "Invalid pointer instance provided.");
			return nullptr;
		}
		
		UPackage* NewPackage = CreatePackage(nullptr);
		if (!NewPackage)
		{
			UE_LOGF(LogTemp, Error, "Unable to create UPackage instance.");
			return nullptr;			
		}
		
		FName NewPackageName = InPackageName.Equals("") ? *FPaths::GetBaseFilename(NewPackage->GetName()) : FName(InPackageName);
		
		return InFactory->FactoryCreateNew(InClass, NewPackage, NewPackageName, RF_Public | RF_Standalone, NULL, GWarn);
	}

	UEdGraphNode* AddUnitNode(UEdGraph* InParentGraph, const FString& InScriptStructPath, TArray<UEdGraphPin*>& InFromPins, const FVector2f& InRigUnitLocation, const FString& InUnitNodeClassPath)
	{		
		if (!InParentGraph)
		{
			UE_LOGF(LogTemp, Error, "Invalid UEdGraph instance provided.");
			return nullptr;
		}
		
		UScriptStruct* ScriptStruct = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UScriptStruct>(InScriptStructPath);
		FText Category = NSLOCTEXT("Category", "Category", "Category");
		FText MenuDesc = NSLOCTEXT("Menu_Desc", "MenuDesc", "Menu Desc");
		FText ToolTip = NSLOCTEXT("Tool_Tip", "ToolTip", "Tool Tip");
		
		UClass* UnitNodeClass = nullptr;
		if (InUnitNodeClassPath.IsEmpty())
		{
			UnitNodeClass = URigVMUnitNode::StaticClass();
		}
		else
		{
			UnitNodeClass = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UClass>(InUnitNodeClassPath);
		}
		
		TSharedPtr<FAnimNextSchemaAction_RigUnit> RigUnitAction = MakeShared<FAnimNextSchemaAction_RigUnit>(UnitNodeClass,
																											ScriptStruct,
																											Category,
																											MenuDesc,
																											ToolTip);

		return RigUnitAction->PerformAction(InParentGraph, InFromPins, InRigUnitLocation);
	}

	URigVMLibraryNode* AddFunctionNode(UUAFRigVMAsset* InAnimNextAsset, const FString& InFunctionName)
	{
		if (!InAnimNextAsset)
		{
			UE_LOGF(LogTemp, Error, "Invalid UUAFRigVMAsset instance provided.");
			return nullptr;
		}
		
		UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(InAnimNextAsset);
		return EditorData->AddFunction(FName(InFunctionName), true);
	}

	UAnimNextVariableEntry* AddVariable(UUAFRigVMAsset* InAnimNextAsset, const FAnimNextParamType& InVariableType, const FString& InVariableName, const FString& InDefaultValue)
	{
		if (!InAnimNextAsset)
		{
			UE_LOGF(LogTemp, Error, "Invalid UUAFRigVMAsset instance provided.");
			return nullptr;
		}
		
		UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(InAnimNextAsset);
		return EditorData->AddVariable(FName(InVariableName), InVariableType, InDefaultValue);
	}

	UUAFSharedVariablesEntry* AddSharedVariables(UUAFRigVMAsset* InAnimNextAsset, UUAFSharedVariables* InSharedVariables)
	{
		if (!InAnimNextAsset)
		{
			UE_LOGF(LogTemp, Error, "Invalid UUAFRigVMAsset instance provided.");
			return nullptr;
		}

		if (!InSharedVariables)
		{
			UE_LOGF(LogTemp, Error, "Invalid UAnimNextSharedVariables instance provided.");
			return nullptr;
		}
		
		UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(InAnimNextAsset);
		return EditorData->AddSharedVariables(InSharedVariables);
	}

	UEdGraphNode* AddVariableNode(UEdGraph* InParentGraph, const UObject* InSourceObject, const FString& InVariableName, const FAnimNextParamType& InVariableType, const FAnimNextSchemaAction_Variable::EVariableAccessorChoice InVariableAccessorChoice, TArray<UEdGraphPin*>& InFromPins, const FVector2f& InVariableLocation)
	{
		if (!InParentGraph)
		{
			UE_LOGF(LogTemp, Error, "Invalid UEdGraph instance provided.");
			return nullptr;
		}
				
		TSharedPtr<FAnimNextSchemaAction_Variable> VariableAction = MakeShared<FAnimNextSchemaAction_Variable>(FName(InVariableName), InSourceObject, InVariableType, InVariableAccessorChoice);
		return VariableAction->PerformAction(InParentGraph, InFromPins, InVariableLocation);
	}

	URigVMPin* AddPin(UUAFRigVMAsset* InAnimNextAsset, URigVMLibraryNode* InLibraryNode, ERigVMPinDirection InDirection, const FString& InCPPName, const FString& InCPPType, const FName& InCPPTypeObjectPath, const FString& InDefaultValue)
	{
		if (!InAnimNextAsset || !InLibraryNode)
		{
			UE_LOGF(LogTemp, Error, "Invalid pointer instance provided.");
			return nullptr;
		}

		UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(InAnimNextAsset);
		URigVMGraph* FunctionGraph = InLibraryNode->GetContainedGraph();
		URigVMController* Controller = CastChecked<IRigVMClientHost>(EditorData)->GetController(FunctionGraph);		
		const FName NewPinName = Controller->AddExposedPin(FName(InCPPName), InDirection, InCPPType, InCPPTypeObjectPath, InDefaultValue);
		
		URigVMPin* NewPin;
		switch (InDirection)
		{
			case ERigVMPinDirection::Input:
				NewPin = InLibraryNode->GetEntryNode()->FindPin(NewPinName.ToString());			
				break;
			case ERigVMPinDirection::Output:
				NewPin = InLibraryNode->GetReturnNode()->FindPin(NewPinName.ToString());
				break;
			default:
				NewPin = nullptr;
				break;
		}
		
		return NewPin;
	}

	bool AddLink(UUAFRigVMAsset* InAnimNextAsset, const FString& InOutputPinPath, const FString& InInputPinPath)
	{
		if (!InAnimNextAsset)
		{
			UE_LOGF(LogTemp, Error, "Invalid pointer instance provided.");
			return false;
		}
		
		UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(InAnimNextAsset);
		URigVMController* Controller = CastChecked<IRigVMClientHost>(EditorData)->GetControllerByName("RigVMGraph");
		return Controller->AddLink(InOutputPinPath, InInputPinPath);
	}

	bool SetNodeSelection(UUAFRigVMAsset* InAnimNextAsset, const TArray<FName>& InNodeNames)
	{
		if (!InAnimNextAsset)
		{
			UE_LOGF(LogTemp, Error, "Invalid pointer instance provided.");
			return false;
		}
		
		UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(InAnimNextAsset);
		URigVMController* Controller = CastChecked<IRigVMClientHost>(EditorData)->GetControllerByName("RigVMGraph");
		return Controller->SetNodeSelection(InNodeNames);
	}

	URigVMCollapseNode* CollapseNodes(UUAFRigVMAsset* InAnimNextAsset, const TArray<FName>& InNodeNames, const FString& InCollapseNodeName, bool InCollapseToFunction)
	{
		if (SetNodeSelection(InAnimNextAsset, InNodeNames))
		{
			UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(InAnimNextAsset);
			URigVMController* Controller = CastChecked<IRigVMClientHost>(EditorData)->GetControllerByName("RigVMGraph");			
			Controller->OpenUndoBracket(TEXT("Collapse Node"));
			URigVMCollapseNode* CollapseNode = Controller->CollapseNodes(InNodeNames, InCollapseNodeName);
			if (InCollapseToFunction)
			{
				Controller->PromoteCollapseNodeToFunctionReferenceNode(CollapseNode->GetFName());
			}
			Controller->CloseUndoBracket();
			return CollapseNode;
		}
		
		return nullptr;
	}
}

#endif
#endif
