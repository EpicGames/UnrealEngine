// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS
#if WITH_EDITOR

#include "UAFTestsUtilities.h"
#include "CQTest.h"
#include "Editor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextAnimationGraphFactory.h"
#include "Module/AnimNextModule.h"
#include "Module/AnimNextModuleFactory.h"
#include "Variables/AnimNextSharedVariables.h"
#include "Variables/AnimNextSharedVariablesFactory.h"
#include "AnimNextRigVMAssetEditorData.h"

TEST_CLASS(FAssetVariableTests, "Animation.UAF.Functional")
{
	UUAFRigVMAsset* AnimNextAsset = nullptr;
	UUAFSharedVariables* AnimNextSharedVariablesAsset = nullptr;
	UAnimNextVariableEntry* VariableEntry = nullptr;
	UUAFSharedVariablesEntry* SharedVariablesEntry = nullptr;
	UAssetEditorSubsystem* AssetEditorSubsystem = nullptr;
	UUAFRigVMAssetEditorData* EditorData = nullptr;

	BEFORE_EACH()
	{
		AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		ASSERT_THAT(IsNotNull(AssetEditorSubsystem));
	}

	// QMetry Test Case:  UE-TC-10712
	TEST_METHOD(AnimNextAnimationGraph_Create_Variable)
	{		
		TestCommandBuilder
			.Do(TEXT("Create AnimNext Animation Graph Asset"), [&]()
			{
				const TSubclassOf<UUAFAnimGraphFactory> AnimNextAnimGraphFactoryClass = UUAFAnimGraphFactory::StaticClass();
				UObject* factoryObject = UAFTestsUtilities::CreateFactoryObject(NewObject<UUAFAnimGraphFactory>(GetTransientPackage(), AnimNextAnimGraphFactoryClass.Get()), UUAFAnimGraph::StaticClass(), "NewUAFAnimGraph");
				AnimNextAsset = CastChecked<UUAFAnimGraph>(factoryObject);				
				ASSERT_THAT(IsTrue(AssetEditorSubsystem->OpenEditorForAsset(AnimNextAsset)));
			})
			.Then(TEXT("Create Bool Variable"), [&]()
			{
				FAnimNextParamType Type = FAnimNextParamType::GetType<bool>();
				VariableEntry = UAFTestsUtilities::AddVariable(AnimNextAsset, Type, "NewVariable", "false");
				ASSERT_THAT(IsNotNull(VariableEntry));
			});
	}
	
	// QMetry Test Case:  UE-TC-21565
	TEST_METHOD(UAFSystem_Add_UAFSharedVariables)
	{
		TestCommandBuilder
			.Do(TEXT("Create UAF Shared Variables Asset"), [&]()
			{
				const TSubclassOf<UUAFSharedVariablesFactory> SharedVariablesFactoryClass = UUAFSharedVariablesFactory::StaticClass();
				UObject* FactoryObject = UAFTestsUtilities::CreateFactoryObject(NewObject<UUAFSharedVariablesFactory>(), UUAFSharedVariables::StaticClass(), "NewAnimNextSharedVariables");
				AnimNextSharedVariablesAsset = CastChecked<UUAFSharedVariables>(FactoryObject);
				ASSERT_THAT(IsNotNull(AnimNextSharedVariablesAsset));
				ASSERT_THAT(IsTrue(AssetEditorSubsystem->OpenEditorForAsset(AnimNextSharedVariablesAsset)));		
			})
			.Then(TEXT("Add Variables to UAF Shared Variables Asset"), [&]()
			{
				UUAFRigVMAsset* Asset = CastChecked<UUAFRigVMAsset>(AnimNextSharedVariablesAsset);
				FAnimNextParamType Type = FAnimNextParamType::GetType<bool>();
				VariableEntry = UAFTestsUtilities::AddVariable(Asset, Type, "NewVariable", "false");
				ASSERT_THAT(IsNotNull(VariableEntry));
				
				Type = FAnimNextParamType::GetType<FString>();
				VariableEntry = UAFTestsUtilities::AddVariable(Asset, Type, "NewVariable", "");
				ASSERT_THAT(IsNotNull(VariableEntry));
			})
			.Then(TEXT("Create UAF System Asset"), [&]()
			{
				const TSubclassOf<UUAFSystemFactory> ModuleFactoryClass = UUAFSystemFactory::StaticClass();
				UObject* FactoryObject = UAFTestsUtilities::CreateFactoryObject(NewObject<UUAFSystemFactory>(), UUAFSystem::StaticClass(), "NewUAFSystem");				
				AnimNextAsset = CastChecked<UUAFSystem>(FactoryObject);
				ASSERT_THAT(IsNotNull(AnimNextAsset));
				ASSERT_THAT(IsTrue(AssetEditorSubsystem->OpenEditorForAsset(AnimNextAsset)));
			})		
			.Then(TEXT("Add UAF Shared Variables Asset to UAF System Asset"), [&]()
			{
				SharedVariablesEntry = UAFTestsUtilities::AddSharedVariables(AnimNextAsset, AnimNextSharedVariablesAsset);
				ASSERT_THAT(IsNotNull(SharedVariablesEntry));
			})
			.Then(TEXT("Add Variable to UAF System Asset"), [&]()
			{
				FAnimNextParamType Type = FAnimNextParamType::GetType<bool>();
				VariableEntry = UAFTestsUtilities::AddVariable(AnimNextAsset, Type, "NewVariable_1", "false");
				ASSERT_THAT(IsNotNull(VariableEntry));
			});
	}
	
	// QMetry Test Case:  UE-TC-21572
	TEST_METHOD(UAFSharedVariables_Public_Private_Access)
	{
		TestCommandBuilder
			.Do(TEXT("Create UAF System Asset"), [&]()
			{
				const TSubclassOf<UUAFSystemFactory> ModuleFactoryClass = UUAFSystemFactory::StaticClass();
				UObject* FactoryObject = UAFTestsUtilities::CreateFactoryObject(NewObject<UUAFSystemFactory>(), UUAFSystem::StaticClass(), "NewUAFSystem");				
				AnimNextSharedVariablesAsset = CastChecked<UUAFSystem>(FactoryObject);
				ASSERT_THAT(IsNotNull(AnimNextSharedVariablesAsset));
				ASSERT_THAT(IsTrue(AssetEditorSubsystem->OpenEditorForAsset(AnimNextSharedVariablesAsset)));
			})
			.Then(TEXT("Add Variable to UAF System Asset"), [&]()
			{
				FAnimNextParamType Type = FAnimNextParamType::GetType<bool>();
				VariableEntry = UAFTestsUtilities::AddVariable(AnimNextSharedVariablesAsset, Type, "NewVariable", "false");
				ASSERT_THAT(IsNotNull(VariableEntry));
				ASSERT_THAT(AreEqual(EAnimNextExportAccessSpecifier::Public, VariableEntry->Access));
			})
			.Then(TEXT("Create UAF Anim Graph Asset"), [&]()
			{
				const TSubclassOf<UUAFAnimGraphFactory> AnimGraphFactoryClass = UUAFAnimGraphFactory::StaticClass();
				UObject* FactoryObject = UAFTestsUtilities::CreateFactoryObject(NewObject<UUAFAnimGraphFactory>(), UUAFAnimGraph::StaticClass(), "NewUAFAnimGraph");
				AnimNextAsset = CastChecked<UUAFAnimGraph>(FactoryObject);
				ASSERT_THAT(IsNotNull(AnimNextAsset));
				ASSERT_THAT(IsTrue(AssetEditorSubsystem->OpenEditorForAsset(AnimNextAsset)));
			})		
			.Then(TEXT("Add UAF Shared Variables Asset to UAF System Asset"), [&]()
			{
				SharedVariablesEntry = UAFTestsUtilities::AddSharedVariables(AnimNextAsset, AnimNextSharedVariablesAsset);
				ASSERT_THAT(IsNotNull(SharedVariablesEntry));
			})
			.Then(TEXT("Add Variable to UAF System Asset"), [&]()
			{
				VariableEntry->Access = EAnimNextExportAccessSpecifier::Private;
				ASSERT_THAT(AreEqual(EAnimNextExportAccessSpecifier::Private, VariableEntry->Access));
			});
	}
	
};

#endif // WITH_EDITOR
#endif // WITH_DEV_AUTOMATION_TESTS
