// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS
#if WITH_EDITOR

#include "UAFTestsUtilities.h"
#include "CQTest.h"
#include "Editor.h"
#include "IWorkspaceEditor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "WorkspaceEditor/Private/Workspace.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "Graph/AnimNextAnimationGraphFactory.h"
#include "Module/AnimNextModule.h"
#include "Module/AnimNextModuleFactory.h"

TEST_CLASS(FUAFWorkspaceTests, "Animation.UAF.Functional")
{
	UUAFAnimGraph* AnimationGraphAsset = nullptr;
	UUAFAnimGraph* AnimationGraphAsset2 = nullptr;
	UUAFSystem* ModuleAsset = nullptr;
	UAssetEditorSubsystem* AssetEditorSubsystem = nullptr;

	BEFORE_EACH()
	{
		AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		ASSERT_THAT(IsNotNull(AssetEditorSubsystem));
	}

	// QMetry Test Case:  UE-TC-10700
	TEST_METHOD(Verify_Open_Animation_Graph_As_Root_In_Workspace)
	{
		TestCommandBuilder
			.Do(TEXT("Create Animation Graph Asset"), [&]()
			{
				const TSubclassOf<UUAFAnimGraphFactory> AnimGraphFactoryClass = UUAFAnimGraphFactory::StaticClass();
				UObject* FactoryObject = UAFTestsUtilities::CreateFactoryObject(NewObject<UUAFAnimGraphFactory>(), UUAFAnimGraph::StaticClass(), "NewAnimNextAnimationGraph");
				AnimationGraphAsset = CastChecked<UUAFAnimGraph>(FactoryObject);
				ASSERT_THAT(IsNotNull(AnimationGraphAsset));
			})
			.Then(TEXT("Open Animation Graph in UAF Workspace"), [&]()
			{
				ASSERT_THAT(IsTrue(AssetEditorSubsystem->OpenEditorForAsset(AnimationGraphAsset)));
			});
	}

	// QMetry Test Case:  UE-TC-22353
	TEST_METHOD(Verify_Cannot_Remove_Workspace_Root_Asset)
	{
		TestCommandBuilder
			.Do(TEXT("Create Animation Graph Asset"), [&]()
			{
				const TSubclassOf<UUAFAnimGraphFactory> AnimGraphFactoryClass = UUAFAnimGraphFactory::StaticClass();
				UObject* FactoryObject = UAFTestsUtilities::CreateFactoryObject(NewObject<UUAFAnimGraphFactory>(), UUAFAnimGraph::StaticClass(), "NewAnimNextAnimationGraph");
				AnimationGraphAsset = CastChecked<UUAFAnimGraph>(FactoryObject);
				ASSERT_THAT(IsNotNull(AnimationGraphAsset));
			})
			.Then(TEXT("Open Animation Graph in UAF Workspace"), [&]()
			{
				ASSERT_THAT(IsTrue(AssetEditorSubsystem->OpenEditorForAsset(AnimationGraphAsset)));
			})
			.Then(TEXT("Try remove Animation Graph from Workspace"), [&]()
			{
				IAssetEditorInstance* AssetEditor = AssetEditorSubsystem->FindEditorForAsset(AnimationGraphAsset, false);
				UE::Workspace::IWorkspaceEditor* WorkspaceEditor = static_cast<UE::Workspace::IWorkspaceEditor*>(AssetEditor);
				UWorkspace* Workspace = CastChecked<UWorkspace>(WorkspaceEditor->GetWorkspaceAsset());

				TestRunner->AddExpectedError("Cannot remove the root-level asset.", EAutomationExpectedErrorFlags::Contains, -1);
				ASSERT_THAT(IsFalse(Workspace->RemoveAsset(AnimationGraphAsset)));
			});
	}

	// QMetry Test Case:  UE-TC-10716
	TEST_METHOD(Verify_Cannot_Add_Multiple_Assets_To_Workspace)
	{
		TestCommandBuilder
			.Do(TEXT("Create Animation Graph Assets"), [&]()
			{
				const TSubclassOf<UUAFAnimGraphFactory> AnimGraphFactoryClass = UUAFAnimGraphFactory::StaticClass();
				UObject* FactoryObject = UAFTestsUtilities::CreateFactoryObject(NewObject<UUAFAnimGraphFactory>(), UUAFAnimGraph::StaticClass(), "NewAnimNextAnimationGraph");
				AnimationGraphAsset = CastChecked<UUAFAnimGraph>(FactoryObject);
				ASSERT_THAT(IsNotNull(AnimationGraphAsset));

				UObject* FactoryObject2 = UAFTestsUtilities::CreateFactoryObject(NewObject<UUAFAnimGraphFactory>(), UUAFAnimGraph::StaticClass(), "NewAnimNextAnimationGraph");
				AnimationGraphAsset2 = CastChecked<UUAFAnimGraph>(FactoryObject2);
				ASSERT_THAT(IsNotNull(AnimationGraphAsset2));
			})
			.Then(TEXT("Open Animation Graph in UAF Workspace"), [&]()
			{
				ASSERT_THAT(IsTrue(AssetEditorSubsystem->OpenEditorForAsset(AnimationGraphAsset)));
			})
			.Then(TEXT("Try add second Animation Graph to Workspace"), [&]()
			{
				IAssetEditorInstance* AssetEditor = AssetEditorSubsystem->FindEditorForAsset(AnimationGraphAsset, false);
				UE::Workspace::IWorkspaceEditor* WorkspaceEditor = static_cast<UE::Workspace::IWorkspaceEditor*>(AssetEditor);
				UWorkspace* Workspace = CastChecked<UWorkspace>(WorkspaceEditor->GetWorkspaceAsset());

				TestRunner->AddExpectedError("Schema does not support adding multiple assets.", EAutomationExpectedErrorFlags::Contains, -1);
				ASSERT_THAT(IsFalse(Workspace->AddAsset(AnimationGraphAsset2)));
			});
	}
};

#endif // WITH_EDITOR
#endif // WITH_DEV_AUTOMATION_TESTS
