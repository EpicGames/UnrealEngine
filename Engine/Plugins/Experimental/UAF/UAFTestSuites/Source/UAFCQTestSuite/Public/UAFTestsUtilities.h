// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_DEV_AUTOMATION_TESTS
#if WITH_EDITOR

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "Common/GraphEditorSchemaActions.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "Param/ParamType.h"

class URigVMLibraryNode;

namespace UAFTestsUtilities
{
	/**
	 * Creates the object instance for the specified asset class.
	 *
	 * @param InFactory  A pointer to the factory instance.
	 * @param InClass  A pointer to the asset static class.
	 * @param InPackageName  Name being assigned to the package, optional.
	 * @return A pointer to the object instance for the specified asset class, or nullptr otherwise.
	 */
	UObject* CreateFactoryObject(UFactory* InFactory, UClass* InClass, const FString& InPackageName = TEXT(""));

	/**
	 * Adds a new RigUnit node to the AnimNext graph specified.
	 *
	 * @param InParentGraph  A pointer to the AnimNext parent graph instance.
	 * @param InScriptStructPath:  Path to the script struct.
	 * @param InFromPins:  Pin to wire from.
	 * @param InRigUnitLocation:  Location of the RigUnit node on the graph.
	 * @param InUnitNodeClassPath:  Path to the unit node class path, optional
	 * @return A pointer to the RigUnit node instance, or nullptr otherwise.
	 */
	UEdGraphNode* AddUnitNode(UEdGraph* InParentGraph, const FString& InScriptStructPath, TArray<UEdGraphPin*>& InFromPins, const FVector2f& InRigUnitLocation, const FString& InUnitNodeClassPath = TEXT(""));

	/**
	 * Adds a new function node instance to the AnimNext Asset specified.
	 *
	 * @param InAnimNextAsset  A pointer to the AnimNext asset instance.
	 * @param InFunctionName  Name to assign to function node, optional.
	 * @return A pointer to the object instance for the function node, or nullptr otherwise.
	 */
	URigVMLibraryNode* AddFunctionNode(UUAFRigVMAsset* InAnimNextAsset, const FString& InFunctionName = TEXT("NewFunction"));

	/**
	 * Adds a new variable instance to the AnimNext Asset specified.
	 *
	 * @param InAnimNextAsset  A pointer to the AnimNext asset instance.
	 * @param InVariableType:  Data type of variable.
	 * @param InVariableName  Name to assign to variable, optional.
	 * @param InDefaultValue  Default value to assign to variable, optional.
	 * @return A pointer to the object instance for the variable, or nullptr otherwise.
	 */
	UAnimNextVariableEntry* AddVariable(UUAFRigVMAsset* InAnimNextAsset, const FAnimNextParamType& InVariableType, const FString& InVariableName = TEXT("NewVariable"), const FString& InDefaultValue = TEXT(""));

	/**
	 * Adds a new shared variables instance to the AnimNext Asset specified.
	 *
	 * @param InAnimNextAsset  A pointer to the AnimNext asset instance.
	 * @param InSharedVariables:  A pointer to the AnimNext shared variables instance.
	 * @return A pointer to the object instance for the shared variables, or nullptr otherwise.
	 */
	UUAFSharedVariablesEntry* AddSharedVariables(UUAFRigVMAsset* InAnimNextAsset, UUAFSharedVariables* InSharedVariables);

	/**
	 * Adds a new variable node to the AnimNext graph specified.
	 *
	 * @param InParentGraph  A pointer to the AnimNext parent graph instance.
	 * @param InSourceObject The source object (asset or struct) that this variable comes from
	 * @param InVariableName  Name to assign to variable.
	 * @param InVariableType:  Data type of variable.
	 * @param InVariableAccessorChoice:  Accessor choice of the variable node.
	 * @param InFromPins:  Pin to wire from.
	 * @param InVariableLocation:  Location of the variable node on the graph.
	 * @return A pointer to the variable node instance, or nullptr otherwise.
	 */
	UEdGraphNode* AddVariableNode(UEdGraph* InParentGraph, const UObject* InSourceObject, const FString& InVariableName, const FAnimNextParamType& InVariableType, const FAnimNextSchemaAction_Variable::EVariableAccessorChoice InVariableAccessorChoice, TArray<UEdGraphPin*>& InFromPins, const FVector2f& InVariableLocation);

	/**
	 * Adds a new pin to the library node specified.
	 *
	 * @param InAnimNextAsset  A pointer to the AnimNext asset instance.
	 * @param InLibraryNode  A pointer to the library node instance.
	 * @param InDirection  Direction of the pin.
	 * @param InCPPName  Name assigned to the pin, optional.
	 * @param InCPPType  Data type assigned to the pin, optional.
	 * @param InCPPTypeObjectPath  Path assigned to the pin, optional.
	 * @param InDefaultValue  Default value assigned to the pin, optional.
	 * @return A pointer to the pin instance for the specified library node, or nullptr otherwise.
	 */
	URigVMPin* AddPin(UUAFRigVMAsset* InAnimNextAsset, URigVMLibraryNode* InLibraryNode, ERigVMPinDirection InDirection, const FString& InCPPName = TEXT("Argument"), const FString& InCPPType = TEXT("bool"), const FName& InCPPTypeObjectPath = TEXT("None"), const FString& InDefaultValue = TEXT("False"));

	/**
	 * Adds a new link between two pins.
	 *
	 * @param InAnimNextAsset  A pointer to the AnimNext asset instance.
	 * @param InOutputPinPath  A path to an output pin.
	 * @param InInputPinPath  A path to an input pin.
	 * @return True if add link is successful, False otherwise.
	 */
	bool AddLink(UUAFRigVMAsset* InAnimNextAsset, const FString& InOutputPinPath, const FString& InInputPinPath);

	/**
	 * Select specified node(s).
	 *
	 * @param InAnimNextAsset  A pointer to the AnimNext asset instance.
	 * @param InNodeNames  Array of node name(s) to select.
	 * @return True if node(s) selection is successful, False otherwise.
	 */
	bool SetNodeSelection(UUAFRigVMAsset* InAnimNextAsset, const TArray<FName>& InNodeNames);

	/**
	 * Collapse selected nodes to a single node.
	 *
	 * @param InAnimNextAsset  A pointer to the AnimNext asset instance.
	 * @param InNodeNames  Array of node names to collapse.
	 * @param InCollapseNodeName  Name of collapse node, optional
	 * @param InCollapseToFunction  Collapse to a function node, optional.
	 * @return A pointer to the pin instance for the collapse node, or nullptr otherwise.
	 */
	URigVMCollapseNode* CollapseNodes(UUAFRigVMAsset* InAnimNextAsset, const TArray<FName>& InNodeNames, const FString& InCollapseNodeName = TEXT(""), bool InCollapseToFunction = false);
}

#endif // WITH_EDITOR
#endif // WITH_DEV_AUTOMATION_TESTS
