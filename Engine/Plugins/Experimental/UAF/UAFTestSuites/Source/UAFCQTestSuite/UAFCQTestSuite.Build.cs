// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class UAFCQTestSuite : ModuleRules
{
	public UAFCQTestSuite(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		bTreatAsEngineModule = true;

		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
		);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
		);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"UnrealEd"
				// ... add other public dependencies that you statically link with here ...
			}
		);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"CQTest",
				"Engine",
				"UAF",
				"WorkspaceEditor",
				"UAFEditor",
				"UAFStateTree",
				"UAFTestSuite",
				"UAFStateTreeEditor",
				"UAFAnimGraph",
				"UAFAnimGraphEditor",
				"UAFUncookedOnly",
				"RigVM",
				"RigVMDeveloper",
				"UAFAnimNodeTestData"
				// ... add private dependencies that you statically link with here ...
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
		);
	}
}
