// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "AnimNextTest.h"
#include "UncookedOnlyUtils.h"
#include "Misc/AutomationTest.h"
#include "Animation/AnimSequence.h"
#include "Entries/AnimNextVariableEntry.h"
#include "Entries/AnimNextEventGraphEntry.h"
#include "Module/AnimNextModule.h"
#include "Module/AnimNextModuleFactory.h"
#include "Module/AnimNextModule_EditorData.h"
#include "Module/RigUnit_AnimNextModuleEvents.h"
#if WITH_EDITOR
#include "ScopedTransaction.h"
#include "Editor.h"
#include "IPythonScriptPlugin.h"
#endif

// AnimNext Editor Tests

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

namespace UE::UAF::Tests
{

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditor_Variables, "Animation.AnimNext.Editor.Variables", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FEditor_Variables::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	struct FFactoryAndClass
	{
		TSubclassOf<UFactory> FactoryClass;
		TSubclassOf<UUAFRigVMAsset> Class;
	};
	
	FFactoryAndClass FactoryClassPairs[] =
	{
		{ UUAFSystemFactory::StaticClass(), UUAFSystem::StaticClass() },
	};

	for(const FFactoryAndClass& FactoryAndClass : FactoryClassPairs)
	{
		ON_SCOPE_EXIT{ FUtils::CleanupAfterTests(); };

		UFactory* Factory = NewObject<UFactory>(GetTransientPackage(), FactoryAndClass.FactoryClass);
		UUAFRigVMAsset* Asset = CastChecked<UUAFRigVMAsset>(Factory->FactoryCreateNew(FactoryAndClass.Class, GetTransientPackage(), TEXT("TestAsset"), RF_Transient, nullptr, nullptr, NAME_None));
		UE_RETURN_ON_ERROR(Asset != nullptr, "FEditor_Variables -> Failed to create asset");

		UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(Asset);
		UE_RETURN_ON_ERROR(EditorData != nullptr, "FEditor_Variables -> Asset has no editor data.");

		static FName TestVariableName = TEXT("TestVar");

		// AddVariable
		UAnimNextVariableEntry* Variable = nullptr;
		{
			FScopedTransaction Transaction(FText::GetEmpty());
			Variable = EditorData->AddVariable(TestVariableName, FAnimNextParamType::GetType<bool>());

			UE_RETURN_ON_ERROR(Variable != nullptr, TEXT("Could not create new variable in asset."));
			AddErrorIfFalse(Variable->GetType() == FAnimNextParamType::GetType<bool>(), TEXT("Incorrect variable type found"));
		}

		GEditor->UndoTransaction();
		AddErrorIfFalse(EditorData->Entries.Num() == 1, FString::Printf(TEXT("Unexpected entry count found in graph (Have %d, expected 1)."), EditorData->Entries.Num()));

		GEditor->RedoTransaction();
		AddErrorIfFalse(EditorData->Entries.Num() == 2, FString::Printf(TEXT("Unexpected entry count found in graph (Have %d, expected 2)."), EditorData->Entries.Num()));

		// Failure cases
		AddExpectedError(TEXT("UUAFRigVMAssetEditorData::AddVariable: Invalid variable name supplied."));
		AddErrorIfFalse(EditorData->AddVariable(NAME_None, FAnimNextParamType::GetType<bool>()) == nullptr, TEXT("Expected invalid argument to fail"));

		auto TestVariableType = [this, EditorData](FAnimNextParamType InType, FName InName = TEXT("TestVar0"), bool bInRemove = true)
		{
			UAnimNextVariableEntry* TypedVariable = EditorData->AddVariable(InName, InType);
			const bool bValidVariable = TypedVariable != nullptr;
			if (bValidVariable && AddErrorIfFalse(bValidVariable, FString::Printf(TEXT("Could not create new variable of type %s in graph."), *InType.ToString())))
			{
				AddErrorIfFalse(TypedVariable->GetType() == InType, TEXT("Incorrect variable type found"));
				if(bInRemove)
				{
					EditorData->RemoveEntry(TypedVariable);
				}
			}
		};

		// Various types
		TestVariableType(FAnimNextParamType::GetType<bool>());
		TestVariableType(FAnimNextParamType::GetType<uint8>());
		TestVariableType(FAnimNextParamType::GetType<int32>());
		TestVariableType(FAnimNextParamType::GetType<int64>());
		TestVariableType(FAnimNextParamType::GetType<float>());
		TestVariableType(FAnimNextParamType::GetType<double>());
		TestVariableType(FAnimNextParamType::GetType<FName>());
		TestVariableType(FAnimNextParamType::GetType<FString>());
		TestVariableType(FAnimNextParamType::GetType<EPropertyBagPropertyType>());
		TestVariableType(FAnimNextParamType::GetType<FVector>());
		TestVariableType(FAnimNextParamType::GetType<FQuat>());
		TestVariableType(FAnimNextParamType::GetType<FTransform>());
		TestVariableType(FAnimNextParamType::GetType<TObjectPtr<UObject>>());
		TestVariableType(FAnimNextParamType::GetType<TObjectPtr<UAnimSequence>>());
		TestVariableType(FAnimNextParamType::GetType<TArray<float>>());
		TestVariableType(FAnimNextParamType::GetType<TArray<TObjectPtr<UAnimSequence>>>());

		// RemoveEntry
		{
			FScopedTransaction Transaction(FText::GetEmpty());
			AddErrorIfFalse(EditorData->RemoveEntry(Variable), TEXT("Failed to remove entry."));
		}

		GEditor->UndoTransaction();

		// FindEntry
		AddErrorIfFalse(EditorData->FindEntry(TestVariableName) != nullptr, TEXT("Could not find entry in graph."));
	}

	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditor_Graphs, "Animation.AnimNext.Editor.Graphs", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FEditor_Graphs::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	struct FTestSettings
	{
		TSubclassOf<UFactory> FactoryClass;
		TSubclassOf<UUAFRigVMAsset> Class;
		bool bEventGraphsAllowed = false;
		bool bExpectExistingEventGraph = false; 
		bool bAnimGraphsAllowed = false;
		bool bExpectExistingAnimGraph = false;
	};
	
	FTestSettings TestSettings[] =
	{
		{ UUAFSystemFactory::StaticClass(), UUAFSystem::StaticClass(), true, true, false, false },
	};

	for(const FTestSettings& TestSetting : TestSettings)
	{
		ON_SCOPE_EXIT{ FUtils::CleanupAfterTests(); };

		UFactory* Factory = NewObject<UFactory>(GetTransientPackage(), TestSetting.FactoryClass);
		UUAFRigVMAsset* Asset = CastChecked<UUAFRigVMAsset>(Factory->FactoryCreateNew(TestSetting.Class, GetTransientPackage(), TEXT("TestAsset"), RF_Transient, nullptr, nullptr, NAME_None));
		UE_RETURN_ON_ERROR(Asset != nullptr, "FEditor_Graphs -> Failed to create asset");

		UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(Asset);
		UE_RETURN_ON_ERROR(EditorData != nullptr, "FEditor_Graphs -> Asset has no editor data.");

		// AddEventGraph
		if(TestSetting.bEventGraphsAllowed)
		{
			UAnimNextEventGraphEntry* EventGraphEntry = nullptr;
			if(TestSetting.bExpectExistingEventGraph)
			{
				EventGraphEntry = Cast<UAnimNextEventGraphEntry>(EditorData->FindEntry(UUAFSystem_EditorData::DefaultEventGraphName));
				UE_RETURN_ON_ERROR(EventGraphEntry != nullptr, TEXT("Could not find existing event graph."));
			}
			else
			{
				EventGraphEntry = EditorData->AddEventGraph(UUAFSystem_EditorData::DefaultEventGraphName, FRigUnit_AnimNextPrePhysicsEvent::StaticStruct());
				UE_RETURN_ON_ERROR(EventGraphEntry != nullptr, TEXT("Could not add event graph."));
			}

			URigVMGraph* RigVMGraph = EventGraphEntry->GetRigVMGraph();
			UE_RETURN_ON_ERROR(RigVMGraph->GetNodes().Num() == 2, TEXT("Unexpected number of nodes in new event graph.")); // Initialize & PrePhysics

			{
				FScopedTransaction Transaction(FText::GetEmpty());
				bool bRemovedEventGraph = EditorData->RemoveEntry(EventGraphEntry);
				UE_RETURN_ON_ERROR(bRemovedEventGraph, "FEditor_Graphs -> Could not remove event graph.");
			}

			GEditor->UndoTransaction();

			UAnimNextEventGraphEntry* FoundEventGraphEntry = Cast<UAnimNextEventGraphEntry>(EditorData->FindEntry(UUAFSystem_EditorData::DefaultEventGraphName));
			UE_RETURN_ON_ERROR(FoundEventGraphEntry != nullptr, "FEditor_Graphs -> Could not find event graph post-undo.");
		}
		else
		{
			AddExpectedError(TEXT("Cannot add an event graph to this asset - entry is not allowed"));
			UAnimNextEventGraphEntry* EventGraphEntry = EditorData->AddEventGraph(UUAFSystem_EditorData::DefaultEventGraphName, FRigUnit_AnimNextPrePhysicsEvent::StaticStruct());
		}
	}

	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditor_Variables_Python, "Animation.AnimNext.Editor.Python.Variables", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FEditor_Variables_Python::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	const TCHAR* Script = TEXT(
		"asset_tools = unreal.AssetToolsHelpers.get_asset_tools()\n"
		"animation_module = unreal.AssetTools.create_asset(asset_tools, asset_name = \"TestModule\", package_path = \"/Game/\", asset_class = unreal.UAFSystem, factory = unreal.UAFSystemFactory())\n"
		"animation_module.add_variable(name = \"TestParam\", value_type = unreal.PropertyBagPropertyType.BOOL, container_type = unreal.PropertyBagContainerType.NONE)\n"
		"unreal.EditorAssetLibrary.delete_loaded_asset(animation_module)\n"
	);

	IPythonScriptPlugin::Get()->ExecPythonCommand(Script);

	FUtils::CleanupAfterTests();

	return true;
}

}

#endif	// WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR
