// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextPool.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::UAF
{

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPoolTest, "Animation.AnimNext.Pool", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FPoolTest::RunTest(const FString& InParameters)
{
	// Test Emplace/Release/IsValidHandle
	{
		struct FPodStruct
		{
			uint32 Sentinel = 0xc01df00d;
		};
		
		TPool<FPodStruct> Pool;
		TArray<TPoolHandle<FPodStruct>> Handles;
		auto CheckPoolIntegrity = [this, &Handles, &Pool]()
		{
			for(TPoolHandle<FPodStruct> Handle : Handles)
			{
				AddErrorIfFalse(Handle.IsValid(), "Invalid handle");
				AddErrorIfFalse(Pool.IsValidHandle(Handle), "Invalid handle");
				AddErrorIfFalse(Pool.Get(Handle).Sentinel == 0xc01df00d, "Invalid sentinel");
			}
		};
		auto EmplaceRange = [&Handles, &Pool, &CheckPoolIntegrity](int32 Count)
		{
			for(int32 Index = 0; Index < Count; ++Index)
			{
				Handles.Add(Pool.Emplace());
			}
			CheckPoolIntegrity();
		};
		auto ReleaseRange = [&Handles, &Pool, &CheckPoolIntegrity](int32 Start, int32 Count)
		{
			for(int32 Index = Start; Index < Count; ++Index)
			{
				if(Handles.IsValidIndex(Start))
				{
					Pool.Release(Handles[Start]);
					Handles.RemoveAtSwap(Start);
				}
			}
			CheckPoolIntegrity();
		};
		
		EmplaceRange(1000);
		ReleaseRange(20, 50);
		EmplaceRange(150);
		ReleaseRange(20, 50);
		ReleaseRange(200, 300);
		ReleaseRange(700, 100);
		EmplaceRange(200);
		EmplaceRange(300);
		ReleaseRange(700, 100);
		ReleaseRange(700, 100);
	}

	// Test iteration
	{
		struct FIndexedStruct
		{
			FIndexedStruct() = default;
			FIndexedStruct(uint32 InIndex) : Index(InIndex){}
			uint32 Index = 0;
		};
		
		TPool<FIndexedStruct> IndexedPool;
		TArray<TPoolHandle<FIndexedStruct>> IndexedHandles;

		for(uint32 Index = 0; Index < 100; ++Index)
		{
			IndexedHandles.Add(IndexedPool.Emplace(FIndexedStruct(Index)));
		}

		uint32 CheckIndex = 0;
		for(FIndexedStruct& Value : IndexedPool)
		{
			AddErrorIfFalse(Value.Index == CheckIndex, "Invalid index");
			CheckIndex++;
		}

		// Remove head
		IndexedPool.Release(IndexedHandles[0]);
		IndexedHandles.RemoveAt(0);

		CheckIndex = 1;
		for(FIndexedStruct& Value : IndexedPool)
		{
			AddErrorIfFalse(Value.Index == CheckIndex, "Invalid index");
			CheckIndex++;
		}

		// Remove tail
		IndexedPool.Release(IndexedHandles.Last());
		IndexedHandles.Pop();
		
		CheckIndex = 1;
		for(FIndexedStruct& Value : IndexedPool)
		{
			AddErrorIfFalse(Value.Index == CheckIndex, "Invalid index");
			CheckIndex++;
		}

		// Remove multiple from near-tail, non tail first
		IndexedPool.Release(IndexedHandles[IndexedHandles.Num() - 2]);
		IndexedHandles.RemoveAt(IndexedHandles.Num() - 2);
		IndexedPool.Release(IndexedHandles.Last());
		IndexedHandles.Pop();
		
		CheckIndex = 1;
		for(FIndexedStruct& Value : IndexedPool)
		{
			AddErrorIfFalse(Value.Index == CheckIndex, "Invalid index");
			CheckIndex++;
		}

		// Test Reverse iteration
		CheckIndex = IndexedHandles.Num() - 1;
		for(FIndexedStruct& Value : ReverseIterate(IndexedPool))
		{
			AddErrorIfFalse(Value.Index == CheckIndex, "Invalid index");
			CheckIndex--;
		}
	}

	return true;
}

}

#endif
