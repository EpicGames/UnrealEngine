// Copyright Epic Games, Inc. All Rights Reserved.


#include "AnimNextTest.h"
#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "DataRegistry.h"
#include "ReferencePose.h"
#include <chrono>
#if WITH_EDITOR
#include "Editor.h"
#include "Editor/Transactor.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextTest)

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::UAF::Tests
{

namespace Private
{

class FHighResTimer
{
public:
	using FTimePoint = std::chrono::time_point<std::chrono::high_resolution_clock>;

	static inline FTimePoint GetTimeMark()
	{
		return std::chrono::high_resolution_clock::now();
	}

	static inline int64 GetTimeDiffMicroSec(const FTimePoint& RefTime)
	{
		return std::chrono::duration_cast<std::chrono::microseconds>(GetTimeMark() - RefTime).count();
	}
	static inline int64 GetTimeDiffNanoSec(const FTimePoint& RefTime)
	{
		return std::chrono::duration_cast<std::chrono::nanoseconds>(GetTimeMark() - RefTime).count();
	}
};

}

void FUtils::CleanupAfterTests()
{
#if WITH_EDITOR
	if(GEditor && GEditor->Trans)
	{
		GEditor->Trans->Reset(NSLOCTEXT("AnimNextTests", "CleanupAfterTest", "Cleaning up after AnimNext test"));
	}
#endif
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
}

class UWorld* FUtils::GetWorld()
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		return GWorld;
	}
#endif // WITH_EDITOR

	if (GEngine->GetWorldContexts().IsEmpty())
	{
		return nullptr;
	}

	return GEngine->GetWorldContexts()[0].World();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FDataRegistryTransformsTest, "Animation.AnimNext.AnimationDataRegistry.TransformsTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FDataRegistryTransformsTest::RunTest(const FString& InParameters)
{
	using namespace UE::UAF;

	FDataRegistry* AnimDataReg = FDataRegistry::Get();


	//constexpr int32 BASIC_TYPE_ALLOC_BLOCK = 1000;
	//AnimDataReg.RegisterDataType<uint8>(EAnimationDataType::RawMemory, BASIC_TYPE_ALLOC_BLOCK);
	//AnimDataReg.RegisterDataType<FVector>(EAnimationDataType::Translation, BASIC_TYPE_ALLOC_BLOCK);
	//AnimDataReg.RegisterDataType<FVector>(EAnimationDataType::Scale, BASIC_TYPE_ALLOC_BLOCK);
	//AnimDataReg.RegisterDataType<FQuat>(EAnimationDataType::Rotation, BASIC_TYPE_ALLOC_BLOCK);
	//AnimDataReg.RegisterDataType<FTransform>(EAnimationDataType::Transform, BASIC_TYPE_ALLOC_BLOCK);
	//AnimDataReg.RegisterDataType<FTransform>(EAnimationDataType::Displacement, BASIC_TYPE_ALLOC_BLOCK);

	//AnimDataReg.RegisterDataType<FAnimTransformArray>(EAnimationDataType::TransformArray, BASIC_TYPE_ALLOC_BLOCK);
	//AnimDataReg.RegisterDataType<FAnimTransformArraySoA>(EAnimationDataType::TransformArraySoA, BASIC_TYPE_ALLOC_BLOCK);

	constexpr int32 NumElements = 1;
	constexpr int32 NumPasses = 100;
	constexpr int32 NumTransforms = 100;

	TArray<int64> TimeDiffAoS;
	TArray<int64> TimeDiffSoA;
	TimeDiffAoS.SetNumZeroed(NumPasses);
	TimeDiffSoA.SetNumZeroed(NumPasses);

	const double BlendFactor = 0.5f;
	const FVector Half = FVector(0.5, 0.5, 0.5);
	const FQuat Rot90 = FQuat::MakeFromEuler(FVector(90.0, 0.0, 0.0));

	const FTransform TargetBlendValue(Rot90, FVector::OneVector, Half);
	FTransform ReferenceResult;
	ReferenceResult.Blend(FTransform::Identity, TargetBlendValue, BlendFactor);

	// --- AoS Test ---

	FTransformArrayAoSHeap ResultAoS(NumTransforms);
	for (int i = 0; i < NumPasses; ++i)
	{
		// *** TransformArrayAoS Test *** 
		{
			FTransformArrayAoSHeap TransformsA(NumTransforms);
			FTransformArrayAoSHeap TransformsB(NumTransforms);

			for (auto& Transform : TransformsB.GetTransforms())
			{
				Transform.SetTranslation(FVector::OneVector);
				Transform.SetRotation(Rot90);
				Transform.SetScale3D(Half);
			}

			Private::FHighResTimer::FTimePoint StartTime = Private::FHighResTimer::GetTimeMark();

			ResultAoS.Blend(TransformsA, TransformsB, BlendFactor);

			TimeDiffAoS[i] = Private::FHighResTimer::GetTimeDiffNanoSec(StartTime);
		}
	}
	for (auto& Transform : ResultAoS.Transforms)
	{
		AddErrorIfFalse(Transform.GetTranslation() == ReferenceResult.GetTranslation(), "AoS Translation Blend error.");
		AddErrorIfFalse(Transform.GetRotation() == ReferenceResult.GetRotation(), "AoS Rotation Blend error.");
		AddErrorIfFalse(Transform.GetScale3D() == ReferenceResult.GetScale3D(), "AoS Scale Blend error.");
	}

	// --- SoA Test --- 
	FTransformArraySoAHeap ResultSoA(NumTransforms);
	for (int i = 0; i < NumPasses; ++i)
	{
		//FAnimTransformArray* TransoformArray = AnimDataReg->AllocateData<FAnimTransformArray>(NumElements, NumTransforms);

		// *** TransformArraySoA Test *** 
		{
			FTransformArraySoAHeap TransformsA(NumTransforms);
			FTransformArraySoAHeap TransformsB(NumTransforms);

			for (auto& Translation : TransformsB.Translations)
			{
				Translation = FVector::OneVector;
			}
			for (auto& Rotation : TransformsB.Rotations)
			{
				Rotation = Rot90;
			}
			for (auto& Scale3D : TransformsB.Scales3D)
			{
				Scale3D = Half;
			}

			Private::FHighResTimer::FTimePoint StartTime = Private::FHighResTimer::GetTimeMark();

			ResultSoA.Blend(TransformsA, TransformsB, BlendFactor);

			TimeDiffSoA[i] = Private::FHighResTimer::GetTimeDiffNanoSec(StartTime);
		}
	}
	for (const auto& Translation : ResultSoA.Translations)
	{
		AddErrorIfFalse(Translation == ReferenceResult.GetTranslation(), "SoA Translation Blend error.");
	}
	for (const auto& Rotation : ResultSoA.Rotations)
	{
		AddErrorIfFalse(Rotation == ReferenceResult.GetRotation(), "SoA Rotation Blend error.");
	}
	for (const auto& Scale3D : ResultSoA.Scales3D)
	{
		AddErrorIfFalse(Scale3D == ReferenceResult.GetScale3D(), "SoA Scale Blend error.");
	}

	// --- Time Averaging ---

	TimeDiffAoS.Sort();
	TimeDiffSoA.Sort();

	const int32 NumToAverage = FMath::Min(3, TimeDiffAoS.Num());
	int64 AverageAoS = 0LL;
	for (int i = 0; i < NumToAverage; ++i)
	{
		AverageAoS += TimeDiffAoS[i];
	}
	AverageAoS /= NumToAverage;

	int64 AverageSoA = 0LL;
	for (int i = 0; i < NumToAverage; ++i)
	{
		AverageSoA += TimeDiffSoA[i];
	}
	AverageSoA /= NumToAverage;


	AddInfo(FString::Printf(TEXT("Average AoS Duration: %lld nanosecs."), AverageAoS));
	AddInfo(FString::Printf(TEXT("Average SoA Duration: %lld nanosecs."), AverageSoA));

	// Final GC to make sure everything is cleaned up
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

	return true;
}

}

#endif // WITH_DEV_AUTOMATION_TESTS
