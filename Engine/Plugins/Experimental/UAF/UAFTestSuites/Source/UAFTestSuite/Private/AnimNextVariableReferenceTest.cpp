// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextVariableReferenceTest.h"

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

#include "CoreMinimal.h"
#include "Misc/TransactionCommon.h"
#include "UncookedOnlyUtils.h"
#include "Misc/AutomationTest.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/BufferWriter.h"
#include "Serialization/MemoryReader.h"
#include "StructUtils/PropertyBag.h"
#include "Variables/AnimNextSharedVariables.h"
#include "Variables/AnimNextSharedVariablesFactory.h"
#include "Variables/AnimNextSharedVariables_EditorData.h"
#include "Variables/AnimNextVariableReference.h"
#include "UObject/Class.h"

namespace UE::UAF::Tests
{

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVariableReferenceTest, "Animation.UAF.VariableReferences", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FVariableReferenceTest::RunTest(const FString& InParameters)
{
	FInstancedPropertyBag PropertyBag;

	const FName BoolName("BoolVariable");
	const FName RenamedBoolName(GET_MEMBER_NAME_CHECKED(FUAFVariableReferenceTestStruct, Renamed_BoolVariable));
	const FName FloatName("FloatVariable");

	UUAFSharedVariablesFactory* Factory = NewObject<UUAFSharedVariablesFactory>(GetTransientPackage());
	UUAFSharedVariables* NewSharedVariables = CastChecked<UUAFSharedVariables>(Factory->FactoryCreateNew(UUAFSharedVariables::StaticClass(), GetTransientPackage(), TEXT("TestSharedVariables"), RF_Public | RF_Standalone | RF_Transactional, nullptr, nullptr, NAME_None));
	check(NewSharedVariables);

	UUAFSharedVariables_EditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFSharedVariables_EditorData>(NewSharedVariables);
	UAnimNextVariableEntry* BoolEntry = EditorData->AddVariable(BoolName, FAnimNextParamType(FAnimNextParamType::EValueType::Bool));
	check(BoolEntry);
	UAnimNextVariableEntry* FloatEntry = EditorData->AddVariable(FloatName, FAnimNextParamType(FAnimNextParamType::EValueType::Float));
	check(FloatEntry);
	
	const FString OwnerName(TEXT("MockOwner"));
	{
		const FAnimNextVariableReference VariableReference = FAnimNextVariableReference::FromName(BoolName, NewSharedVariables);
		check(VariableReference.IsValid());
	
		// Unchanged PostSerialize
		{
			FAnimNextVariableReference CopiedVariableReference = VariableReference;
			FAnimNextVariableReference::ValidateVariableNameAndGuid(CopiedVariableReference.Name, CopiedVariableReference.CachedGuid, CopiedVariableReference.GetObject(), OwnerName);
		}
		
		// Invalid Reference
		{
			FAnimNextVariableReference InvalidReference = VariableReference;
			InvalidReference.Object = nullptr;
			AddExpectedMessage(TEXT("Could not find variable BoolVariable"), ELogVerbosity::Warning);
			FAnimNextVariableReference::ValidateVariableNameAndGuid(InvalidReference.Name, InvalidReference.CachedGuid, InvalidReference.GetObject(), OwnerName);
		}
	
		UncookedOnly::FUtils::RenameVariable(BoolEntry, RenamedBoolName);
	
		// Renamed variable PostSerialize
		{
			FAnimNextVariableReference CopiedVariableReference = VariableReference;

			AddExpectedMessage(TEXT("Out-of-date variable reference"), ELogVerbosity::Warning);
			FAnimNextVariableReference::ValidateVariableNameAndGuid(CopiedVariableReference.Name, CopiedVariableReference.CachedGuid, CopiedVariableReference.GetObject(), OwnerName);
			AddErrorIfFalse(CopiedVariableReference.GetName() == RenamedBoolName, TEXT("Expected Name to match renamed Variable its name"));
			AddErrorIfFalse(CopiedVariableReference.CachedGuid == VariableReference.CachedGuid, TEXT("Expected CachedGuid to be consistent after variable rename"));
		}
	
		UncookedOnly::FUtils::DeleteVariable(BoolEntry);
	
		// Remove variable PostSerialize
		{
			FAnimNextVariableReference CopiedVariableReference = VariableReference;

			AddExpectedMessage(TEXT("Could not find variable"), ELogVerbosity::Warning);			
			FAnimNextVariableReference::ValidateVariableNameAndGuid(CopiedVariableReference.Name, CopiedVariableReference.CachedGuid, CopiedVariableReference.GetObject(), OwnerName);
			AddErrorIfFalse(CopiedVariableReference.GetName() == NAME_None, TEXT("Expected Name to be invalidated (None) for removed Variable"));
			AddErrorIfFalse(!CopiedVariableReference.CachedGuid.IsValid(), TEXT("Expected Guid to have been invalidated for removed Variable"));
		}
	}
	
	{
		const FAnimNextVariableReference VariableReference = FAnimNextVariableReference::FromName(FloatName, NewSharedVariables);
		check(VariableReference.IsValid());

		UncookedOnly::FUtils::DeleteVariable(FloatEntry);
		UAnimNextVariableEntry* NewFloatEntry = EditorData->AddVariable(FloatName, FAnimNextParamType(FAnimNextParamType::EValueType::Float));
	
		// Removed and re-added variable (different Guid) PostSerialize
		{
			FAnimNextVariableReference CopiedVariableReference = VariableReference;
			
			AddExpectedMessage(TEXT("Out-of-date variable reference"), ELogVerbosity::Warning);
			FAnimNextVariableReference::ValidateVariableNameAndGuid(CopiedVariableReference.Name, CopiedVariableReference.CachedGuid, CopiedVariableReference.GetObject(), OwnerName);

			AddErrorIfFalse(CopiedVariableReference.GetName() == FloatName, TEXT("Expected Name to match original Variable name for removed Variable"));
			AddErrorIfFalse(CopiedVariableReference.CachedGuid == NewFloatEntry->Guid, TEXT("Expected Guid to match (re-added) Variable"));
		}
	}
	
	{
		const FAnimNextVariableReference FloatVariableReference = FAnimNextVariableReference::FromName(FloatName, CastChecked<UObject>(FUAFVariableReferenceTestStruct::StaticStruct()));
		
		// Unchanged PostSerialize
		{
			FAnimNextVariableReference CopiedVariableReference = FloatVariableReference;
			FAnimNextVariableReference::ValidateVariableNameAndGuid(CopiedVariableReference.Name, CopiedVariableReference.CachedGuid, CopiedVariableReference.GetObject(), OwnerName);
		}
		
		
		FAnimNextVariableReference BoolVariableReference = FAnimNextVariableReference::FromName(BoolName, CastChecked<UObject>(FUAFVariableReferenceTestStruct::StaticStruct()));
		// Force CachedGuid to non-existing one
		BoolVariableReference.CachedGuid = FGuid(1,1,1,1); 
		
		// Take reference to renamed, and previously redirect, struct property for validation
		const FAnimNextVariableReference RenamedBoolVariableReference = FAnimNextVariableReference::FromName(RenamedBoolName, CastChecked<UObject>(FUAFVariableReferenceTestStruct::StaticStruct()));

		// Redirected UStruct property PostSerialize
		{
			FAnimNextVariableReference CopiedVariableReference = BoolVariableReference;
			
			AddExpectedMessage(TEXT("Out-of-date variable reference"), ELogVerbosity::Warning);
			FAnimNextVariableReference::ValidateVariableNameAndGuid(CopiedVariableReference.Name, CopiedVariableReference.CachedGuid, CopiedVariableReference.GetObject(), OwnerName);
			AddErrorIfFalse(CopiedVariableReference == RenamedBoolVariableReference, TEXT("Expected Variable Reference for redirect Struct Property to match direct (struct property) Variable Reference"));
		}
	}

	return true;
}

}

#endif // WITH_DEV_AUTOMATION_TESTS

