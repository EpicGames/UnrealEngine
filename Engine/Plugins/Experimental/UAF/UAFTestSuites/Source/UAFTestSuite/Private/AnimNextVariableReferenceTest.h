// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextVariableReferenceTest.generated.h"

USTRUCT()
struct FUAFVariableReferenceTestStruct
{
	GENERATED_BODY()
	
	UPROPERTY()
	bool Renamed_BoolVariable = false;
	
	UPROPERTY()
	float FloatVariable = 0.f;
};
