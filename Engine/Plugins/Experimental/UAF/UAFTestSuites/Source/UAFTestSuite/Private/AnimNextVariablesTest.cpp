// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextVariablesTest.h"
#include "CoreMinimal.h"
#include "AnimNextTest.h"
#include "ScopedTransaction.h"
#include "UncookedOnlyUtils.h"
#include "Algo/Compare.h"
#include "Misc/AutomationTest.h"
#include "Param/ParamType.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimMontage.h"
#include "Async/ParallelFor.h"
#include "Component/AnimNextComponent.h"
#include "Entries/AnimNextEventGraphEntry.h"
#include "Module/AnimNextModule.h"
#include "Module/AnimNextModule_EditorData.h"
#include "Entries/AnimNextVariableEntry.h"
#include "Factories/Factory.h"
#include "Module/AnimNextModuleFactory.h"
#include "Module/RigUnit_AnimNextModuleEvents.h"
#include "Variables/AnimNextUniversalObjectLocatorBindingData.h"
#include "Module/AnimNextModuleInstance.h"
#include "Script/UAFRigVMComponent.h"

// AnimNext Parameters Tests

#include "Factory/UAFSystemFactoryParams.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextVariablesTest)

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::UAF::Tests
{

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVariableTypesTest, "Animation.AnimNext.VariableTypes", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FVariableTypesTest::RunTest(const FString& InParameters)
{
	// None is invalid
	FAnimNextParamType ParameterTypeValueNone(FAnimNextParamType::EValueType::None); 
	AddErrorIfFalse(!ParameterTypeValueNone.IsValid(), TEXT("Parameter type None is valid."));

	// None is invalid for all containers
	for(uint8 ContainerType = (uint8)FAnimNextParamType::EContainerType::None; ContainerType <= (uint8)FAnimNextParamType::EContainerType::Array; ++ContainerType)
	{
		FAnimNextParamType ParameterTypeValueContainerNone(FAnimNextParamType::EValueType::None, (FAnimNextParamType::EContainerType)ContainerType); 
		AddErrorIfFalse(!ParameterTypeValueContainerNone.IsValid(), FString::Printf(TEXT("Parameter type None, container type %d is valid."), ContainerType));
	}

	// Null object types
	for(uint8 ObjectValueType = (uint8)FAnimNextParamType::EValueType::Enum; ObjectValueType <= (uint8)FAnimNextParamType::EValueType::SoftClass; ++ObjectValueType)
	{
		for(uint8 ContainerType = (uint8)FAnimNextParamType::EContainerType::None; ContainerType <= (uint8)FAnimNextParamType::EContainerType::Array; ++ContainerType)
		{
			FAnimNextParamType ParameterTypeNullObject((FAnimNextParamType::EValueType)ObjectValueType, (FAnimNextParamType::EContainerType)ContainerType, nullptr); 
			AddErrorIfFalse(!ParameterTypeNullObject.IsValid(), FString::Printf(TEXT("Parameter type %d, container type %d with null object is valid."), ObjectValueType, ContainerType));
		}
	}

	// Non object types
	for(uint8 ValueType = (uint8)FAnimNextParamType::EValueType::Bool; ValueType < (uint8)FAnimNextParamType::EValueType::Enum; ++ValueType)
	{
		for(uint8 ContainerType = (uint8)FAnimNextParamType::EContainerType::None; ContainerType <= (uint8)FAnimNextParamType::EContainerType::Array; ++ContainerType)
		{
			FAnimNextParamType TestValueContainerParameterType((FAnimNextParamType::EValueType)ValueType, (FAnimNextParamType::EContainerType)ContainerType);
			AddErrorIfFalse(TestValueContainerParameterType.IsValid(), FString::Printf(TEXT("Parameter type %d, container type %d is invalid."), ValueType, ContainerType));
		}
	}

	UObject* ExampleValidObjects[(uint8)FAnimNextParamType::EValueType::SoftClass + 1] =
	{
		nullptr,
		nullptr,
		nullptr,
		nullptr,
		nullptr,
		nullptr,
		nullptr,
		nullptr,
		nullptr,
		nullptr,
		FindObjectChecked<UEnum>(nullptr, TEXT("/Script/StructUtils.EPropertyBagPropertyType")),
		FAnimNextParamType::StaticStruct(),
		UObject::StaticClass(),
		UObject::StaticClass(),
		UObject::StaticClass(),
		UObject::StaticClass()
	};

	// Non-null valid object types
	for(uint8 ObjectValueType = (uint8)FAnimNextParamType::EValueType::Enum; ObjectValueType <= (uint8)FAnimNextParamType::EValueType::SoftClass; ++ObjectValueType)
	{
		for(uint8 ContainerType = (uint8)FAnimNextParamType::EContainerType::None; ContainerType <= (uint8)FAnimNextParamType::EContainerType::Array; ++ContainerType)
		{
			FAnimNextParamType TestValueContainerParameterType((FAnimNextParamType::EValueType)ObjectValueType, (FAnimNextParamType::EContainerType)ContainerType, ExampleValidObjects[ObjectValueType]);
			AddErrorIfFalse(TestValueContainerParameterType.IsValid(), FString::Printf(TEXT("Object parameter type %d, container type %d is invalid."), ObjectValueType, ContainerType));
		}
	}

	UObject* ExampleInvalidObjects[(uint8)FAnimNextParamType::EValueType::SoftClass + 1] =
	{
		nullptr,
		nullptr,
		nullptr,
		nullptr,
		nullptr,
		nullptr,
		nullptr,
		nullptr,
		nullptr,
		nullptr,
		FAnimNextParamType::StaticStruct(),
		FindObjectChecked<UEnum>(nullptr, TEXT("/Script/StructUtils.EPropertyBagPropertyType")),
		FAnimNextParamType::StaticStruct(),
		FAnimNextParamType::StaticStruct(),
		FAnimNextParamType::StaticStruct(),
		FAnimNextParamType::StaticStruct()
	};

	// Non-null invalid object types
	for(uint8 ObjectValueType = (uint8)FAnimNextParamType::EValueType::Enum; ObjectValueType <= (uint8)FAnimNextParamType::EValueType::SoftClass; ++ObjectValueType)
	{
		for(uint8 ContainerType = (uint8)FAnimNextParamType::EContainerType::None; ContainerType <= (uint8)FAnimNextParamType::EContainerType::Array; ++ContainerType)
		{
			FAnimNextParamType TestValueContainerParameterType((FAnimNextParamType::EValueType)ObjectValueType, (FAnimNextParamType::EContainerType)ContainerType, ExampleInvalidObjects[ObjectValueType]);
			AddErrorIfFalse(!TestValueContainerParameterType.IsValid(), FString::Printf(TEXT("Object parameter type %d, container type %d is valid."), ObjectValueType, ContainerType));
		}
	}

	// Check type inference
	AddErrorIfFalse(FAnimNextParamType::GetType<bool>().IsValid(), TEXT("bool parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<uint8>().IsValid(), TEXT("uint8 parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<int32>().IsValid(), TEXT("int32 parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<int64>().IsValid(), TEXT("int64 parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<float>().IsValid(), TEXT("float parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<double>().IsValid(), TEXT("double parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<FName>().IsValid(), TEXT("FName parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<FString>().IsValid(), TEXT("FString parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<EPropertyBagContainerType>().IsValid(), TEXT("Enum parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<FAnimNextParamType>().IsValid(), TEXT("Struct parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<FVector>().IsValid(), TEXT("Struct parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<FTransform>().IsValid(), TEXT("Struct parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<FQuat>().IsValid(), TEXT("Struct parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<UObject*>().IsValid(), TEXT("UObject parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<TObjectPtr<UObject>>().IsValid(), TEXT("TObjectPtr<UObject> parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<UClass*>().IsValid(), TEXT("UClass parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<TSubclassOf<UObject>>().IsValid(), TEXT("TSubclassOf<UObject> parameter is invalid."));

	// TODO: Soft obj/class ptrs not supported yet in RigVM
//	AddErrorIfFalse(FAnimNextParamType::GetType<TSoftObjectPtr<UObject>>().IsValid(), TEXT("TSoftObjectPtr<UObject> parameter is invalid."));
//	AddErrorIfFalse(FAnimNextParamType::GetType<TSoftClassPtr<UObject>>().IsValid(), TEXT("TSoftClassPtr<UObject> parameter is invalid."));

	AddErrorIfFalse(FAnimNextParamType::GetType<TArray<bool>>().IsValid(), TEXT("bool array parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<TArray<uint8>>().IsValid(), TEXT("uint8 array parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<TArray<int32>>().IsValid(), TEXT("int32 array parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<TArray<int64>>().IsValid(), TEXT("int64 array parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<TArray<float>>().IsValid(), TEXT("float array parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<TArray<double>>().IsValid(), TEXT("double array parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<TArray<FName>>().IsValid(), TEXT("FName array parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<TArray<FString>>().IsValid(), TEXT("FString array parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<TArray<EPropertyBagContainerType>>().IsValid(), TEXT("Enum array parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<TArray<FAnimNextParamType>>().IsValid(), TEXT("Struct array parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<TArray<FVector>>().IsValid(), TEXT("Struct array parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<TArray<FTransform>>().IsValid(), TEXT("Struct array parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<TArray<FQuat>>().IsValid(), TEXT("Struct array parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<TArray<UObject*>>().IsValid(), TEXT("UObject array parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<TArray<TObjectPtr<UObject>>>().IsValid(), TEXT("TObjectPtr<UObject> array parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<TArray<UClass*>>().IsValid(), TEXT("UClass array parameter is invalid."));
	AddErrorIfFalse(FAnimNextParamType::GetType<TArray<TSubclassOf<UObject>>>().IsValid(), TEXT("TSubclassOf<UObject> array parameter is invalid."));

	// TODO: Soft obj/class ptrs not supported yet in RigVM
//	AddErrorIfFalse(FAnimNextParamType::GetType<TArray<TSoftObjectPtr<UObject>>>().IsValid(), TEXT("TSoftObjectPtr<UObject> array parameter is invalid."));
//	AddErrorIfFalse(FAnimNextParamType::GetType<TArray<TSoftClassPtr<UObject>>>().IsValid(), TEXT("TSoftClassPtr<UObject> array parameter is invalid."));

	// Check type from property
	#define TEST_ANIMNEXT_PROPERTY(Type, Property) AddErrorIfFalse(FAnimNextParamType::FromProperty(FAnimNextParamTypeTestStruct::StaticStruct()->FindPropertyByName(#Property)) == FAnimNextParamType::GetType<Type>(), #Type " param type is invalid")
	#define TEST_ANIMNEXT_PROPERTY_ARRAY(Type, Property) AddErrorIfFalse(FAnimNextParamType::FromProperty(FAnimNextParamTypeTestStruct::StaticStruct()->FindPropertyByName(#Property"Array")) == FAnimNextParamType::GetType<TArray<Type>>(), #Type " array param type is invalid")

	TEST_ANIMNEXT_PROPERTY(bool, bBool);
	TEST_ANIMNEXT_PROPERTY(uint8, Uint8);
	TEST_ANIMNEXT_PROPERTY(int32, Int32);
	TEST_ANIMNEXT_PROPERTY(int64, Int64);
	TEST_ANIMNEXT_PROPERTY(float, Float);
	TEST_ANIMNEXT_PROPERTY(double, Double);
	TEST_ANIMNEXT_PROPERTY(FName, Name);
	TEST_ANIMNEXT_PROPERTY(FString, String);
	TEST_ANIMNEXT_PROPERTY(EPropertyBagContainerType, Enum);
	TEST_ANIMNEXT_PROPERTY(FAnimNextParamType, Struct);
	TEST_ANIMNEXT_PROPERTY(FVector, Vector);
	TEST_ANIMNEXT_PROPERTY(FTransform, Transform);
	TEST_ANIMNEXT_PROPERTY(TObjectPtr<UObject>, Object);
	TEST_ANIMNEXT_PROPERTY(TObjectPtr<UClass>, Class);
	TEST_ANIMNEXT_PROPERTY(TSubclassOf<UObject>, SubclassOf);
	// TODO: Soft obj/class ptrs not supported yet in RigVM
//	TEST_ANIMNEXT_PROPERTY(TSoftObjectPtr<UObject>, SoftObjectPtr);
//	TEST_ANIMNEXT_PROPERTY(TSoftClassPtr<UObject>, SoftClassPtr);

	TEST_ANIMNEXT_PROPERTY_ARRAY(bool, Bool);
	TEST_ANIMNEXT_PROPERTY_ARRAY(uint8, Uint8);
	TEST_ANIMNEXT_PROPERTY_ARRAY(int32, Int32);
	TEST_ANIMNEXT_PROPERTY_ARRAY(int64, Int64);
	TEST_ANIMNEXT_PROPERTY_ARRAY(float, Float);
	TEST_ANIMNEXT_PROPERTY_ARRAY(double, Double);
	TEST_ANIMNEXT_PROPERTY_ARRAY(FName, Name);
	TEST_ANIMNEXT_PROPERTY_ARRAY(FString, String);
	TEST_ANIMNEXT_PROPERTY_ARRAY(EPropertyBagContainerType, Enum);
	TEST_ANIMNEXT_PROPERTY_ARRAY(FAnimNextParamType, Struct);
	TEST_ANIMNEXT_PROPERTY_ARRAY(FVector, Vector);
	TEST_ANIMNEXT_PROPERTY_ARRAY(FTransform, Transform);
	TEST_ANIMNEXT_PROPERTY_ARRAY(TObjectPtr<UObject>, Object);
	TEST_ANIMNEXT_PROPERTY_ARRAY(TObjectPtr<UClass>, Class);
	TEST_ANIMNEXT_PROPERTY_ARRAY(TSubclassOf<UObject>, SubclassOf);
	// TODO: Soft obj/class ptrs not supported yet in RigVM
//	TEST_ANIMNEXT_PROPERTY_ARRAY(TSoftObjectPtr<UObject>, SoftObjectPtr);
//	TEST_ANIMNEXT_PROPERTY_ARRAY(TSoftClassPtr<UObject>, SoftClassPtr);

	#undef TEST_ANIMNEXT_PROPERTY
	#undef TEST_ANIMNEXT_PROPERTY_ARRAY

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVariables, "Animation.AnimNext.Variables.Variables", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FVariables::RunTest(const FString& InParameters)
{
	ON_SCOPE_EXIT{ FUtils::CleanupAfterTests(); };

	UFactory* Factory = NewObject<UFactory>(GetTransientPackage(), UUAFSystemFactory::StaticClass());
	UUAFRigVMAsset* Asset = CastChecked<UUAFRigVMAsset>(Factory->FactoryCreateNew(UUAFSystem::StaticClass(), GetTransientPackage(), TEXT("TestAsset"), RF_Transient, nullptr, nullptr, NAME_None));
	UE_RETURN_ON_ERROR(Asset != nullptr, "FVariables -> Failed to create asset");

	UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(Asset);
	UE_RETURN_ON_ERROR(EditorData != nullptr, "FVariables -> Asset has no editor data.");

	// Add variables
	UAnimNextVariableEntry* OperandAEntry = EditorData->AddVariable(TEXT("A"), FAnimNextParamType::GetType<int32>(), TEXT("1"));
	UE_RETURN_ON_ERROR(OperandAEntry != nullptr, TEXT("Could not create new variable in graph."));
	UAnimNextVariableEntry* OperandBEntry = EditorData->AddVariable(TEXT("B"), FAnimNextParamType::GetType<int32>(), TEXT("2"));
	UE_RETURN_ON_ERROR(OperandBEntry != nullptr, TEXT("Could not create new variable in graph."));
	UAnimNextVariableEntry* ResultEntry = EditorData->AddVariable(TEXT("Result"), FAnimNextParamType::GetType<int32>(), TEXT("12"));
	UE_RETURN_ON_ERROR(ResultEntry != nullptr, TEXT("Could not create new variable in graph."));

	// Get event graph
	UAnimNextEventGraphEntry* EventGraph = Cast<UAnimNextEventGraphEntry>(EditorData->FindEntry(UUAFSystem_EditorData::DefaultEventGraphName));
	URigVMGraph* RigVMGraph = nullptr;
	URigVMNode* PrePhysicsEventNode = nullptr;
	
	if(EventGraph == nullptr)
	{
		EventGraph = EditorData->AddEventGraph(UUAFSystem_EditorData::DefaultEventGraphName, FRigUnit_AnimNextPrePhysicsEvent::StaticStruct());
		UE_RETURN_ON_ERROR(EventGraph != nullptr, TEXT("Could not create new event graph in asset."));

		RigVMGraph = EventGraph->GetRigVMGraph();
		UE_RETURN_ON_ERROR(RigVMGraph->GetNodes().Num() == 1, TEXT("Unexpected number of nodes in new event graph.")); // PrePhysics

		PrePhysicsEventNode = RigVMGraph->GetNodes()[0];
		UE_RETURN_ON_ERROR(PrePhysicsEventNode->GetEventName() == "PrePhysics", TEXT("Failed to find PrePhysics event node."));
	}
	else
	{
		RigVMGraph = EventGraph->GetRigVMGraph();
		UE_RETURN_ON_ERROR(RigVMGraph->GetNodes().Num() == 2, TEXT("Unexpected number of nodes in new event graph."));  // Initialize & PrePhysics

		PrePhysicsEventNode = RigVMGraph->GetNodes()[1];
		UE_RETURN_ON_ERROR(PrePhysicsEventNode->GetEventName() == "PrePhysics", TEXT("Failed to find PrePhysics event node."));
	}
	
	check(PrePhysicsEventNode);
	URigVMPin* ExecutePin = PrePhysicsEventNode->FindPin("ExecuteContext");
	UE_RETURN_ON_ERROR(ExecutePin != nullptr, TEXT("Could find initial execute pin."));

	URigVMController* Controller = EditorData->GetController(EventGraph->GetRigVMGraph());

	URigVMVariableNode* VariableANode = Controller->AddVariableNode("A", RigVMTypeUtils::Int32Type, nullptr, true, TEXT(""));
	UE_RETURN_ON_ERROR(VariableANode != nullptr, TEXT("Could not add get variable node."));
	URigVMVariableNode* VariableBNode = Controller->AddVariableNode("B", RigVMTypeUtils::Int32Type, nullptr, true, TEXT(""));
	UE_RETURN_ON_ERROR(VariableBNode != nullptr, TEXT("Could not add get variable node."));
	URigVMVariableNode* SetResultNode = Controller->AddVariableNode("Result", RigVMTypeUtils::Int32Type, nullptr, false, TEXT(""));
	UE_RETURN_ON_ERROR(SetResultNode != nullptr, TEXT("Could not add set variable node."));

	URigVMUnitNode* TestOpUnitNode = Controller->AddUnitNode(FAnimNextTests_TestOperation::StaticStruct());
	bool bLinkAAdded = Controller->AddLink(VariableANode->FindPin("Value"), TestOpUnitNode->FindPin("A"));
	UE_RETURN_ON_ERROR(bLinkAAdded, TEXT("Could not link variable node."));
	bool bLinkBAdded = Controller->AddLink(VariableBNode->FindPin("Value"), TestOpUnitNode->FindPin("B"));
	UE_RETURN_ON_ERROR(bLinkBAdded, TEXT("Could not link variable node."));
	bool bLinkResultAdded = Controller->AddLink(TestOpUnitNode->FindPin("Result"), SetResultNode->FindPin("Value"));
	UE_RETURN_ON_ERROR(bLinkResultAdded, TEXT("Could not link variable node."));

	bool bLinkExec1Added = Controller->AddLink(PrePhysicsEventNode->FindPin(FRigVMStruct::ExecuteContextName.ToString()), TestOpUnitNode->FindPin(FRigVMStruct::ExecuteContextName.ToString()));
	UE_RETURN_ON_ERROR(bLinkExec1Added, TEXT("Could not link variable node exec."));

	bool bLinkExec2Added = Controller->AddLink(TestOpUnitNode->FindPin(FRigVMStruct::ExecuteContextName.ToString()), SetResultNode->FindPin(FRigVMStruct::ExecuteContextName.ToString()));
	UE_RETURN_ON_ERROR(bLinkExec2Added, TEXT("Could not link variable node exec."));

	URigVMUnitNode* PrintResultUnitNode = Controller->AddUnitNode(FAnimNextTests_PrintResult::StaticStruct());
	bool bLinkExec3Added = Controller->AddLink(SetResultNode->FindPin(FRigVMStruct::ExecuteContextName.ToString()), PrintResultUnitNode->FindPin(FRigVMStruct::ExecuteContextName.ToString()));
	UE_RETURN_ON_ERROR(bLinkExec3Added, TEXT("Could not link variable node exec."));

	URigVMVariableNode* GetResultNode = Controller->AddVariableNode("Result", RigVMTypeUtils::Int32Type, nullptr, true, TEXT(""));
	UE_RETURN_ON_ERROR(GetResultNode != nullptr, TEXT("Could not add get variable node."));
	bool bLinkResult2Added = Controller->AddLink(GetResultNode->FindPin("Value"), PrintResultUnitNode->FindPin("Result"));
	UE_RETURN_ON_ERROR(bLinkResult2Added, TEXT("Could not link variable node."));

	TArray<FString> Messages;
	FRigVMRuntimeSettings RuntimeSettings;
	RuntimeSettings.SetLogFunction([&Messages](const FRigVMLogSettings& InLogSettings, const FRigVMExecuteContext* InContext, const FString& Message)
	{
		Messages.Add(Message);
	});
	Asset->GetRigVMExtendedExecuteContext().SetRuntimeSettings(RuntimeSettings);

	Asset->GetVM()->ExecuteVM(Asset->GetRigVMExtendedExecuteContext(), FRigUnit_AnimNextPrePhysicsEvent::DefaultEventName);

	UE_RETURN_ON_ERROR(Messages.Num() == 1, TEXT("unexpected number of messages"));
	UE_RETURN_ON_ERROR(Messages[0] == TEXT("Result = 3"), TEXT("unexpected result message"));
	
	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVariables_UOLBindings, "Animation.AnimNext.Variables.UOLBindings", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FVariables_UOLBindings::RunTest(const FString& InParameters)
{
	ON_SCOPE_EXIT{ FUtils::CleanupAfterTests(); };

	UFactory* Factory = NewObject<UFactory>(GetTransientPackage(), UUAFSystemFactory::StaticClass());
	UUAFSystem* Asset = CastChecked<UUAFSystem>(Factory->FactoryCreateNew(UUAFSystem::StaticClass(), GetTransientPackage(), TEXT("TestAsset"), RF_Transient, nullptr, nullptr, NAME_None));
	UE_RETURN_ON_ERROR(Asset != nullptr, "FVariables_UOLBindings -> Failed to create asset");

	UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(Asset);
	UE_RETURN_ON_ERROR(EditorData != nullptr, "FVariables_UOLBindings -> Asset has no editor data.");

	UniversalObjectLocator::FParseStringParams ParseStringParams;
	ParseStringParams.Flags |= UniversalObjectLocator::EParseStringFlags::ErrorMessaging;

	// Add variables
	UAnimNextVariableEntry* PropertyEntry = EditorData->AddVariable(TEXT("Property"), FAnimNextParamType::GetType<int32>(), TEXT("1"));
	UE_RETURN_ON_ERROR(PropertyEntry != nullptr, TEXT("Could not create new variable in graph."));
	TInstancedStruct<FAnimNextUniversalObjectLocatorBindingData> BindingA = TInstancedStruct<FAnimNextUniversalObjectLocatorBindingData>::Make();
	FAnimNextUniversalObjectLocatorBindingData& BindingDataA = BindingA.GetMutable<FAnimNextUniversalObjectLocatorBindingData>();
	BindingDataA.Type = FAnimNextUniversalObjectLocatorBindingType::Property;
	BindingDataA.Property = UAnimNextTestFuncLib::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAnimNextTestFuncLib, ValueA));
	UE_RETURN_ON_ERROR(BindingDataA.Property != nullptr, TEXT("Could not find property."));
	UE::UniversalObjectLocator::FParseStringResult ResultA = BindingDataA.Locator.TryParseString(TEXT("uobj://animobjfunc?&payload0=/Script/UAFTestSuite.AnimNextTestFuncLib:GetObj"), ParseStringParams);
	UE_RETURN_ON_ERROR(ResultA.bSuccess, TEXT("Could not parse UOL."));
	PropertyEntry->SetBinding(MoveTemp(BindingA));

	UAnimNextVariableEntry* ObjectAccessorEntry = EditorData->AddVariable(TEXT("ObjectAccessor"), FAnimNextParamType::GetType<int32>(), TEXT("2"));
	UE_RETURN_ON_ERROR(ObjectAccessorEntry != nullptr, TEXT("Could not create new variable in graph."));
	TInstancedStruct<FAnimNextUniversalObjectLocatorBindingData> BindingB = TInstancedStruct<FAnimNextUniversalObjectLocatorBindingData>::Make();
	FAnimNextUniversalObjectLocatorBindingData& BindingDataB = BindingB.GetMutable<FAnimNextUniversalObjectLocatorBindingData>();
	BindingDataB.Type = FAnimNextUniversalObjectLocatorBindingType::Function;
	BindingDataB.Function = UAnimNextTestFuncLib::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UAnimNextTestFuncLib, GetValueB));
	UE_RETURN_ON_ERROR(BindingDataB.Function != nullptr, TEXT("Could not find function."));
	UE::UniversalObjectLocator::FParseStringResult ResultB = BindingDataB.Locator.TryParseString(TEXT("uobj://animobjfunc?&payload0=/Script/UAFTestSuite.AnimNextTestFuncLib:GetObj"), ParseStringParams);
	UE_RETURN_ON_ERROR(ResultB.bSuccess, TEXT("Could not parse UOL."));
	ObjectAccessorEntry->SetBinding(MoveTemp(BindingB));

	UAnimNextVariableEntry* HoistedAccessorEntry = EditorData->AddVariable(TEXT("HoistedAccessor"), FAnimNextParamType::GetType<int32>(), TEXT("12"));
	UE_RETURN_ON_ERROR(HoistedAccessorEntry != nullptr, TEXT("Could not create new variable in graph."));
	TInstancedStruct<FAnimNextUniversalObjectLocatorBindingData> BindingC = TInstancedStruct<FAnimNextUniversalObjectLocatorBindingData>::Make();
	FAnimNextUniversalObjectLocatorBindingData& BindingDataC = BindingC.GetMutable<FAnimNextUniversalObjectLocatorBindingData>();
	BindingDataC.Type = FAnimNextUniversalObjectLocatorBindingType::HoistedFunction;
	BindingDataC.Function = UAnimNextTestFuncLib::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UAnimNextTestFuncLib, GetValueC));
	UE_RETURN_ON_ERROR(BindingDataC.Function != nullptr, TEXT("Could not find function."));
	UE::UniversalObjectLocator::FParseStringResult ResultC = BindingDataC.Locator.TryParseString(TEXT("uobj://animobjfunc?&payload0=/Script/UAFTestSuite.AnimNextTestFuncLib:GetObj"), ParseStringParams);
	UE_RETURN_ON_ERROR(ResultC.bSuccess, TEXT("Could not parse UOL."));
	HoistedAccessorEntry->SetBinding(MoveTemp(BindingC));

	// Get event graph
	UAnimNextEventGraphEntry* EventGraph = Cast<UAnimNextEventGraphEntry>(EditorData->FindEntry(UUAFSystem_EditorData::DefaultEventGraphName));
	URigVMGraph* RigVMGraph = nullptr;
	URigVMNode* PrePhysicsEventNode = nullptr;
	
	if(EventGraph == nullptr)
	{
		EventGraph = EditorData->AddEventGraph(UUAFSystem_EditorData::DefaultEventGraphName, FRigUnit_AnimNextPrePhysicsEvent::StaticStruct());
		UE_RETURN_ON_ERROR(EventGraph != nullptr, TEXT("Could not create new event graph in asset."));

		RigVMGraph = EventGraph->GetRigVMGraph();
		UE_RETURN_ON_ERROR(RigVMGraph->GetNodes().Num() == 1, TEXT("Unexpected number of nodes in new event graph.")); // PrePhysics

		PrePhysicsEventNode = RigVMGraph->GetNodes()[0];
		UE_RETURN_ON_ERROR(PrePhysicsEventNode->GetEventName() == "PrePhysics", TEXT("Failed to find PrePhysics event node."));
	}
	else
	{
		RigVMGraph = EventGraph->GetRigVMGraph();
		UE_RETURN_ON_ERROR(RigVMGraph->GetNodes().Num() == 2, TEXT("Unexpected number of nodes in new event graph."));  // Initialize & PrePhysics

		PrePhysicsEventNode = RigVMGraph->GetNodes()[1];
		UE_RETURN_ON_ERROR(PrePhysicsEventNode->GetEventName() == "PrePhysics", TEXT("Failed to find PrePhysics event node."));
	}
	
	check(PrePhysicsEventNode);
	URigVMPin* ExecutePin = PrePhysicsEventNode->FindPin("ExecuteContext");
	UE_RETURN_ON_ERROR(ExecutePin != nullptr, TEXT("Could find initial execute pin."));

	URigVMController* Controller = EditorData->GetController(EventGraph->GetRigVMGraph());

	URigVMVariableNode* PropertyEntryNode = Controller->AddVariableNode("Property", RigVMTypeUtils::Int32Type, nullptr, true, TEXT(""));
	UE_RETURN_ON_ERROR(PropertyEntryNode != nullptr, TEXT("Could not add get variable node."));
	URigVMVariableNode* ObjectAccessorEntryNode = Controller->AddVariableNode("ObjectAccessor", RigVMTypeUtils::Int32Type, nullptr, true, TEXT(""));
	UE_RETURN_ON_ERROR(ObjectAccessorEntryNode != nullptr, TEXT("Could not add get variable node."));
	URigVMVariableNode* HoistedAccessorEntryNode = Controller->AddVariableNode("HoistedAccessor", RigVMTypeUtils::Int32Type, nullptr, true, TEXT(""));
	UE_RETURN_ON_ERROR(HoistedAccessorEntryNode != nullptr, TEXT("Could not add get variable node."));

	URigVMUnitNode* PrintPropertyUnitNode = Controller->AddUnitNode(FAnimNextTests_PrintResult::StaticStruct());
	bool bLinkAAdded = Controller->AddLink(PropertyEntryNode->FindPin("Value"), PrintPropertyUnitNode->FindPin("Result"));
	UE_RETURN_ON_ERROR(bLinkAAdded, TEXT("Could not link variable node."));

	bool bLinkExec1Added = Controller->AddLink(PrePhysicsEventNode->FindPin(FRigVMStruct::ExecuteContextName.ToString()), PrintPropertyUnitNode->FindPin(FRigVMStruct::ExecuteContextName.ToString()));
	UE_RETURN_ON_ERROR(bLinkExec1Added, TEXT("Could not link variable node exec."));
	
	URigVMUnitNode* PrintObjectAccessorUnitNode = Controller->AddUnitNode(FAnimNextTests_PrintResult::StaticStruct());
	bool bLinkBAdded = Controller->AddLink(ObjectAccessorEntryNode->FindPin("Value"), PrintObjectAccessorUnitNode->FindPin("Result"));
	UE_RETURN_ON_ERROR(bLinkBAdded, TEXT("Could not link variable node."));
	
	bool bLinkExec2Added = Controller->AddLink(PrintPropertyUnitNode->FindPin(FRigVMStruct::ExecuteContextName.ToString()), PrintObjectAccessorUnitNode->FindPin(FRigVMStruct::ExecuteContextName.ToString()));
	UE_RETURN_ON_ERROR(bLinkExec2Added, TEXT("Could not link variable node exec."));

	URigVMUnitNode* PrintHoistedAccessorUnitNode = Controller->AddUnitNode(FAnimNextTests_PrintResult::StaticStruct());
	bool bLinkCAdded = Controller->AddLink(HoistedAccessorEntryNode->FindPin("Value"), PrintHoistedAccessorUnitNode->FindPin("Result"));
	UE_RETURN_ON_ERROR(bLinkCAdded, TEXT("Could not link variable node."));

	bool bLinkExec3Added = Controller->AddLink(PrintObjectAccessorUnitNode->FindPin(FRigVMStruct::ExecuteContextName.ToString()), PrintHoistedAccessorUnitNode->FindPin(FRigVMStruct::ExecuteContextName.ToString()));
	UE_RETURN_ON_ERROR(bLinkExec3Added, TEXT("Could not link variable node exec."));

	TArray<FString> Messages;
	FRigVMRuntimeSettings RuntimeSettings;
	RuntimeSettings.SetLogFunction([&Messages](const FRigVMLogSettings& InLogSettings, const FRigVMExecuteContext* InContext, const FString& Message)
	{
		Messages.Add(Message);
	});
	Asset->GetRigVMExtendedExecuteContext().SetRuntimeSettings(RuntimeSettings);

	FAnimNextModuleInstance Instance(Asset, NewObject<UUAFComponent>(), EAnimNextModuleInitMethod::None);
	Instance.Initialize(FUAFSystemFactoryParams());
	FUAFRigVMComponent& RigVMComponent = Instance.GetOrAddComponent<FUAFRigVMComponent>();

	// Execute bindings to fetch data
	RigVMComponent.CallEventByName(FAnimNextModuleContextData(Instance, FRigUnit_AnimNextExecuteBindings_GT::EventName, 0.0f));

	// Run PrePhysics event to print messages
	RigVMComponent.CallEventByName(FAnimNextModuleContextData(Instance, FRigUnit_AnimNextPrePhysicsEvent::DefaultEventName, 0.0f));

	UE_RETURN_ON_ERROR(Messages.Num() == 3, TEXT("unexpected number of messages"));
	UE_RETURN_ON_ERROR(Messages[0] == TEXT("Result = 23"), TEXT("unexpected result message"));
	UE_RETURN_ON_ERROR(Messages[1] == TEXT("Result = 42"), TEXT("unexpected result message"));
	UE_RETURN_ON_ERROR(Messages[2] == TEXT("Result = 12345"), TEXT("unexpected result message"));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVariables_Shared, "Animation.AnimNext.Variables.Shared", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

namespace Private
{
	struct FTestReadWrite
	{
		template<typename Type>
		static bool Test(FAutomationTestBase& Test, FAnimNextModuleInstance& InInstance, const TCHAR* InMemberName, const Type& InValue)
		{
			const FAnimNextVariableReference Reference = FAnimNextVariableReference::FromName(FName(InMemberName), FAnimNextParamTypeTestStruct::StaticStruct());
			Test.AddErrorIfFalse(Reference.IsValid(), *FString::Printf(TEXT("FVariables_Shared -> Variable reference to %s was invalid."), InMemberName));
			EPropertyBagResult SetResult = InInstance.SetVariable<Type>(Reference, InValue);
			Test.AddErrorIfFalse(SetResult == EPropertyBagResult::Success, *FString::Printf(TEXT("FVariables_Shared -> SetVariable call for %s failed."), InMemberName));
			Type TestValue;
			EPropertyBagResult GetResult = InInstance.GetVariable<Type>(Reference, TestValue);
			Test.AddErrorIfFalse(GetResult == EPropertyBagResult::Success, *FString::Printf(TEXT("FVariables_Shared -> GetVariable call for %s failed."), InMemberName));

			// Specialize comparisons for types that dont have operator==
			if constexpr(std::is_same_v<Type, FTransform>)
			{
				Test.AddErrorIfFalse(TestValue.Equals(InValue), *FString::Printf(TEXT("FVariables_Shared -> Variable value did not match for %s."), InMemberName));
			}
			else if constexpr(std::is_same_v<Type, TArray<FTransform>>)
			{
				Test.AddErrorIfFalse(Algo::Compare(TestValue, InValue, [](const FTransform& A, const FTransform& B){ return A.Equals(B); }), *FString::Printf(TEXT("FVariables_Shared -> Variable value did not match for %s."), InMemberName));
			}
			else
			{
				Test.AddErrorIfFalse(TestValue == InValue, *FString::Printf(TEXT("FVariables_Shared -> Variable value did not match for %s."), InMemberName));
			}
			return true;
		}
	};
}

bool FVariables_Shared::RunTest(const FString& InParameters)
{
	ON_SCOPE_EXIT{ FUtils::CleanupAfterTests(); };

	UFactory* Factory = NewObject<UFactory>(GetTransientPackage(), UUAFSystemFactory::StaticClass());
	UUAFSystem* Asset = CastChecked<UUAFSystem>(Factory->FactoryCreateNew(UUAFSystem::StaticClass(), GetTransientPackage(), TEXT("TestAsset"), RF_Transient, nullptr, nullptr, NAME_None));
	UE_RETURN_ON_ERROR(Asset != nullptr, "FVariables_Shared -> Failed to create asset");

	UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(Asset);
	UE_RETURN_ON_ERROR(EditorData != nullptr, "FVariables_Shared -> Asset has no editor data.");

	UUAFSharedVariablesEntry* Entry = EditorData->AddSharedVariablesStruct(FAnimNextParamTypeTestStruct::StaticStruct());
	UE_RETURN_ON_ERROR(Entry != nullptr, "FVariables_Shared -> Could not add shared variables struct.");

	FAnimNextModuleInstance Instance(Asset, EAnimNextModuleInitMethod::None);
	Instance.Initialize(FUAFSystemFactoryParams());

	Private::FTestReadWrite::Test<bool>(*this, Instance, TEXT("bBool"), true);
	Private::FTestReadWrite::Test<bool>(*this, Instance, TEXT("bBitfield0"), true);
	Private::FTestReadWrite::Test<bool>(*this, Instance, TEXT("bBitfield1"), true);
	Private::FTestReadWrite::Test<uint8>(*this, Instance, TEXT("Uint8"), 78);
	Private::FTestReadWrite::Test<int32>(*this, Instance, TEXT("Int32"), 4578990);
	Private::FTestReadWrite::Test<int64>(*this, Instance, TEXT("Int64"), 5235987347);
	Private::FTestReadWrite::Test<float>(*this, Instance, TEXT("Float"), 0.25f);
	Private::FTestReadWrite::Test<double>(*this, Instance, TEXT("Double"), 8756498576.0);
	Private::FTestReadWrite::Test<FName>(*this, Instance, TEXT("Name"), FName("TestName"));
	Private::FTestReadWrite::Test<FString>(*this, Instance, TEXT("String"), TEXT("TestString"));
	Private::FTestReadWrite::Test<EPropertyBagContainerType>(*this, Instance, TEXT("Enum"), EPropertyBagContainerType::Set);
	Private::FTestReadWrite::Test<FAnimNextParamType>(*this, Instance, TEXT("Struct"), FAnimNextParamType::GetType<FAnimNextParamType>());
	Private::FTestReadWrite::Test<FVector>(*this, Instance, TEXT("Vector"), FVector(1.0f, 2.0f, 3.0f));
	Private::FTestReadWrite::Test<FTransform>(*this, Instance, TEXT("Transform"), FTransform(FQuat::MakeFromRotator(FRotator(1.0f, 2.0f, 3.0f)), FVector(3.0f, 2.0f, 1.0f), FVector(1.0f, 2.0f, 3.0f)));
	Private::FTestReadWrite::Test<TObjectPtr<UObject>>(*this, Instance, TEXT("Object"), UUAFRigVMAsset::StaticClass()->GetDefaultObject());
	Private::FTestReadWrite::Test<TObjectPtr<UClass>>(*this, Instance, TEXT("Class"), UUAFRigVMAsset::StaticClass());
	Private::FTestReadWrite::Test<TSubclassOf<UObject>>(*this, Instance, TEXT("SubclassOf"), UUAFRigVMAsset::StaticClass());
	// TODO: Soft obj/class ptrs not supported yet in RigVM
//	Private::FTestReadWrite::Test<TSoftObjectPtr<UObject>>(*this, Instance, TEXT("SoftObjectPtr"), UAnimNextRigVMAsset::StaticClass()->GetDefaultObject());
//	Private::FTestReadWrite::Test<TSoftClassPtr<UObject>>(*this, Instance, TEXT("SoftClassPtr"), UAnimNextRigVMAsset::StaticClass());

	Private::FTestReadWrite::Test<TArray<bool>>(*this, Instance, TEXT("BoolArray"), { true });
	Private::FTestReadWrite::Test<TArray<uint8>>(*this, Instance, TEXT("Uint8Array"), { 78 });
	Private::FTestReadWrite::Test<TArray<int32>>(*this, Instance, TEXT("Int32Array"), { 4578990 });
	Private::FTestReadWrite::Test<TArray<int64>>(*this, Instance, TEXT("Int64Array"), { 5235987347 });
	Private::FTestReadWrite::Test<TArray<float>>(*this, Instance, TEXT("FloatArray"), { 0.25f });
	Private::FTestReadWrite::Test<TArray<double>>(*this, Instance, TEXT("DoubleArray"), { 8756498576.0 });
	Private::FTestReadWrite::Test<TArray<FName>>(*this, Instance, TEXT("NameArray"), { FName("TestName") });
	Private::FTestReadWrite::Test<TArray<FString>>(*this, Instance, TEXT("StringArray"), { TEXT("TestString") });
	Private::FTestReadWrite::Test<TArray<EPropertyBagContainerType>>(*this, Instance, TEXT("EnumArray"), { EPropertyBagContainerType::Set });
	Private::FTestReadWrite::Test<TArray<FAnimNextParamType>>(*this, Instance, TEXT("StructArray"), { FAnimNextParamType::GetType<FAnimNextParamType>() });
	Private::FTestReadWrite::Test<TArray<FVector>>(*this, Instance, TEXT("VectorArray"), { FVector(1.0f, 2.0f, 3.0f) });
	Private::FTestReadWrite::Test<TArray<FTransform>>(*this, Instance, TEXT("TransformArray"), { FTransform(FQuat::MakeFromRotator(FRotator(1.0f, 2.0f, 3.0f)), FVector(3.0f, 2.0f, 1.0f), FVector(1.0f, 2.0f, 3.0f)) });
	Private::FTestReadWrite::Test<TArray<TObjectPtr<UObject>>>(*this, Instance, TEXT("ObjectArray"), { UUAFRigVMAsset::StaticClass()->GetDefaultObject() });
	Private::FTestReadWrite::Test<TArray<TObjectPtr<UClass>>>(*this, Instance, TEXT("ClassArray"), { UUAFRigVMAsset::StaticClass() });
	Private::FTestReadWrite::Test<TArray<TSubclassOf<UObject>>>(*this, Instance, TEXT("SubclassOfArray"), { UUAFRigVMAsset::StaticClass() });
	// TODO: Soft obj/class ptrs not supported yet in RigVM
//	Private::FTestReadWrite::Test<TArray<TSoftObjectPtr<UObject>>>(*this, Instance, TEXT("SoftObjectPtrArray"), { UAnimNextRigVMAsset::StaticClass()->GetDefaultObject() });
//	Private::FTestReadWrite::Test<TArray<TSoftClassPtr<UObject>>>(*this, Instance, TEXT("SoftClassPtrArray"), { UAnimNextRigVMAsset::StaticClass() });

	bool bResult = Instance.AccessVariablesStruct<FAnimNextParamTypeTestStruct>([this](FAnimNextParamTypeTestStruct& InStruct)
	{
		auto TestValue = [this]<typename T0>(const TCHAR* InMemberName, const auto& InValue, const T0& InComparison)
		{
			// Specialize comparisons for types that dont have operator==
			if constexpr(std::is_same_v<T0, FTransform>)
			{
				AddErrorIfFalse(InComparison.Equals(InValue), *FString::Printf(TEXT("FVariables_Shared -> Set value does not match for %s."), InMemberName));
			}
			else if constexpr(std::is_same_v<T0, TArray<FTransform>>)
			{
				AddErrorIfFalse(Algo::Compare(InComparison, InValue, [](const FTransform& A, const FTransform& B){ return A.Equals(B); }), *FString::Printf(TEXT("FVariables_Shared -> Set value does not match for %s."), InMemberName));
			}
			else
			{
				AddErrorIfFalse(InValue == InComparison, *FString::Printf(TEXT("FVariables_Shared -> Set value does not match for %s."), InMemberName));
			}
		};

		TestValue(TEXT("bBool"), InStruct.bBool, true);
		TestValue(TEXT("bBitfield0"), InStruct.bBitfield0 != 0, true);
		TestValue(TEXT("bBitfield1"), InStruct.bBitfield1 != 0, true);
		TestValue(TEXT("Uint8"), InStruct.Uint8, 78);
		TestValue(TEXT("Int32"), InStruct.Int32, 4578990);
		TestValue(TEXT("Int64"), InStruct.Int64, 5235987347);
		TestValue(TEXT("Float"), InStruct.Float, 0.25f);
		TestValue(TEXT("Double"), InStruct.Double, 8756498576.0);
		TestValue(TEXT("Name"), InStruct.Name, FName("TestName"));
		TestValue(TEXT("String"), InStruct.String, FString("TestString"));
		TestValue(TEXT("Enum"), InStruct.Enum, EPropertyBagContainerType::Set);
		TestValue(TEXT("Struct"), InStruct.Struct, FAnimNextParamType::GetType<FAnimNextParamType>());
		TestValue(TEXT("Vector"), InStruct.Vector, FVector(1.0f, 2.0f, 3.0f));
		TestValue(TEXT("Transform"), InStruct.Transform, FTransform(FQuat::MakeFromRotator(FRotator(1.0f, 2.0f, 3.0f)), FVector(3.0f, 2.0f, 1.0f), FVector(1.0f, 2.0f, 3.0f)));
		TestValue(TEXT("Object"), InStruct.Object, UUAFRigVMAsset::StaticClass()->GetDefaultObject());
		TestValue(TEXT("Class"), InStruct.Class, UUAFRigVMAsset::StaticClass());
		TestValue(TEXT("SubclassOf"), InStruct.SubclassOf, UUAFRigVMAsset::StaticClass());
		// TODO: Soft obj/class ptrs not supported yet in RigVM
	//	TestValue(TEXT("SoftObjectPtr"), InStruct.SoftObjectPtr, UAnimNextRigVMAsset::StaticClass()->GetDefaultObject());
	//	TestValue(TEXT("SoftClassPtr"), InStruct.SoftClassPtr, UAnimNextRigVMAsset::StaticClass());
	});

	UE_RETURN_ON_ERROR(bResult, "FVariables_Shared -> Could not access variable struct on instance.");

	return true;
}

}

#endif	// WITH_DEV_AUTOMATION_TESTS

FAnimNextTests_TestOperation_Execute()
{
	Result = A + B;
}

FAnimNextTests_PrintResult_Execute()
{
	ExecuteContext.Logf(EMessageSeverity::Info, TEXT("Result = %d"), Result);
}
