// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextExecuteContext.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Param/ParamType.h"
#include "RigVMCore/RigVMStruct.h"
#include "Templates/SubclassOf.h"
#include "AnimNextVariablesTest.generated.h"

class UUAFComponent;

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNextParamTypeTestStruct
{
	GENERATED_BODY()

	UPROPERTY()
	bool bBool = false;

	UPROPERTY()
	uint8 bBitfield0 : 1 = false;

	UPROPERTY()
	uint8 bBitfield1 : 1 = false;
	
	UPROPERTY()
	uint8 Uint8 = 0;

	UPROPERTY()
	int32 Int32 = 0;

	UPROPERTY()
	int64 Int64 = 0;

	UPROPERTY()
	float Float = 0.0f;

	UPROPERTY()
	double Double = 0.0;

	UPROPERTY()
	FName Name;

	UPROPERTY()
	FString String;

	UPROPERTY()
	EPropertyBagContainerType Enum = EPropertyBagContainerType::None;

	UPROPERTY()
	FAnimNextParamType Struct;

	UPROPERTY()
	FVector Vector = FVector::ZeroVector;

	UPROPERTY()
	FTransform Transform;

	UPROPERTY()
	FQuat Quat = FQuat::Identity;

	UPROPERTY()
	TObjectPtr<UObject> Object;

	UPROPERTY()
	TObjectPtr<UClass> Class;

	UPROPERTY()
	TSubclassOf<UObject> SubclassOf;

	// TODO: Soft obj/class ptrs not supported yet in RigVM
//	UPROPERTY()
//	TSoftObjectPtr<UObject> SoftObjectPtr;

//	UPROPERTY()
//	TSoftClassPtr<UObject> SoftClassPtr;

	UPROPERTY()
	TArray<bool> BoolArray;

	UPROPERTY()
	TArray<uint8> Uint8Array;

	UPROPERTY()
	TArray<int32> Int32Array;

	UPROPERTY()
	TArray<int64> Int64Array;

	UPROPERTY()
	TArray<float> FloatArray;

	UPROPERTY()
	TArray<double> DoubleArray;

	UPROPERTY()
	TArray<FName> NameArray;

	UPROPERTY()
	TArray<FString> StringArray;

	UPROPERTY()
	TArray<EPropertyBagContainerType> EnumArray;

	UPROPERTY()
	TArray<FAnimNextParamType> StructArray;

	UPROPERTY()
	TArray<FVector> VectorArray;
	
	UPROPERTY()
	TArray<FTransform> TransformArray;

	UPROPERTY()
	TArray<FQuat> QuatArray;
	
	UPROPERTY()
	TArray<TObjectPtr<UObject>> ObjectArray;

	UPROPERTY()
	TArray<TObjectPtr<UClass>> ClassArray;

	UPROPERTY()
	TArray<TSubclassOf<UObject>> SubclassOfArray;

	// TODO: Soft obj/class ptrs not supported yet in RigVM
//	UPROPERTY()
//	TArray<TSoftObjectPtr<UObject>> SoftObjectPtrArray;

//	UPROPERTY()
//	TArray<TSoftClassPtr<UObject>> SoftClassPtrArray;
};

USTRUCT(meta=(ExecuteContext="FAnimNextExecuteContext", Hidden, DocumentationPolicy="None"))
struct FAnimNextTests_TestBase : public FRigVMStructMutable
{
	GENERATED_BODY()
};

USTRUCT()
struct FAnimNextTests_TestOperation : public FAnimNextTests_TestBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	virtual void Execute() override;

	UPROPERTY(meta=(Input))
	int32 A = 0;

	UPROPERTY(meta=(Input))
	int32 B = 0;

	UPROPERTY(meta=(Output))
	int32 Result = 0;
};

USTRUCT()
struct FAnimNextTests_PrintResult : public FAnimNextTests_TestBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	virtual void Execute() override;

	UPROPERTY(meta=(Input))
	int32 Result = 0;
};

UCLASS()
class UAnimNextTestFuncLib : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category=Test)
	static UAnimNextTestFuncLib* GetObj(UUAFComponent* InObj)
	{
		return GetMutableDefault<UAnimNextTestFuncLib>();
	}

	UFUNCTION(BlueprintCallable, Category=Test)
	int32 GetValueB()
	{
		return ValueB;
	}

	UFUNCTION(BlueprintCallable, Category=Test)
	static int32 GetValueC(UUAFComponent* InObj)
	{
		return GetDefault<UAnimNextTestFuncLib>()->ValueC;
	}

	UPROPERTY()
	int32 ValueA = 23;

	UPROPERTY()
	int32 ValueB = 42;

	UPROPERTY()
	int32 ValueC = 12345;
};
