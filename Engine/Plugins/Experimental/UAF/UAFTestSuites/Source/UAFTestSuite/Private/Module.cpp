// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

namespace UE::UAF::Tests
{

class FModule : public IModuleInterface
{
public:
};

}

IMPLEMENT_MODULE(UE::UAF::Tests::FModule, UAFTestSuite)
