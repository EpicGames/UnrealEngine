// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "AnimNextTest.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "UAF/AbstractSkeleton/SetBindingFactory.h"
#include "UAF/AbstractSkeleton/SetCollectionFactory.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetCollection.h"
#include "UAF/Attributes/AttributeBindingData.h"
#include "UAF/Attributes/EngineAttributes.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

namespace UE::UAF::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAttributeBindingDataTest, "Animation.UAF.Attributes.AttributeBindingData", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool FAttributeBindingDataTest::RunTest(const FString& InParameters)
	{
		ON_SCOPE_EXIT{ FUtils::CleanupAfterTests(); };

		UFactory* BindingFactory = NewObject<UFactory>(GetTransientPackage(), UAbstractSkeletonSetBindingFactory::StaticClass());
		UE_RETURN_ON_ERROR(BindingFactory != nullptr, "Failed to create factory");

		UAbstractSkeletonSetBinding* Binding = CastChecked<UAbstractSkeletonSetBinding>(BindingFactory->FactoryCreateNew(UAbstractSkeletonSetBinding::StaticClass(), GetTransientPackage(), TEXT("TestBindingAsset"), RF_Transient, nullptr, nullptr, NAME_None));
		UE_RETURN_ON_ERROR(Binding != nullptr, "Failed to create binding asset");

		{
			FAttributeBindingDataPtr BindingData = MakeAttributeBindingData(Binding, nullptr);
			AddErrorIfFalse(!BindingData.IsValid(), TEXT("Attribute binding data should be invalid"));
		}

		USkeleton* Skeleton = NewObject<USkeleton>(GetTransientPackage(), USkeleton::StaticClass(), TEXT("TestSkeletonAsset"), RF_Transient);
		UE_RETURN_ON_ERROR(Skeleton != nullptr, "Failed to create skeleton asset");

		{
			// Not terribly clean, we cast away the 'const' to modify the skeleton
			FReferenceSkeleton& RefSkeleton = const_cast<FReferenceSkeleton&>(Skeleton->GetReferenceSkeleton());
			FReferenceSkeletonModifier SkeletonModifier(RefSkeleton, Skeleton);

			SkeletonModifier.Add(FMeshBoneInfo(TEXT("Root"), TEXT("Root"), INDEX_NONE), FTransform::Identity);
			SkeletonModifier.Add(FMeshBoneInfo(TEXT("Pelvis"), TEXT("Root"), 0), FTransform::Identity);
			SkeletonModifier.Add(FMeshBoneInfo(TEXT("Spine_01"), TEXT("Spine_01"), 1), FTransform::Identity);
			SkeletonModifier.Add(FMeshBoneInfo(TEXT("Spine_02"), TEXT("Spine_02"), 2), FTransform::Identity);
			SkeletonModifier.Add(FMeshBoneInfo(TEXT("Head"), TEXT("Head"), 3), FTransform::Identity);
			SkeletonModifier.Add(FMeshBoneInfo(TEXT("LeftArm"), TEXT("LeftArm"), 3), FTransform::Identity);
			SkeletonModifier.Add(FMeshBoneInfo(TEXT("RightArm"), TEXT("RightArm"), 3), FTransform::Identity);

			// Not in any sets
			SkeletonModifier.Add(FMeshBoneInfo(TEXT("Camera"), TEXT("Root"), 0), FTransform::Identity);

			// When our modifier is destroyed here, it will rebuild the skeleton
		}

		UE_RETURN_ON_ERROR(Skeleton->GetReferenceSkeleton().GetNum() == 8, "Failed to build skeleton asset");

		bool bSuccess = Binding->SetSkeleton(Skeleton);
		UE_RETURN_ON_ERROR(bSuccess, "Failed to set skeleton on our binding");

		{
			// We have the everything set
			FAttributeBindingDataPtr BindingData = MakeAttributeBindingData(Binding, nullptr);
			
			UE_RETURN_ON_ERROR(BindingData.IsValid(), TEXT("Attribute binding data should be valid"));
			AddErrorIfFalse(!BindingData->IsEmpty(), TEXT("Attribute binding data should not be empty"));
			AddErrorIfFalse(BindingData->NumNamedSets() == 1, TEXT("Unexpected number of named sets"));
			AddErrorIfFalse(BindingData->NumLODs() == 1, TEXT("Unexpected number of LODs"));
			AddErrorIfFalse(BindingData->NumAttributes(FBoneTransformAnimationAttribute::StaticStruct()) == 8, TEXT("Unexpected number of bone attributes"));
			AddErrorIfFalse(BindingData->NumAttributes(FFloatAnimationAttribute::StaticStruct()) == 0, TEXT("Unexpected number of float attributes"));
			AddErrorIfFalse(BindingData->NumAttributes(FIntegerAnimationAttribute::StaticStruct()) == 0, TEXT("Unexpected number of integer attributes"));

			// The everything set
			{
				FAttributeNamedSetPtr EverythingSet = BindingData->FindNamedSet(NAME_None);
				UE_RETURN_ON_ERROR(EverythingSet.IsValid(), TEXT("Everything named set should be valid"));
				AddErrorIfFalse(!EverythingSet->IsEmpty(), TEXT("Everything named set should not be empty"));
				AddErrorIfFalse(EverythingSet->GetName().IsNone(), TEXT("Unexpected named set name"));
				AddErrorIfFalse(EverythingSet->GetLOD() == 0, TEXT("Unexpected named set LOD"));
				AddErrorIfFalse(EverythingSet->NumTypedSets() == 1, TEXT("Unexpected named set number of typed set"));
				AddErrorIfFalse(EverythingSet->NumLODs() == 1, TEXT("Unexpected named set number of LODs"));
				AddErrorIfFalse(!EverythingSet->AtLOD(INDEX_NONE).IsValid(), TEXT("Unexpected named set ptr at LOD"));
				AddErrorIfFalse(EverythingSet->AtLOD(0) == EverythingSet, TEXT("Unexpected named set ptr at LOD"));
				AddErrorIfFalse(!EverythingSet->AtLOD(1).IsValid(), TEXT("Unexpected named set ptr at LOD"));

				{
					const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();

					FAttributeTypedSetPtr BoneSet = EverythingSet->FindTypedSet(FBoneTransformAnimationAttribute::StaticStruct());
					UE_RETURN_ON_ERROR(BoneSet.IsValid(), TEXT("Typed set should be valid"));
					AddErrorIfFalse(!BoneSet->IsEmpty(), TEXT("Typed set should not be empty"));
					AddErrorIfFalse(BoneSet->GetType() == FBoneTransformAnimationAttribute::StaticStruct(), TEXT("Unexpected typed set type"));
					AddErrorIfFalse(BoneSet->GetLOD() == 0, TEXT("Unexpected typed set LOD"));
					AddErrorIfFalse(BoneSet->Num() == 8, TEXT("Unexpected typed set number of attributes"));
					AddErrorIfFalse(BoneSet->NumLODs() == 1, TEXT("Unexpected typed set number of LODs"));
					AddErrorIfFalse(!BoneSet->AtLOD(INDEX_NONE).IsValid(), TEXT("Unexpected typed set ptr at LOD"));
					AddErrorIfFalse(BoneSet->AtLOD(0) == BoneSet, TEXT("Unexpected typed set ptr at LOD"));
					AddErrorIfFalse(!BoneSet->AtLOD(1).IsValid(), TEXT("Unexpected typed set ptr at LOD"));

					AddErrorIfFalse(!BoneSet->FindKey(NAME_None).IsValid(), TEXT("FindKey returned an unexpected value"));
					AddErrorIfFalse(BoneSet->FindName(FAttributeSetKey()) == NAME_None, TEXT("FindName returned an unexpected value"));
					AddErrorIfFalse(!BoneSet->FindIndex(NAME_None).IsValid(), TEXT("FindIndex returned an unexpected value"));
					AddErrorIfFalse(!BoneSet->FindIndex(FAttributeSetKey()).IsValid(), TEXT("FindIndex returned an unexpected value"));
					AddErrorIfFalse(!BoneSet->FindBindingIndex(NAME_None).IsValid(), TEXT("FindBindingIndex returned an unexpected value"));

					AddErrorIfFalse(!BoneSet->GetIndex(FAttributeBindingIndex()).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
					AddErrorIfFalse(BoneSet->GetIndex(FAttributeBindingIndex(0)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
					AddErrorIfFalse(BoneSet->GetIndex(FAttributeBindingIndex(7)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
					AddErrorIfFalse(!BoneSet->GetIndex(FAttributeBindingIndex(8)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
					AddErrorIfFalse(BoneSet->GetName(BoneSet->GetIndex(FAttributeBindingIndex(0))) == RefSkeleton.GetBoneName(0), TEXT("GetSetIndex returned an unexpected value"));
					AddErrorIfFalse(BoneSet->GetName(BoneSet->GetIndex(FAttributeBindingIndex(7))) == RefSkeleton.GetBoneName(7), TEXT("GetSetIndex returned an unexpected value"));
					AddErrorIfFalse(!BoneSet->GetBindingIndex(FAttributeSetIndex()).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
					AddErrorIfFalse(BoneSet->GetBindingIndex(FAttributeSetIndex(0)).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
					AddErrorIfFalse(BoneSet->GetBindingIndex(FAttributeSetIndex(7)).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
					AddErrorIfFalse(!BoneSet->GetBindingIndex(FAttributeSetIndex(8)).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
					AddErrorIfFalse(BoneSet->GetBindingIndex(BoneSet->FindIndex(TEXT("Root"))) == 0, TEXT("GetBindingIndex returned an unexpected value"));
					AddErrorIfFalse(BoneSet->GetBindingIndex(BoneSet->FindIndex(TEXT("RightArm"))) == 6, TEXT("GetBindingIndex returned an unexpected value"));
					AddErrorIfFalse(BoneSet->GetName(FAttributeSetIndex()) == NAME_None, TEXT("GetName returned an unexpected value"));
					AddErrorIfFalse(!BoneSet->GetKey(FAttributeSetIndex()).IsValid(), TEXT("GetKey returned an unexpected value"));
					AddErrorIfFalse(!BoneSet->GetParentIndex(FAttributeSetIndex()).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));
					AddErrorIfFalse(!BoneSet->GetParentIndex(FAttributeSetIndex(0)).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));
					AddErrorIfFalse(BoneSet->GetName(BoneSet->GetParentIndex(BoneSet->FindIndex(TEXT("Pelvis")))) == TEXT("Root"), TEXT("GetParentSetIndex returned an unexpected value"));
					AddErrorIfFalse(BoneSet->GetName(BoneSet->GetParentIndex(BoneSet->FindIndex(TEXT("Head")))) == TEXT("Spine_02"), TEXT("GetParentSetIndex returned an unexpected value"));
					AddErrorIfFalse(BoneSet->GetName(BoneSet->GetParentIndex(BoneSet->FindIndex(TEXT("Camera")))) == TEXT("Root"), TEXT("GetParentSetIndex returned an unexpected value"));
					AddErrorIfFalse(!BoneSet->GetParentIndex(FAttributeSetIndex(8)).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));

					for (int32 SkeletonBoneIndex = 0; SkeletonBoneIndex < RefSkeleton.GetNum(); ++SkeletonBoneIndex)
					{
						const FName BoneName = RefSkeleton.GetBoneName(SkeletonBoneIndex);
						const int32 ParentBoneIndex = RefSkeleton.GetParentIndex(SkeletonBoneIndex);
						const FName ParentBoneName = ParentBoneIndex != INDEX_NONE ? RefSkeleton.GetBoneName(ParentBoneIndex) : NAME_None;

						const FAttributeSetKey Key = BoneSet->FindKey(BoneName);
						const FAttributeSetIndex Index = BoneSet->FindIndex(BoneName);
						const FAttributeSetIndex ParentIndex = BoneSet->FindIndex(ParentBoneName);
						const FAttributeBindingIndex BindingIndex = BoneSet->FindBindingIndex(BoneName);

						AddErrorIfFalse(Key.IsValid(), TEXT("FindKey returned an unexpected value"));
						AddErrorIfFalse(Index.IsValid(), TEXT("FindIndex returned an unexpected value"));
						AddErrorIfFalse(BindingIndex.IsValid(), TEXT("FindBindingIndex returned an unexpected value"));
						if (ParentBoneName.IsNone())
						{
							AddErrorIfFalse(!ParentIndex.IsValid(), TEXT("FindIndex returned an unexpected value"));
						}
						else
						{
							AddErrorIfFalse(ParentIndex.IsValid(), TEXT("FindIndex returned an unexpected value"));
						}

						AddErrorIfFalse(BoneSet->FindIndex(Key) == Index, TEXT("FindIndex failed to retrieve attribute metadata"));
						AddErrorIfFalse(BoneSet->FindName(Key) == BoneName, TEXT("FindName failed to retrieve attribute metadata"));

						AddErrorIfFalse(BoneSet->GetName(Index) == BoneName, TEXT("GetName failed to retrieve attribute metadata"));
						AddErrorIfFalse(BoneSet->GetKey(Index) == Key, TEXT("GetKey failed to retrieve attribute metadata"));
						AddErrorIfFalse(BoneSet->GetParentIndex(Index) == ParentIndex, TEXT("GetParentSetIndex failed to retrieve attribute metadata"));
						AddErrorIfFalse(BoneSet->GetIndex(BindingIndex) == Index, TEXT("FindBindingIndex failed to retrieve attribute metadata"));
					}
				}
			}
		}

		UFactory* SetCollectionFactory = NewObject<UFactory>(GetTransientPackage(), UAbstractSkeletonSetCollectionFactory::StaticClass());
		UE_RETURN_ON_ERROR(SetCollectionFactory != nullptr, "Failed to create factory");

		{
			UAbstractSkeletonSetCollection* SetCollection = CastChecked<UAbstractSkeletonSetCollection>(SetCollectionFactory->FactoryCreateNew(UAbstractSkeletonSetCollection::StaticClass(), GetTransientPackage(), TEXT("TestSetCollectionAsset"), RF_Transient, nullptr, nullptr, NAME_None));
			UE_RETURN_ON_ERROR(SetCollection != nullptr, "Failed to create set collection asset");

			SetCollection->AddSet(TEXT("FullBody"), NAME_None);
			SetCollection->AddSet(TEXT("UpperBody"), TEXT("FullBody"));

			bSuccess = Binding->SetSetCollection(SetCollection);
			UE_RETURN_ON_ERROR(bSuccess, "Failed to set set collection on our binding");

			{
				// We have the everything set
				FAttributeBindingDataPtr BindingData = MakeAttributeBindingData(Binding, nullptr);

				UE_RETURN_ON_ERROR(BindingData.IsValid(), TEXT("Attribute binding data should be valid"));
				AddErrorIfFalse(!BindingData->IsEmpty(), TEXT("Attribute binding data should not be empty"));
				AddErrorIfFalse(BindingData->NumNamedSets() == 3, TEXT("Unexpected number of named sets")); // We expect None (Everything), FullBody, UpperBody
				AddErrorIfFalse(BindingData->NumLODs() == 1, TEXT("Unexpected number of LODs"));
				AddErrorIfFalse(BindingData->NumAttributes(FBoneTransformAnimationAttribute::StaticStruct()) == 8, TEXT("Unexpected number of bone attributes"));
				AddErrorIfFalse(BindingData->NumAttributes(FFloatAnimationAttribute::StaticStruct()) == 0, TEXT("Unexpected number of float attributes"));
				AddErrorIfFalse(BindingData->NumAttributes(FIntegerAnimationAttribute::StaticStruct()) == 0, TEXT("Unexpected number of integer attributes"));

				// The everything set
				{
					FAttributeNamedSetPtr EverythingSet = BindingData->FindNamedSet(NAME_None);
					UE_RETURN_ON_ERROR(EverythingSet.IsValid(), TEXT("Everything named set should be valid"));
					AddErrorIfFalse(!EverythingSet->IsEmpty(), TEXT("Everything named set should not be empty"));
					AddErrorIfFalse(EverythingSet->GetName().IsNone(), TEXT("Unexpected named set name"));
					AddErrorIfFalse(EverythingSet->GetLOD() == 0, TEXT("Unexpected named set LOD"));
					AddErrorIfFalse(EverythingSet->NumTypedSets() == 1, TEXT("Unexpected named set number of typed set"));
					AddErrorIfFalse(EverythingSet->NumLODs() == 1, TEXT("Unexpected named set number of LODs"));
					AddErrorIfFalse(!EverythingSet->AtLOD(INDEX_NONE).IsValid(), TEXT("Unexpected named set ptr at LOD"));
					AddErrorIfFalse(EverythingSet->AtLOD(0) == EverythingSet, TEXT("Unexpected named set ptr at LOD"));
					AddErrorIfFalse(!EverythingSet->AtLOD(1).IsValid(), TEXT("Unexpected named set ptr at LOD"));

					{
						const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();

						FAttributeTypedSetPtr BoneSet = EverythingSet->FindTypedSet(FBoneTransformAnimationAttribute::StaticStruct());
						UE_RETURN_ON_ERROR(BoneSet.IsValid(), TEXT("Typed set should be valid"));
						AddErrorIfFalse(!BoneSet->IsEmpty(), TEXT("Typed set should not be empty"));
						AddErrorIfFalse(BoneSet->GetType() == FBoneTransformAnimationAttribute::StaticStruct(), TEXT("Unexpected typed set type"));
						AddErrorIfFalse(BoneSet->GetLOD() == 0, TEXT("Unexpected typed set LOD"));
						AddErrorIfFalse(BoneSet->Num() == 8, TEXT("Unexpected typed set number of attributes"));
						AddErrorIfFalse(BoneSet->NumLODs() == 1, TEXT("Unexpected typed set number of LODs"));
						AddErrorIfFalse(!BoneSet->AtLOD(INDEX_NONE).IsValid(), TEXT("Unexpected typed set ptr at LOD"));
						AddErrorIfFalse(BoneSet->AtLOD(0) == BoneSet, TEXT("Unexpected typed set ptr at LOD"));
						AddErrorIfFalse(!BoneSet->AtLOD(1).IsValid(), TEXT("Unexpected typed set ptr at LOD"));

						AddErrorIfFalse(!BoneSet->FindKey(NAME_None).IsValid(), TEXT("FindKey returned an unexpected value"));
						AddErrorIfFalse(BoneSet->FindName(FAttributeSetKey()) == NAME_None, TEXT("FindName returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->FindIndex(NAME_None).IsValid(), TEXT("FindIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->FindIndex(FAttributeSetKey()).IsValid(), TEXT("FindIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->FindBindingIndex(NAME_None).IsValid(), TEXT("FindBindingIndex returned an unexpected value"));

						AddErrorIfFalse(!BoneSet->GetIndex(FAttributeBindingIndex()).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetIndex(FAttributeBindingIndex(0)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetIndex(FAttributeBindingIndex(7)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetIndex(FAttributeBindingIndex(8)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetName(BoneSet->GetIndex(FAttributeBindingIndex(0))) == RefSkeleton.GetBoneName(0), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetName(BoneSet->GetIndex(FAttributeBindingIndex(7))) == RefSkeleton.GetBoneName(7), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetBindingIndex(FAttributeSetIndex()).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetBindingIndex(FAttributeSetIndex(0)).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetBindingIndex(FAttributeSetIndex(7)).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetBindingIndex(FAttributeSetIndex(8)).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetBindingIndex(BoneSet->FindIndex(TEXT("Root"))) == 0, TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetBindingIndex(BoneSet->FindIndex(TEXT("RightArm"))) == 6, TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetName(FAttributeSetIndex()) == NAME_None, TEXT("GetName returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetKey(FAttributeSetIndex()).IsValid(), TEXT("GetKey returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetParentIndex(FAttributeSetIndex()).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetParentIndex(FAttributeSetIndex(0)).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetName(BoneSet->GetParentIndex(BoneSet->FindIndex(TEXT("Pelvis")))) == TEXT("Root"), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetName(BoneSet->GetParentIndex(BoneSet->FindIndex(TEXT("Head")))) == TEXT("Spine_02"), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetName(BoneSet->GetParentIndex(BoneSet->FindIndex(TEXT("Camera")))) == TEXT("Root"), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetParentIndex(FAttributeSetIndex(8)).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));

						for (int32 SkeletonBoneIndex = 0; SkeletonBoneIndex < RefSkeleton.GetNum(); ++SkeletonBoneIndex)
						{
							const FName BoneName = RefSkeleton.GetBoneName(SkeletonBoneIndex);
							const int32 ParentBoneIndex = RefSkeleton.GetParentIndex(SkeletonBoneIndex);
							const FName ParentBoneName = ParentBoneIndex != INDEX_NONE ? RefSkeleton.GetBoneName(ParentBoneIndex) : NAME_None;

							const FAttributeSetKey Key = BoneSet->FindKey(BoneName);
							const FAttributeSetIndex Index = BoneSet->FindIndex(BoneName);
							const FAttributeSetIndex ParentIndex = BoneSet->FindIndex(ParentBoneName);
							const FAttributeBindingIndex BindingIndex = BoneSet->FindBindingIndex(BoneName);

							AddErrorIfFalse(Key.IsValid(), TEXT("FindKey returned an unexpected value"));
							AddErrorIfFalse(Index.IsValid(), TEXT("FindIndex returned an unexpected value"));
							AddErrorIfFalse(BindingIndex.IsValid(), TEXT("FindBindingIndex returned an unexpected value"));
							if (ParentBoneName.IsNone())
							{
								AddErrorIfFalse(!ParentIndex.IsValid(), TEXT("FindIndex returned an unexpected value"));
							}
							else
							{
								AddErrorIfFalse(ParentIndex.IsValid(), TEXT("FindIndex returned an unexpected value"));
							}

							AddErrorIfFalse(BoneSet->FindIndex(Key) == Index, TEXT("FindIndex failed to retrieve attribute metadata"));
							AddErrorIfFalse(BoneSet->FindName(Key) == BoneName, TEXT("FindName failed to retrieve attribute metadata"));

							AddErrorIfFalse(BoneSet->GetName(Index) == BoneName, TEXT("GetName failed to retrieve attribute metadata"));
							AddErrorIfFalse(BoneSet->GetKey(Index) == Key, TEXT("GetKey failed to retrieve attribute metadata"));
							AddErrorIfFalse(BoneSet->GetParentIndex(Index) == ParentIndex, TEXT("GetParentSetIndex failed to retrieve attribute metadata"));
							AddErrorIfFalse(BoneSet->GetIndex(BindingIndex) == Index, TEXT("FindBindingIndex failed to retrieve attribute metadata"));
						}
					}
				}
			}

			Binding->AddBoneToSet(TEXT("Root"), TEXT("FullBody"));
			Binding->AddBoneToSet(TEXT("Pelvis"), TEXT("FullBody"));
			Binding->AddBoneToSet(TEXT("Spine_01"), TEXT("FullBody"));
			Binding->AddBoneToSet(TEXT("Spine_02"), TEXT("FullBody"));
			Binding->AddAttributeToSet(FAnimationAttributeIdentifier(TEXT("AttrFloat0"), INDEX_NONE, NAME_None, FFloatAnimationAttribute::StaticStruct()), TEXT("FullBody"));

			Binding->AddBoneToSet(TEXT("Head"), TEXT("UpperBody"));
			Binding->AddBoneToSet(TEXT("LeftArm"), TEXT("UpperBody"));
			Binding->AddBoneToSet(TEXT("RightArm"), TEXT("UpperBody"));
			Binding->AddAttributeToSet(FAnimationAttributeIdentifier(TEXT("AttrFloat1"), INDEX_NONE, NAME_None, FFloatAnimationAttribute::StaticStruct()), TEXT("UpperBody"));

			TArray<FName> FloatAttributeNames;
			FloatAttributeNames.Add(TEXT("AttrFloat0"));
			FloatAttributeNames.Add(TEXT("AttrFloat1"));

			{
				FAttributeBindingDataPtr BindingData = MakeAttributeBindingData(Binding, nullptr);
				UE_RETURN_ON_ERROR(BindingData.IsValid(), TEXT("Attribute binding data should be valid"));
				AddErrorIfFalse(!BindingData->IsEmpty(), TEXT("Attribute binding data should not be empty"));
				AddErrorIfFalse(BindingData->NumNamedSets() == 3, TEXT("Unexpected number of named sets"));
				AddErrorIfFalse(BindingData->NumLODs() == 1, TEXT("Unexpected number of LODs"));
				AddErrorIfFalse(BindingData->NumAttributes(FBoneTransformAnimationAttribute::StaticStruct()) == 8, TEXT("Unexpected number of bone attributes"));
				AddErrorIfFalse(BindingData->NumAttributes(FFloatAnimationAttribute::StaticStruct()) == 2, TEXT("Unexpected number of float attributes"));
				AddErrorIfFalse(BindingData->NumAttributes(FIntegerAnimationAttribute::StaticStruct()) == 0, TEXT("Unexpected number of integer attributes"));

				// The everything set
				{
					FAttributeNamedSetPtr EverythingSet = BindingData->FindNamedSet(NAME_None);
					UE_RETURN_ON_ERROR(EverythingSet.IsValid(), TEXT("Everything named set should be valid"));
					AddErrorIfFalse(!EverythingSet->IsEmpty(), TEXT("Everything named set should not be empty"));
					AddErrorIfFalse(EverythingSet->GetName().IsNone(), TEXT("Unexpected named set name"));
					AddErrorIfFalse(EverythingSet->GetLOD() == 0, TEXT("Unexpected named set LOD"));
					AddErrorIfFalse(EverythingSet->NumTypedSets() == 2, TEXT("Unexpected named set number of typed set"));
					AddErrorIfFalse(EverythingSet->NumLODs() == 1, TEXT("Unexpected named set number of LODs"));
					AddErrorIfFalse(!EverythingSet->AtLOD(INDEX_NONE).IsValid(), TEXT("Unexpected named set ptr at LOD"));
					AddErrorIfFalse(EverythingSet->AtLOD(0) == EverythingSet, TEXT("Unexpected named set ptr at LOD"));
					AddErrorIfFalse(!EverythingSet->AtLOD(1).IsValid(), TEXT("Unexpected named set ptr at LOD"));

					{
						const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();

						FAttributeTypedSetPtr BoneSet = EverythingSet->FindTypedSet(FBoneTransformAnimationAttribute::StaticStruct());
						UE_RETURN_ON_ERROR(BoneSet.IsValid(), TEXT("Typed set should be valid"));
						AddErrorIfFalse(!BoneSet->IsEmpty(), TEXT("Typed set should not be empty"));
						AddErrorIfFalse(BoneSet->GetType() == FBoneTransformAnimationAttribute::StaticStruct(), TEXT("Unexpected typed set type"));
						AddErrorIfFalse(BoneSet->GetLOD() == 0, TEXT("Unexpected typed set LOD"));
						AddErrorIfFalse(BoneSet->Num() == 8, TEXT("Unexpected typed set number of attributes"));
						AddErrorIfFalse(BoneSet->NumLODs() == 1, TEXT("Unexpected typed set number of LODs"));
						AddErrorIfFalse(!BoneSet->AtLOD(INDEX_NONE).IsValid(), TEXT("Unexpected typed set ptr at LOD"));
						AddErrorIfFalse(BoneSet->AtLOD(0) == BoneSet, TEXT("Unexpected typed set ptr at LOD"));
						AddErrorIfFalse(!BoneSet->AtLOD(1).IsValid(), TEXT("Unexpected typed set ptr at LOD"));

						AddErrorIfFalse(!BoneSet->FindKey(NAME_None).IsValid(), TEXT("FindKey returned an unexpected value"));
						AddErrorIfFalse(BoneSet->FindName(FAttributeSetKey()) == NAME_None, TEXT("FindName returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->FindIndex(NAME_None).IsValid(), TEXT("FindIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->FindIndex(FAttributeSetKey()).IsValid(), TEXT("FindIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->FindBindingIndex(NAME_None).IsValid(), TEXT("FindBindingIndex returned an unexpected value"));

						AddErrorIfFalse(!BoneSet->GetIndex(FAttributeBindingIndex()).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetIndex(FAttributeBindingIndex(0)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetIndex(FAttributeBindingIndex(7)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetIndex(FAttributeBindingIndex(8)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetName(BoneSet->GetIndex(FAttributeBindingIndex(0))) == RefSkeleton.GetBoneName(0), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetName(BoneSet->GetIndex(FAttributeBindingIndex(6))) == RefSkeleton.GetBoneName(6), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetBindingIndex(FAttributeSetIndex()).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetBindingIndex(FAttributeSetIndex(0)).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetBindingIndex(FAttributeSetIndex(7)).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetBindingIndex(FAttributeSetIndex(8)).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetBindingIndex(BoneSet->FindIndex(TEXT("Root"))) == 0, TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetBindingIndex(BoneSet->FindIndex(TEXT("RightArm"))) == 6, TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetName(FAttributeSetIndex()) == NAME_None, TEXT("GetName returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetKey(FAttributeSetIndex()).IsValid(), TEXT("GetKey returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetParentIndex(FAttributeSetIndex()).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetParentIndex(FAttributeSetIndex(0)).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetName(BoneSet->GetParentIndex(BoneSet->FindIndex(TEXT("Pelvis")))) == TEXT("Root"), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetName(BoneSet->GetParentIndex(BoneSet->FindIndex(TEXT("Head")))) == TEXT("Spine_02"), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetName(BoneSet->GetParentIndex(BoneSet->FindIndex(TEXT("Camera")))) == TEXT("Root"), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetParentIndex(FAttributeSetIndex(8)).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));

						for (int32 SkeletonBoneIndex = 0; SkeletonBoneIndex < RefSkeleton.GetNum(); ++SkeletonBoneIndex)
						{
							const FName BoneName = RefSkeleton.GetBoneName(SkeletonBoneIndex);
							const int32 ParentBoneIndex = RefSkeleton.GetParentIndex(SkeletonBoneIndex);
							const FName ParentBoneName = ParentBoneIndex != INDEX_NONE ? RefSkeleton.GetBoneName(ParentBoneIndex) : NAME_None;

							const FAttributeSetKey Key = BoneSet->FindKey(BoneName);
							const FAttributeSetIndex Index = BoneSet->FindIndex(BoneName);
							const FAttributeSetIndex ParentIndex = BoneSet->FindIndex(ParentBoneName);
							const FAttributeBindingIndex BindingIndex = BoneSet->FindBindingIndex(BoneName);

							AddErrorIfFalse(Key.IsValid(), TEXT("FindKey returned an unexpected value"));
							AddErrorIfFalse(Index.IsValid(), TEXT("FindIndex returned an unexpected value"));
							AddErrorIfFalse(BindingIndex.IsValid(), TEXT("FindBindingIndex returned an unexpected value"));
							if (ParentBoneName.IsNone())
							{
								AddErrorIfFalse(!ParentIndex.IsValid(), TEXT("FindIndex returned an unexpected value"));
							}
							else
							{
								AddErrorIfFalse(ParentIndex.IsValid(), TEXT("FindIndex returned an unexpected value"));
							}

							AddErrorIfFalse(BoneSet->FindIndex(Key) == Index, TEXT("FindIndex failed to retrieve attribute metadata"));
							AddErrorIfFalse(BoneSet->FindName(Key) == BoneName, TEXT("FindName failed to retrieve attribute metadata"));

							AddErrorIfFalse(BoneSet->GetName(Index) == BoneName, TEXT("GetName failed to retrieve attribute metadata"));
							AddErrorIfFalse(BoneSet->GetKey(Index) == Key, TEXT("GetKey failed to retrieve attribute metadata"));
							AddErrorIfFalse(BoneSet->GetParentIndex(Index) == ParentIndex, TEXT("GetParentSetIndex failed to retrieve attribute metadata"));
							AddErrorIfFalse(BoneSet->GetIndex(BindingIndex) == Index, TEXT("FindBindingIndex failed to retrieve attribute metadata"));
						}
					}

					{
						FAttributeTypedSetPtr FloatSet = EverythingSet->FindTypedSet(FFloatAnimationAttribute::StaticStruct());
						UE_RETURN_ON_ERROR(FloatSet.IsValid(), TEXT("Typed set should be valid"));
						AddErrorIfFalse(!FloatSet->IsEmpty(), TEXT("Typed set should not be empty"));
						AddErrorIfFalse(FloatSet->GetType() == FFloatAnimationAttribute::StaticStruct(), TEXT("Unexpected typed set type"));
						AddErrorIfFalse(FloatSet->GetLOD() == 0, TEXT("Unexpected typed set LOD"));
						AddErrorIfFalse(FloatSet->Num() == 2, TEXT("Unexpected typed set number of attributes"));
						AddErrorIfFalse(FloatSet->NumLODs() == 1, TEXT("Unexpected typed set number of LODs"));
						AddErrorIfFalse(!FloatSet->AtLOD(INDEX_NONE).IsValid(), TEXT("Unexpected typed set ptr at LOD"));
						AddErrorIfFalse(FloatSet->AtLOD(0) == FloatSet, TEXT("Unexpected typed set ptr at LOD"));
						AddErrorIfFalse(!FloatSet->AtLOD(1).IsValid(), TEXT("Unexpected typed set ptr at LOD"));

						AddErrorIfFalse(!FloatSet->FindKey(NAME_None).IsValid(), TEXT("FindKey returned an unexpected value"));
						AddErrorIfFalse(FloatSet->FindName(FAttributeSetKey()) == NAME_None, TEXT("FindName returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->FindIndex(NAME_None).IsValid(), TEXT("FindIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->FindIndex(FAttributeSetKey()).IsValid(), TEXT("FindIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->FindBindingIndex(NAME_None).IsValid(), TEXT("FindBindingIndex returned an unexpected value"));

						AddErrorIfFalse(!FloatSet->GetIndex(FAttributeBindingIndex()).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetIndex(FAttributeBindingIndex(0)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetIndex(FAttributeBindingIndex(1)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetIndex(FAttributeBindingIndex(2)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetName(FloatSet->GetIndex(FAttributeBindingIndex(0))) == TEXT("AttrFloat0"), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetName(FloatSet->GetIndex(FAttributeBindingIndex(1))) == TEXT("AttrFloat1"), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetBindingIndex(FAttributeSetIndex()).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetBindingIndex(FAttributeSetIndex(0)).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetBindingIndex(FAttributeSetIndex(1)).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetBindingIndex(FAttributeSetIndex(2)).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetBindingIndex(FloatSet->FindIndex(TEXT("AttrFloat0"))) == 0, TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetBindingIndex(FloatSet->FindIndex(TEXT("AttrFloat1"))) == 1, TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetName(FAttributeSetIndex()) == NAME_None, TEXT("GetName returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetKey(FAttributeSetIndex()).IsValid(), TEXT("GetKey returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetParentIndex(FAttributeSetIndex()).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetParentIndex(FAttributeSetIndex(0)).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetParentIndex(FAttributeSetIndex(1)).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetParentIndex(FAttributeSetIndex(2)).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));

						for (int32 FloatAttributeIndex = 0; FloatAttributeIndex < FloatAttributeNames.Num(); ++FloatAttributeIndex)
						{
							const FName AttributeName = FloatAttributeNames[FloatAttributeIndex];

							const FAttributeSetKey Key = FloatSet->FindKey(AttributeName);
							const FAttributeSetIndex Index = FloatSet->FindIndex(AttributeName);
							const FAttributeBindingIndex BindingIndex = FloatSet->FindBindingIndex(AttributeName);

							AddErrorIfFalse(Key.IsValid(), TEXT("FindKey returned an unexpected value"));
							AddErrorIfFalse(Index.IsValid(), TEXT("FindIndex returned an unexpected value"));
							AddErrorIfFalse(BindingIndex.IsValid(), TEXT("FindBindingIndex returned an unexpected value"));

							AddErrorIfFalse(FloatSet->FindIndex(Key) == Index, TEXT("FindIndex failed to retrieve attribute metadata"));
							AddErrorIfFalse(FloatSet->FindName(Key) == AttributeName, TEXT("FindName failed to retrieve attribute metadata"));

							AddErrorIfFalse(FloatSet->GetName(Index) == AttributeName, TEXT("GetName failed to retrieve attribute metadata"));
							AddErrorIfFalse(FloatSet->GetKey(Index) == Key, TEXT("GetKey failed to retrieve attribute metadata"));
							AddErrorIfFalse(!FloatSet->GetParentIndex(Index).IsValid(), TEXT("GetParentSetIndex failed to retrieve attribute metadata"));
							AddErrorIfFalse(FloatSet->GetIndex(BindingIndex) == Index, TEXT("FindBindingIndex failed to retrieve attribute metadata"));
						}
					}
				}

				{
					FAttributeNamedSetPtr FullBodySet = BindingData->FindNamedSet(TEXT("FullBody"));
					UE_RETURN_ON_ERROR(FullBodySet.IsValid(), TEXT("FullBody named set should be valid"));
					AddErrorIfFalse(!FullBodySet->IsEmpty(), TEXT("FullBody named set should not be empty"));
					AddErrorIfFalse(FullBodySet->GetName() == TEXT("FullBody"), TEXT("Unexpected named set name"));
					AddErrorIfFalse(FullBodySet->GetLOD() == 0, TEXT("Unexpected named set LOD"));
					AddErrorIfFalse(FullBodySet->NumTypedSets() == 2, TEXT("Unexpected named set number of typed set"));
					AddErrorIfFalse(FullBodySet->NumLODs() == 1, TEXT("Unexpected named set number of LODs"));
					AddErrorIfFalse(!FullBodySet->AtLOD(INDEX_NONE).IsValid(), TEXT("Unexpected named set ptr at LOD"));
					AddErrorIfFalse(FullBodySet->AtLOD(0) == FullBodySet, TEXT("Unexpected named set ptr at LOD"));
					AddErrorIfFalse(!FullBodySet->AtLOD(1).IsValid(), TEXT("Unexpected named set ptr at LOD"));

					{
						const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();

						FAttributeTypedSetPtr BoneSet = FullBodySet->FindTypedSet(FBoneTransformAnimationAttribute::StaticStruct());
						UE_RETURN_ON_ERROR(BoneSet.IsValid(), TEXT("Typed set should be valid"));
						AddErrorIfFalse(!BoneSet->IsEmpty(), TEXT("Typed set should not be empty"));
						AddErrorIfFalse(BoneSet->GetType() == FBoneTransformAnimationAttribute::StaticStruct(), TEXT("Unexpected typed set type"));
						AddErrorIfFalse(BoneSet->GetLOD() == 0, TEXT("Unexpected typed set LOD"));
						AddErrorIfFalse(BoneSet->Num() == 7, TEXT("Unexpected typed set number of attributes"));
						AddErrorIfFalse(BoneSet->NumLODs() == 1, TEXT("Unexpected typed set number of LODs"));
						AddErrorIfFalse(!BoneSet->AtLOD(INDEX_NONE).IsValid(), TEXT("Unexpected typed set ptr at LOD"));
						AddErrorIfFalse(BoneSet->AtLOD(0) == BoneSet, TEXT("Unexpected typed set ptr at LOD"));
						AddErrorIfFalse(!BoneSet->AtLOD(1).IsValid(), TEXT("Unexpected typed set ptr at LOD"));

						AddErrorIfFalse(!BoneSet->FindKey(NAME_None).IsValid(), TEXT("FindKey returned an unexpected value"));
						AddErrorIfFalse(BoneSet->FindName(FAttributeSetKey()) == NAME_None, TEXT("FindName returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->FindIndex(NAME_None).IsValid(), TEXT("FindIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->FindIndex(FAttributeSetKey()).IsValid(), TEXT("FindIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->FindBindingIndex(NAME_None).IsValid(), TEXT("FindBindingIndex returned an unexpected value"));

						AddErrorIfFalse(!BoneSet->GetIndex(FAttributeBindingIndex()).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetIndex(FAttributeBindingIndex(0)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetIndex(FAttributeBindingIndex(6)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetIndex(FAttributeBindingIndex(7)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetName(BoneSet->GetIndex(FAttributeBindingIndex(0))) == RefSkeleton.GetBoneName(0), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetName(BoneSet->GetIndex(FAttributeBindingIndex(6))) == RefSkeleton.GetBoneName(6), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetBindingIndex(FAttributeSetIndex()).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetBindingIndex(FAttributeSetIndex(0)).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetBindingIndex(FAttributeSetIndex(6)).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetBindingIndex(FAttributeSetIndex(7)).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetBindingIndex(BoneSet->FindIndex(TEXT("Root"))) == 0, TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetBindingIndex(BoneSet->FindIndex(TEXT("RightArm"))) == 6, TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetName(FAttributeSetIndex()) == NAME_None, TEXT("GetName returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetKey(FAttributeSetIndex()).IsValid(), TEXT("GetKey returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetParentIndex(FAttributeSetIndex()).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetParentIndex(FAttributeSetIndex(0)).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetName(BoneSet->GetParentIndex(BoneSet->FindIndex(TEXT("Pelvis")))) == TEXT("Root"), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetName(BoneSet->GetParentIndex(BoneSet->FindIndex(TEXT("Head")))) == TEXT("Spine_02"), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetParentIndex(FAttributeSetIndex(7)).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));

						for (int32 SkeletonBoneIndex = 0; SkeletonBoneIndex < RefSkeleton.GetNum(); ++SkeletonBoneIndex)
						{
							const FName BoneName = RefSkeleton.GetBoneName(SkeletonBoneIndex);
							const int32 ParentBoneIndex = RefSkeleton.GetParentIndex(SkeletonBoneIndex);
							const FName ParentBoneName = ParentBoneIndex != INDEX_NONE ? RefSkeleton.GetBoneName(ParentBoneIndex) : NAME_None;

							const FAttributeSetKey Key = BoneSet->FindKey(BoneName);
							const FAttributeSetIndex Index = BoneSet->FindIndex(BoneName);
							const FAttributeSetIndex ParentIndex = BoneSet->FindIndex(ParentBoneName);
							const FAttributeBindingIndex BindingIndex = BoneSet->FindBindingIndex(BoneName);

							AddErrorIfFalse(BindingIndex.IsValid(), TEXT("FindBindingIndex returned an unexpected value"));

							if (Binding->IsBoneInSet(BoneName))	// check whole binding since full body is fully specified
							{
								AddErrorIfFalse(Key.IsValid(), TEXT("FindKey returned an unexpected value"));
								AddErrorIfFalse(Index.IsValid(), TEXT("FindIndex returned an unexpected value"));
								if (ParentBoneName.IsNone())
								{
									AddErrorIfFalse(!ParentIndex.IsValid(), TEXT("FindIndex returned an unexpected value"));
								}
								else
								{
									AddErrorIfFalse(ParentIndex.IsValid(), TEXT("FindIndex returned an unexpected value"));
								}

								AddErrorIfFalse(BoneSet->FindIndex(Key) == Index, TEXT("FindIndex failed to retrieve attribute metadata"));
								AddErrorIfFalse(BoneSet->FindName(Key) == BoneName, TEXT("FindName failed to retrieve attribute metadata"));

								AddErrorIfFalse(BoneSet->GetName(Index) == BoneName, TEXT("GetName failed to retrieve attribute metadata"));
								AddErrorIfFalse(BoneSet->GetKey(Index) == Key, TEXT("GetKey failed to retrieve attribute metadata"));
								AddErrorIfFalse(BoneSet->GetParentIndex(Index) == ParentIndex, TEXT("GetParentSetIndex failed to retrieve attribute metadata"));
								AddErrorIfFalse(BoneSet->GetIndex(BindingIndex) == Index, TEXT("FindBindingIndex failed to retrieve attribute metadata"));
							}
							else
							{
								AddErrorIfFalse(!Key.IsValid(), TEXT("FindKey returned an unexpected value"));
								AddErrorIfFalse(!Index.IsValid(), TEXT("FindIndex returned an unexpected value"));
								if (ParentBoneName.IsNone())
								{
									AddErrorIfFalse(!ParentIndex.IsValid(), TEXT("FindIndex returned an unexpected value"));
								}
								else
								{
									AddErrorIfFalse(ParentIndex.IsValid(), TEXT("FindIndex returned an unexpected value"));
								}
							}
						}
					}

					{
						FAttributeTypedSetPtr FloatSet = FullBodySet->FindTypedSet(FFloatAnimationAttribute::StaticStruct());
						UE_RETURN_ON_ERROR(FloatSet.IsValid(), TEXT("Typed set should be valid"));
						AddErrorIfFalse(!FloatSet->IsEmpty(), TEXT("Typed set should not be empty"));
						AddErrorIfFalse(FloatSet->GetType() == FFloatAnimationAttribute::StaticStruct(), TEXT("Unexpected typed set type"));
						AddErrorIfFalse(FloatSet->GetLOD() == 0, TEXT("Unexpected typed set LOD"));
						AddErrorIfFalse(FloatSet->Num() == 2, TEXT("Unexpected typed set number of attributes"));
						AddErrorIfFalse(FloatSet->NumLODs() == 1, TEXT("Unexpected typed set number of LODs"));
						AddErrorIfFalse(!FloatSet->AtLOD(INDEX_NONE).IsValid(), TEXT("Unexpected typed set ptr at LOD"));
						AddErrorIfFalse(FloatSet->AtLOD(0) == FloatSet, TEXT("Unexpected typed set ptr at LOD"));
						AddErrorIfFalse(!FloatSet->AtLOD(1).IsValid(), TEXT("Unexpected typed set ptr at LOD"));

						AddErrorIfFalse(!FloatSet->FindKey(NAME_None).IsValid(), TEXT("FindKey returned an unexpected value"));
						AddErrorIfFalse(FloatSet->FindName(FAttributeSetKey()) == NAME_None, TEXT("FindName returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->FindIndex(NAME_None).IsValid(), TEXT("FindIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->FindIndex(FAttributeSetKey()).IsValid(), TEXT("FindIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->FindBindingIndex(NAME_None).IsValid(), TEXT("FindBindingIndex returned an unexpected value"));

						AddErrorIfFalse(!FloatSet->GetIndex(FAttributeBindingIndex()).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetIndex(FAttributeBindingIndex(0)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetIndex(FAttributeBindingIndex(1)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetIndex(FAttributeBindingIndex(2)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetName(FloatSet->GetIndex(FAttributeBindingIndex(0))) == TEXT("AttrFloat0"), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetName(FloatSet->GetIndex(FAttributeBindingIndex(1))) == TEXT("AttrFloat1"), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetBindingIndex(FAttributeSetIndex()).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetBindingIndex(FAttributeSetIndex(0)).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetBindingIndex(FAttributeSetIndex(1)).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetBindingIndex(FAttributeSetIndex(2)).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetBindingIndex(FloatSet->FindIndex(TEXT("AttrFloat0"))) == 0, TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetBindingIndex(FloatSet->FindIndex(TEXT("AttrFloat1"))) == 1, TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetName(FAttributeSetIndex()) == NAME_None, TEXT("GetName returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetKey(FAttributeSetIndex()).IsValid(), TEXT("GetKey returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetParentIndex(FAttributeSetIndex()).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetParentIndex(FAttributeSetIndex(0)).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetParentIndex(FAttributeSetIndex(1)).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetParentIndex(FAttributeSetIndex(2)).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));

						for (int32 FloatAttributeIndex = 0; FloatAttributeIndex < FloatAttributeNames.Num(); ++FloatAttributeIndex)
						{
							const FName AttributeName = FloatAttributeNames[FloatAttributeIndex];

							const FAttributeSetKey Key = FloatSet->FindKey(AttributeName);
							const FAttributeSetIndex Index = FloatSet->FindIndex(AttributeName);
							const FAttributeBindingIndex BindingIndex = FloatSet->FindBindingIndex(AttributeName);

							AddErrorIfFalse(Key.IsValid(), TEXT("FindKey returned an unexpected value"));
							AddErrorIfFalse(Index.IsValid(), TEXT("FindIndex returned an unexpected value"));
							AddErrorIfFalse(BindingIndex.IsValid(), TEXT("FindBindingIndex returned an unexpected value"));

							AddErrorIfFalse(FloatSet->FindIndex(Key) == Index, TEXT("FindIndex failed to retrieve attribute metadata"));
							AddErrorIfFalse(FloatSet->FindName(Key) == AttributeName, TEXT("FindName failed to retrieve attribute metadata"));

							AddErrorIfFalse(FloatSet->GetName(Index) == AttributeName, TEXT("GetName failed to retrieve attribute metadata"));
							AddErrorIfFalse(FloatSet->GetKey(Index) == Key, TEXT("GetKey failed to retrieve attribute metadata"));
							AddErrorIfFalse(!FloatSet->GetParentIndex(Index).IsValid(), TEXT("GetParentSetIndex failed to retrieve attribute metadata"));
							AddErrorIfFalse(FloatSet->GetIndex(BindingIndex) == Index, TEXT("FindBindingIndex failed to retrieve attribute metadata"));
						}
					}
				}

				{
					FAttributeNamedSetPtr UpperBodySet = BindingData->FindNamedSet(TEXT("UpperBody"));
					UE_RETURN_ON_ERROR(UpperBodySet.IsValid(), TEXT("UpperBody named set should be valid"));
					AddErrorIfFalse(!UpperBodySet->IsEmpty(), TEXT("UpperBody named set should not be empty"));
					AddErrorIfFalse(UpperBodySet->GetName() == TEXT("UpperBody"), TEXT("Unexpected named set name"));
					AddErrorIfFalse(UpperBodySet->GetLOD() == 0, TEXT("Unexpected named set LOD"));
					AddErrorIfFalse(UpperBodySet->NumTypedSets() == 2, TEXT("Unexpected named set number of typed set"));
					AddErrorIfFalse(UpperBodySet->NumLODs() == 1, TEXT("Unexpected named set number of LODs"));
					AddErrorIfFalse(!UpperBodySet->AtLOD(INDEX_NONE).IsValid(), TEXT("Unexpected named set ptr at LOD"));
					AddErrorIfFalse(UpperBodySet->AtLOD(0) == UpperBodySet, TEXT("Unexpected named set ptr at LOD"));
					AddErrorIfFalse(!UpperBodySet->AtLOD(1).IsValid(), TEXT("Unexpected named set ptr at LOD"));

					{
						const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();

						FAttributeTypedSetPtr BoneSet = UpperBodySet->FindTypedSet(FBoneTransformAnimationAttribute::StaticStruct());
						UE_RETURN_ON_ERROR(BoneSet.IsValid(), TEXT("Typed set should be valid"));
						AddErrorIfFalse(!BoneSet->IsEmpty(), TEXT("Typed set should not be empty"));
						AddErrorIfFalse(BoneSet->GetType() == FBoneTransformAnimationAttribute::StaticStruct(), TEXT("Unexpected typed set type"));
						AddErrorIfFalse(BoneSet->GetLOD() == 0, TEXT("Unexpected typed set LOD"));
						AddErrorIfFalse(BoneSet->Num() == 3, TEXT("Unexpected typed set number of attributes"));
						AddErrorIfFalse(BoneSet->NumLODs() == 1, TEXT("Unexpected typed set number of LODs"));
						AddErrorIfFalse(!BoneSet->AtLOD(INDEX_NONE).IsValid(), TEXT("Unexpected typed set ptr at LOD"));
						AddErrorIfFalse(BoneSet->AtLOD(0) == BoneSet, TEXT("Unexpected typed set ptr at LOD"));
						AddErrorIfFalse(!BoneSet->AtLOD(1).IsValid(), TEXT("Unexpected typed set ptr at LOD"));

						AddErrorIfFalse(!BoneSet->FindKey(NAME_None).IsValid(), TEXT("FindKey returned an unexpected value"));
						AddErrorIfFalse(BoneSet->FindName(FAttributeSetKey()) == NAME_None, TEXT("FindName returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->FindIndex(NAME_None).IsValid(), TEXT("FindIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->FindIndex(FAttributeSetKey()).IsValid(), TEXT("FindIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->FindBindingIndex(NAME_None).IsValid(), TEXT("FindBindingIndex returned an unexpected value"));

						AddErrorIfFalse(!BoneSet->GetIndex(FAttributeBindingIndex()).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetIndex(FAttributeBindingIndex(0)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetIndex(FAttributeBindingIndex(6)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetIndex(FAttributeBindingIndex(7)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetName(BoneSet->GetIndex(FAttributeBindingIndex(6))) == RefSkeleton.GetBoneName(6), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetBindingIndex(FAttributeSetIndex()).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetBindingIndex(FAttributeSetIndex(0)).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetBindingIndex(FAttributeSetIndex(2)).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetBindingIndex(FAttributeSetIndex(3)).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetBindingIndex(BoneSet->FindIndex(TEXT("RightArm"))) == 6, TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(BoneSet->GetName(FAttributeSetIndex()) == NAME_None, TEXT("GetName returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetKey(FAttributeSetIndex()).IsValid(), TEXT("GetKey returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetParentIndex(FAttributeSetIndex()).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetParentIndex(FAttributeSetIndex(0)).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetParentIndex(FAttributeSetIndex(1)).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetParentIndex(FAttributeSetIndex(2)).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(!BoneSet->GetParentIndex(FAttributeSetIndex(3)).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));

						for (int32 SkeletonBoneIndex = 0; SkeletonBoneIndex < RefSkeleton.GetNum(); ++SkeletonBoneIndex)
						{
							const FName BoneName = RefSkeleton.GetBoneName(SkeletonBoneIndex);
							const int32 ParentBoneIndex = RefSkeleton.GetParentIndex(SkeletonBoneIndex);
							const FName ParentBoneName = ParentBoneIndex != INDEX_NONE ? RefSkeleton.GetBoneName(ParentBoneIndex) : NAME_None;

							const FAttributeSetKey Key = BoneSet->FindKey(BoneName);
							const FAttributeSetIndex Index = BoneSet->FindIndex(BoneName);
							const FAttributeSetIndex ParentIndex = BoneSet->FindIndex(ParentBoneName);
							const FAttributeBindingIndex BindingIndex = BoneSet->FindBindingIndex(BoneName);

							AddErrorIfFalse(BindingIndex.IsValid(), TEXT("FindBindingIndex returned an unexpected value"));

							if (Binding->IsBoneInSet(BoneName, UpperBodySet->GetName()))
							{
								AddErrorIfFalse(Key.IsValid(), TEXT("FindKey returned an unexpected value"));
								AddErrorIfFalse(Index.IsValid(), TEXT("FindIndex returned an unexpected value"));
								AddErrorIfFalse(!ParentIndex.IsValid(), TEXT("FindIndex returned an unexpected value"));

								AddErrorIfFalse(BoneSet->FindIndex(Key) == Index, TEXT("FindIndex failed to retrieve attribute metadata"));
								AddErrorIfFalse(BoneSet->FindName(Key) == BoneName, TEXT("FindName failed to retrieve attribute metadata"));

								AddErrorIfFalse(BoneSet->GetName(Index) == BoneName, TEXT("GetName failed to retrieve attribute metadata"));
								AddErrorIfFalse(BoneSet->GetKey(Index) == Key, TEXT("GetKey failed to retrieve attribute metadata"));
								AddErrorIfFalse(BoneSet->GetParentIndex(Index) == ParentIndex, TEXT("GetParentSetIndex failed to retrieve attribute metadata"));
								AddErrorIfFalse(BoneSet->GetIndex(BindingIndex) == Index, TEXT("FindBindingIndex failed to retrieve attribute metadata"));
							}
							else
							{
								AddErrorIfFalse(!Key.IsValid(), TEXT("FindKey returned an unexpected value"));
								AddErrorIfFalse(!Index.IsValid(), TEXT("FindIndex returned an unexpected value"));
								AddErrorIfFalse(!ParentIndex.IsValid(), TEXT("FindIndex returned an unexpected value"));
							}
						}
					}

					{
						FAttributeTypedSetPtr FloatSet = UpperBodySet->FindTypedSet(FFloatAnimationAttribute::StaticStruct());
						UE_RETURN_ON_ERROR(FloatSet.IsValid(), TEXT("Typed set should be valid"));
						AddErrorIfFalse(!FloatSet->IsEmpty(), TEXT("Typed set should not be empty"));
						AddErrorIfFalse(FloatSet->GetType() == FFloatAnimationAttribute::StaticStruct(), TEXT("Unexpected typed set type"));
						AddErrorIfFalse(FloatSet->GetLOD() == 0, TEXT("Unexpected typed set LOD"));
						AddErrorIfFalse(FloatSet->Num() == 1, TEXT("Unexpected typed set number of attributes"));
						AddErrorIfFalse(FloatSet->NumLODs() == 1, TEXT("Unexpected typed set number of LODs"));
						AddErrorIfFalse(!FloatSet->AtLOD(INDEX_NONE).IsValid(), TEXT("Unexpected typed set ptr at LOD"));
						AddErrorIfFalse(FloatSet->AtLOD(0) == FloatSet, TEXT("Unexpected typed set ptr at LOD"));
						AddErrorIfFalse(!FloatSet->AtLOD(1).IsValid(), TEXT("Unexpected typed set ptr at LOD"));

						AddErrorIfFalse(!FloatSet->FindKey(NAME_None).IsValid(), TEXT("FindKey returned an unexpected value"));
						AddErrorIfFalse(FloatSet->FindName(FAttributeSetKey()) == NAME_None, TEXT("FindName returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->FindIndex(NAME_None).IsValid(), TEXT("FindIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->FindIndex(FAttributeSetKey()).IsValid(), TEXT("FindIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->FindBindingIndex(NAME_None).IsValid(), TEXT("FindBindingIndex returned an unexpected value"));

						AddErrorIfFalse(!FloatSet->GetIndex(FAttributeBindingIndex()).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetIndex(FAttributeBindingIndex(0)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetIndex(FAttributeBindingIndex(1)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetIndex(FAttributeBindingIndex(2)).IsValid(), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetName(FloatSet->GetIndex(FAttributeBindingIndex(1))) == TEXT("AttrFloat1"), TEXT("GetSetIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetBindingIndex(FAttributeSetIndex()).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetBindingIndex(FAttributeSetIndex(0)).IsValid(), TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetBindingIndex(FloatSet->FindIndex(TEXT("AttrFloat1"))) == 1, TEXT("GetBindingIndex returned an unexpected value"));
						AddErrorIfFalse(FloatSet->GetName(FAttributeSetIndex()) == NAME_None, TEXT("GetName returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetKey(FAttributeSetIndex()).IsValid(), TEXT("GetKey returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetParentIndex(FAttributeSetIndex()).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetParentIndex(FAttributeSetIndex(0)).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));
						AddErrorIfFalse(!FloatSet->GetParentIndex(FAttributeSetIndex(1)).IsValid(), TEXT("GetParentSetIndex returned an unexpected value"));

						for (int32 FloatAttributeIndex = 0; FloatAttributeIndex < FloatAttributeNames.Num(); ++FloatAttributeIndex)
						{
							const FName AttributeName = FloatAttributeNames[FloatAttributeIndex];
							FAnimationAttributeIdentifier Attribute(AttributeName, INDEX_NONE, NAME_None, FFloatAnimationAttribute::StaticStruct());

							const FAttributeSetKey Key = FloatSet->FindKey(AttributeName);
							const FAttributeSetIndex Index = FloatSet->FindIndex(AttributeName);
							const FAttributeBindingIndex BindingIndex = FloatSet->FindBindingIndex(AttributeName);

							AddErrorIfFalse(BindingIndex.IsValid(), TEXT("FindBindingIndex returned an unexpected value"));

							if (Binding->IsAttributeInSet(Attribute, UpperBodySet->GetName()))
							{
								AddErrorIfFalse(Key.IsValid(), TEXT("FindKey returned an unexpected value"));
								AddErrorIfFalse(Index.IsValid(), TEXT("FindIndex returned an unexpected value"));

								AddErrorIfFalse(FloatSet->FindIndex(Key) == Index, TEXT("FindIndex failed to retrieve attribute metadata"));
								AddErrorIfFalse(FloatSet->FindName(Key) == AttributeName, TEXT("FindName failed to retrieve attribute metadata"));

								AddErrorIfFalse(FloatSet->GetName(Index) == AttributeName, TEXT("GetName failed to retrieve attribute metadata"));
								AddErrorIfFalse(FloatSet->GetKey(Index) == Key, TEXT("GetKey failed to retrieve attribute metadata"));
								AddErrorIfFalse(!FloatSet->GetParentIndex(Index).IsValid(), TEXT("GetParentSetIndex failed to retrieve attribute metadata"));
								AddErrorIfFalse(FloatSet->GetIndex(BindingIndex) == Index, TEXT("FindBindingIndex failed to retrieve attribute metadata"));
							}
							else
							{
								AddErrorIfFalse(!Key.IsValid(), TEXT("FindKey returned an unexpected value"));
								AddErrorIfFalse(!Index.IsValid(), TEXT("FindIndex returned an unexpected value"));
							}
						}
					}
				}
			}
		}

		{
			//  Set Collection:
			//  
			//	FullBody
			//	\_ UpperBody
			//	   \_ Neck
			//
			// Set Binding: Assign a single bone 'Spine_01' to UpperBody
			// Verify that FullBody & UpperBody contains that bone
			// Verify that Neck is empty

			UAbstractSkeletonSetCollection* SetCollection = CastChecked<UAbstractSkeletonSetCollection>(SetCollectionFactory->FactoryCreateNew(UAbstractSkeletonSetCollection::StaticClass(), GetTransientPackage(), TEXT("TestSetCollectionAsset"), RF_Transient, nullptr, nullptr, NAME_None));
			UE_RETURN_ON_ERROR(SetCollection != nullptr, "Failed to create set collection asset");

			SetCollection->AddSet(TEXT("FullBody"), NAME_None);
			SetCollection->AddSet(TEXT("UpperBody"), TEXT("FullBody"));
			SetCollection->AddSet(TEXT("Neck"), TEXT("UpperBody"));

			bSuccess = Binding->SetSetCollection(SetCollection);
			UE_RETURN_ON_ERROR(bSuccess, "Failed to set set collection on our binding");
		
			UAbstractSkeletonSetBinding* SetBinding2 = CastChecked<UAbstractSkeletonSetBinding>(BindingFactory->FactoryCreateNew(UAbstractSkeletonSetBinding::StaticClass(), GetTransientPackage(), TEXT("TestSetBindingAsset"), RF_Transient, nullptr, nullptr, NAME_None));
			UE_RETURN_ON_ERROR(SetBinding2 != nullptr, "Failed to create set collection asset");

			bSuccess = SetBinding2->SetSkeleton(Skeleton);
			UE_RETURN_ON_ERROR(bSuccess, "Failed to assign skeleton to binding");

			bSuccess = SetBinding2->SetSetCollection(SetCollection);
			UE_RETURN_ON_ERROR(bSuccess, "Failed to assign set collection to binding");

			bSuccess = SetBinding2->AddBoneToSet(TEXT("Spine_01"), TEXT("UpperBody"));
			UE_RETURN_ON_ERROR(bSuccess, "Failed to bind Spine_01 to UpperBody");

			FAttributeBindingDataPtr BindingData = MakeAttributeBindingData(SetBinding2, nullptr);
			UE_RETURN_ON_ERROR(BindingData.IsValid(), TEXT("Attribute binding data should be valid"));

			AddErrorIfFalse(!BindingData->IsEmpty(), TEXT("Attribute binding data should not be empty"));
			AddErrorIfFalse(BindingData->NumNamedSets() == 4, TEXT("Unexpected number of named sets"));
			AddErrorIfFalse(BindingData->NumLODs() == 1, TEXT("Unexpected number of LODs"));
			AddErrorIfFalse(BindingData->NumAttributes(FBoneTransformAnimationAttribute::StaticStruct()) == 8, TEXT("Unexpected number of bone attributes"));

			FAttributeNamedSetPtr EverythingSet = BindingData->FindNamedSet(NAME_None);
			UE_RETURN_ON_ERROR(EverythingSet.IsValid(), TEXT("Everything named set should be valid"));
			AddErrorIfFalse(!EverythingSet->IsEmpty(), TEXT("Everything named set should not be empty"));
			AddErrorIfFalse(EverythingSet->GetName().IsNone(), TEXT("Unexpected named set name"));
			AddErrorIfFalse(EverythingSet->GetLOD() == 0, TEXT("Unexpected named set LOD"));
			AddErrorIfFalse(EverythingSet->NumTypedSets() == 1, TEXT("Unexpected named set number of typed set"));
			AddErrorIfFalse(EverythingSet->NumLODs() == 1, TEXT("Unexpected named set number of LODs"));
			AddErrorIfFalse(!EverythingSet->AtLOD(INDEX_NONE).IsValid(), TEXT("Unexpected named set ptr at LOD"));
			AddErrorIfFalse(EverythingSet->AtLOD(0) == EverythingSet, TEXT("Unexpected named set ptr at LOD"));
			AddErrorIfFalse(!EverythingSet->AtLOD(1).IsValid(), TEXT("Unexpected named set ptr at LOD"));
			{
				// Unlike other sets, the 'Everything' set will also contain all bones of the source skeleton

				FAttributeTypedSetPtr BoneSet = EverythingSet->FindTypedSet(FBoneTransformAnimationAttribute::StaticStruct());
				UE_RETURN_ON_ERROR(BoneSet.IsValid(), TEXT("Typed set should be valid"));
				AddErrorIfFalse(!BoneSet->IsEmpty(), TEXT("Typed set should not be empty"));
				AddErrorIfFalse(BoneSet->Num() == 8, TEXT("Unexpected typed set number of attributes"));
			}

			FAttributeNamedSetPtr FullBodySet = BindingData->FindNamedSet(TEXT("FullBody"));
			UE_RETURN_ON_ERROR(FullBodySet.IsValid(), TEXT("FullBody named set should be valid"));
			AddErrorIfFalse(!FullBodySet->IsEmpty(), TEXT("FullBody named set should not be empty"));
			AddErrorIfFalse(FullBodySet->GetName() == TEXT("FullBody"), TEXT("Unexpected named set name"));
			AddErrorIfFalse(FullBodySet->GetLOD() == 0, TEXT("Unexpected named set LOD"));
			AddErrorIfFalse(FullBodySet->NumTypedSets() == 1, TEXT("Unexpected named set number of typed set"));
			AddErrorIfFalse(FullBodySet->NumLODs() == 1, TEXT("Unexpected named set number of LODs"));
			AddErrorIfFalse(!FullBodySet->AtLOD(INDEX_NONE).IsValid(), TEXT("Unexpected named set ptr at LOD"));
			AddErrorIfFalse(FullBodySet->AtLOD(0) == FullBodySet, TEXT("Unexpected named set ptr at LOD"));
			AddErrorIfFalse(!FullBodySet->AtLOD(1).IsValid(), TEXT("Unexpected named set ptr at LOD"));
			{
				// The 'FullBody' set should include the 'Spine_01' attribute from its child 'UpperBody' set

				FAttributeTypedSetPtr BoneSet = FullBodySet->FindTypedSet(FBoneTransformAnimationAttribute::StaticStruct());
				UE_RETURN_ON_ERROR(BoneSet.IsValid(), TEXT("Typed set should be valid"));
				AddErrorIfFalse(!BoneSet->IsEmpty(), TEXT("Typed set should not be empty"));
				AddErrorIfFalse(BoneSet->Num() == 1, TEXT("Unexpected typed set number of attributes"));
			}

			FAttributeNamedSetPtr UpperBodySet = BindingData->FindNamedSet(TEXT("UpperBody"));
			UE_RETURN_ON_ERROR(UpperBodySet.IsValid(), TEXT("UpperBody named set should be valid"));
			AddErrorIfFalse(!UpperBodySet->IsEmpty(), TEXT("UpperBody named set should not be empty"));
			AddErrorIfFalse(UpperBodySet->GetName() == TEXT("UpperBody"), TEXT("Unexpected named set name"));
			AddErrorIfFalse(UpperBodySet->GetLOD() == 0, TEXT("Unexpected named set LOD"));
			AddErrorIfFalse(UpperBodySet->NumTypedSets() == 1, TEXT("Unexpected named set number of typed set"));
			AddErrorIfFalse(UpperBodySet->NumLODs() == 1, TEXT("Unexpected named set number of LODs"));
			AddErrorIfFalse(!UpperBodySet->AtLOD(INDEX_NONE).IsValid(), TEXT("Unexpected named set ptr at LOD"));
			AddErrorIfFalse(UpperBodySet->AtLOD(0) == UpperBodySet, TEXT("Unexpected named set ptr at LOD"));
			AddErrorIfFalse(!UpperBodySet->AtLOD(1).IsValid(), TEXT("Unexpected named set ptr at LOD"));
			{
				// The 'UpperBody' set has an explicit binding to the 'Spine_01' attribute

				FAttributeTypedSetPtr BoneSet = UpperBodySet->FindTypedSet(FBoneTransformAnimationAttribute::StaticStruct());
				UE_RETURN_ON_ERROR(BoneSet.IsValid(), TEXT("Typed set should be valid"));
				AddErrorIfFalse(!BoneSet->IsEmpty(), TEXT("Typed set should not be empty"));
				AddErrorIfFalse(BoneSet->Num() == 1, TEXT("Unexpected typed set number of attributes"));
			}

			FAttributeNamedSetPtr NeckSet = BindingData->FindNamedSet(TEXT("Neck"));
			UE_RETURN_ON_ERROR(NeckSet.IsValid(), TEXT("Neck named set should be valid"));
			AddErrorIfFalse(!NeckSet->IsEmpty(), TEXT("Neck named set should not be empty"));
			AddErrorIfFalse(NeckSet->GetName() == TEXT("Neck"), TEXT("Unexpected named set name"));
			AddErrorIfFalse(NeckSet->GetLOD() == 0, TEXT("Unexpected named set LOD"));
			AddErrorIfFalse(NeckSet->NumTypedSets() == 1, TEXT("Unexpected named set number of typed set"));
			AddErrorIfFalse(NeckSet->NumLODs() == 1, TEXT("Unexpected named set number of LODs"));
			AddErrorIfFalse(!NeckSet->AtLOD(INDEX_NONE).IsValid(), TEXT("Unexpected named set ptr at LOD"));
			AddErrorIfFalse(NeckSet->AtLOD(0) == NeckSet, TEXT("Unexpected named set ptr at LOD"));
			AddErrorIfFalse(!NeckSet->AtLOD(1).IsValid(), TEXT("Unexpected named set ptr at LOD"));
			{
				// Nothing is bound to the 'Neck' set and it has no children sets with any bindings either

				FAttributeTypedSetPtr BoneSet = NeckSet->FindTypedSet(FBoneTransformAnimationAttribute::StaticStruct());
				UE_RETURN_ON_ERROR(BoneSet.IsValid(), TEXT("Typed set should be valid"));
				AddErrorIfFalse(BoneSet->IsEmpty(), TEXT("Typed set should be empty"));
			}
		}

		return true;
	}
}

#endif	// WITH_DEV_AUTOMATION_TESTS
