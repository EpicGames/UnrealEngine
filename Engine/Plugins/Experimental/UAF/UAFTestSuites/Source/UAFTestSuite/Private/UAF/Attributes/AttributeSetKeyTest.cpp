// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "UAF/Attributes/AttributeSetKey.h"

namespace UE::UAF::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAttributeSetKeyTest, "Animation.UAF.Attributes.AttributeSetKey", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool FAttributeSetKeyTest::RunTest(const FString& InParameters)
	{
		FAttributeSetKey InvalidKey(FAttributeBindingIndex(), FAttributeBindingIndex(2), 0);
		FAttributeSetKey Key0(FAttributeBindingIndex(0), FAttributeBindingIndex(), 0);
		FAttributeSetKey Key1(FAttributeBindingIndex(10), FAttributeBindingIndex(0), 0);
		FAttributeSetKey Key2(FAttributeBindingIndex(2), FAttributeBindingIndex(1), 1);

		AddErrorIfFalse(!InvalidKey.IsValid(), TEXT("Unexpected attribute set key generated"));
		AddErrorIfFalse(Key0.IsValid() && Key0 != Key1, TEXT("Unexpected attribute set key generated"));
		AddErrorIfFalse(Key1.IsValid() && Key1 != Key2, TEXT("Unexpected attribute set key generated"));
		AddErrorIfFalse(Key2.IsValid() && Key2 != Key0, TEXT("Unexpected attribute set key generated"));

		AddErrorIfFalse(!InvalidKey.GetBindingIndex().IsValid(), TEXT("Unexpected attribute set key generated"));
		AddErrorIfFalse(Key0.GetBindingIndex().GetInt() == 0, TEXT("Unexpected attribute set key generated"));
		AddErrorIfFalse(Key1.GetBindingIndex().GetInt() == 10, TEXT("Unexpected attribute set key generated"));
		AddErrorIfFalse(Key2.GetBindingIndex().GetInt() == 2, TEXT("Unexpected attribute set key generated"));

		AddErrorIfFalse(!InvalidKey.GetParentBindingIndex().IsValid(), TEXT("Unexpected attribute set key generated"));
		AddErrorIfFalse(!Key0.GetParentBindingIndex().IsValid(), TEXT("Unexpected attribute set key generated"));
		AddErrorIfFalse(Key1.GetParentBindingIndex().GetInt() == 0, TEXT("Unexpected attribute set key generated"));
		AddErrorIfFalse(Key2.GetParentBindingIndex().GetInt() == 1, TEXT("Unexpected attribute set key generated"));

		AddErrorIfFalse(InvalidKey.GetLOD() == 0, TEXT("Unexpected attribute set key generated"));
		AddErrorIfFalse(Key0.GetLOD() == 0, TEXT("Unexpected attribute set key generated"));
		AddErrorIfFalse(Key1.GetLOD() == 0, TEXT("Unexpected attribute set key generated"));
		AddErrorIfFalse(Key2.GetLOD() == 1, TEXT("Unexpected attribute set key generated"));

		// Invalid keys are always smaller
		AddErrorIfFalse(InvalidKey < Key0, TEXT("Unexpected attribute set key generated"));
		AddErrorIfFalse(InvalidKey < Key1, TEXT("Unexpected attribute set key generated"));
		AddErrorIfFalse(InvalidKey < Key2, TEXT("Unexpected attribute set key generated"));

		// LOD 0 comes first
		AddErrorIfFalse(Key0 < Key1, TEXT("Unexpected attribute set key generated"));
		AddErrorIfFalse(Key0 < Key2, TEXT("Unexpected attribute set key generated"));
		AddErrorIfFalse(Key1 < Key2, TEXT("Unexpected attribute set key generated"));
		AddErrorIfFalse(FAttributeSetKey(FAttributeBindingIndex(2), FAttributeBindingIndex(1), 1) < FAttributeSetKey(FAttributeBindingIndex(2), FAttributeBindingIndex(1), 5), TEXT("Unexpected attribute set key generated"));

		// Parents come in increasing order and invalid parents come first
		AddErrorIfFalse(Key0 < Key1, TEXT("Unexpected attribute set key generated"));
		AddErrorIfFalse(Key0 < Key2, TEXT("Unexpected attribute set key generated"));
		AddErrorIfFalse(Key1 < Key2, TEXT("Unexpected attribute set key generated"));
		AddErrorIfFalse(FAttributeSetKey(FAttributeBindingIndex(2), FAttributeBindingIndex(1), 0) < FAttributeSetKey(FAttributeBindingIndex(2), FAttributeBindingIndex(2), 0), TEXT("Unexpected attribute set key generated"));
		AddErrorIfFalse(FAttributeSetKey(FAttributeBindingIndex(2), FAttributeBindingIndex(), 0) < FAttributeSetKey(FAttributeBindingIndex(2), FAttributeBindingIndex(0), 0), TEXT("Unexpected attribute set key generated"));

		// When the LOD and parents are equal, we come in natural order
		AddErrorIfFalse(FAttributeSetKey(FAttributeBindingIndex(1), FAttributeBindingIndex(), 0) < FAttributeSetKey(FAttributeBindingIndex(2), FAttributeBindingIndex(), 0), TEXT("Unexpected attribute set key generated"));

		return true;
	}
}

#endif	// WITH_DEV_AUTOMATION_TESTS
