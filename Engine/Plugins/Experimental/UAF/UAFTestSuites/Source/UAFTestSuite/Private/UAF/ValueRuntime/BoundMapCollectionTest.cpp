// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "AnimNextTest.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "UAF/AbstractSkeleton/SetBindingFactory.h"
#include "UAF/AbstractSkeleton/SetCollectionFactory.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetCollection.h"
#include "UAF/Attributes/AttributeBindingData.h"
#include "UAF/Attributes/EngineAttributes.h"
#include "UAF/ValueRuntime/BoundMapCollection.h"
#include "UAF/ValueRuntime/IteratorTestUtils.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

namespace UE::UAF::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBoundMapCollectionTest, "Animation.UAF.ValueRuntime.BoundMapCollection", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool FBoundMapCollectionTest::RunTest(const FString& InParameters)
	{
		ON_SCOPE_EXIT{ FUtils::CleanupAfterTests(); };

		FAttributeBindingDataPtr BindingData;
		{
			UFactory* BindingFactory = NewObject<UFactory>(GetTransientPackage(), UAbstractSkeletonSetBindingFactory::StaticClass());
			UE_RETURN_ON_ERROR(BindingFactory != nullptr, "Failed to create factory");

			UAbstractSkeletonSetBinding* Binding = CastChecked<UAbstractSkeletonSetBinding>(BindingFactory->FactoryCreateNew(UAbstractSkeletonSetBinding::StaticClass(), GetTransientPackage(), TEXT("TestBindingAsset"), RF_Transient, nullptr, nullptr, NAME_None));
			UE_RETURN_ON_ERROR(Binding != nullptr, "Failed to create binding asset");

			USkeleton* Skeleton = NewObject<USkeleton>(GetTransientPackage(), USkeleton::StaticClass(), TEXT("TestSkeletonAsset"), RF_Transient);
			UE_RETURN_ON_ERROR(Skeleton != nullptr, "Failed to create skeleton asset");

			{
				// Not terribly clean, we cast away the 'const' to modify the skeleton
				FReferenceSkeleton& RefSkeleton = const_cast<FReferenceSkeleton&>(Skeleton->GetReferenceSkeleton());
				FReferenceSkeletonModifier SkeletonModifier(RefSkeleton, Skeleton);

				SkeletonModifier.Add(FMeshBoneInfo(TEXT("Root"), TEXT("Root"), INDEX_NONE), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("Pelvis"), TEXT("Root"), 0), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("Spine_01"), TEXT("Spine_01"), 1), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("Spine_02"), TEXT("Spine_02"), 2), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("Head"), TEXT("Head"), 3), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("LeftArm"), TEXT("LeftArm"), 3), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("RightArm"), TEXT("RightArm"), 3), FTransform::Identity);

				// When our modifier is destroyed here, it will rebuild the skeleton
			}

			UE_RETURN_ON_ERROR(Skeleton->GetReferenceSkeleton().GetNum() == 7, "Failed to build skeleton asset");

			bool bSuccess = Binding->SetSkeleton(Skeleton);
			UE_RETURN_ON_ERROR(bSuccess, "Failed to set skeleton on our binding");

			UFactory* SecCollectionFactory = NewObject<UFactory>(GetTransientPackage(), UAbstractSkeletonSetCollectionFactory::StaticClass());
			UE_RETURN_ON_ERROR(SecCollectionFactory != nullptr, "Failed to create factory");

			UAbstractSkeletonSetCollection* SetCollection = CastChecked<UAbstractSkeletonSetCollection>(SecCollectionFactory->FactoryCreateNew(UAbstractSkeletonSetCollection::StaticClass(), GetTransientPackage(), TEXT("TestSetCollectionAsset"), RF_Transient, nullptr, nullptr, NAME_None));
			UE_RETURN_ON_ERROR(SetCollection != nullptr, "Failed to create set collection asset");

			SetCollection->AddSet(TEXT("FullBody"), NAME_None);
			SetCollection->AddSet(TEXT("UpperBody"), TEXT("FullBody"));

			bSuccess = Binding->SetSetCollection(SetCollection);
			UE_RETURN_ON_ERROR(bSuccess, "Failed to set set collection on our binding");

			Binding->AddBoneToSet(TEXT("Root"), TEXT("FullBody"));
			Binding->AddBoneToSet(TEXT("Pelvis"), TEXT("FullBody"));
			Binding->AddBoneToSet(TEXT("Spine_01"), TEXT("FullBody"));
			Binding->AddBoneToSet(TEXT("Spine_02"), TEXT("FullBody"));
			Binding->AddAttributeToSet(FAnimationAttributeIdentifier(TEXT("AttrFloat0"), INDEX_NONE, NAME_None, FFloatAnimationAttribute::StaticStruct()), TEXT("FullBody"));

			Binding->AddBoneToSet(TEXT("Head"), TEXT("UpperBody"));
			Binding->AddBoneToSet(TEXT("LeftArm"), TEXT("UpperBody"));
			Binding->AddBoneToSet(TEXT("RightArm"), TEXT("UpperBody"));
			Binding->AddAttributeToSet(FAnimationAttributeIdentifier(TEXT("AttrFloat1"), INDEX_NONE, NAME_None, FFloatAnimationAttribute::StaticStruct()), TEXT("UpperBody"));

			BindingData = MakeAttributeBindingData(Binding, nullptr);
			UE_RETURN_ON_ERROR(BindingData.IsValid(), TEXT("Attribute binding data should be valid"));
		}

		{
			FAttributeNamedSetPtr FullBodySet = BindingData->FindNamedSet(TEXT("FullBody"));
			UE_RETURN_ON_ERROR(FullBodySet.IsValid(), TEXT("FullBody named set should be valid"));

			FBoundMapCollectionHeap Collection(FullBodySet);

			AddErrorIfFalse(Collection.IsEmpty(), TEXT("Attribute collection should be empty"));
			AddErrorIfFalse(Collection.GetNamedSet() == FullBodySet, TEXT("Unexpected named set"));

			FAttributeMappingKey FloatToFloatKey = FAttributeMappingKey::MakeFromTo<FFloatAnimationAttribute, FFloatAnimationAttribute>();
			TBoundValueMap<FFloatAnimationAttribute>* FloatToFloatMap = Collection.Add<FFloatAnimationAttribute>(FloatToFloatKey);
			UE_RETURN_ON_ERROR(FloatToFloatMap != nullptr, TEXT("Float->Float map should be valid"));

			FAttributeMappingKey FloatToIntegerKey = FAttributeMappingKey::MakeFromTo<FFloatAnimationAttribute, FIntegerAnimationAttribute>();
			TBoundValueMap<FIntegerAnimationAttribute>* FloatToIntegerMap = MakeBoundValueMap<FIntegerAnimationAttribute>(FBoundValueMap::FConstructArgs(FullBodySet->FindTypedSet<FFloatAnimationAttribute>(), FIntegerAnimationAttribute::StaticStruct(), Collection.GetAllocator()));
			UE_RETURN_ON_ERROR(FloatToIntegerMap != nullptr, TEXT("Float->Integer map should be valid"));
			bool bAdded = Collection.Add(FloatToIntegerMap);
			UE_RETURN_ON_ERROR(bAdded, TEXT("Float->Integer map should be added"));

			// Can't add twice
			bAdded = Collection.Add(FloatToIntegerMap);
			AddErrorIfFalse(!bAdded, TEXT("Float->Integer map should not be added"));

			FAttributeMappingKey FloatToStringKey = FAttributeMappingKey::MakeFromTo<FFloatAnimationAttribute, FStringAnimationAttribute>();
			TBoundValueMap<FStringAnimationAttribute>* FloatToStringMap = Collection.Add<FStringAnimationAttribute>(FloatToStringKey);
			UE_RETURN_ON_ERROR(FloatToStringMap != nullptr, TEXT("Float->String map should be valid"));

			FAttributeMappingKey TransformToTransformKey = FAttributeMappingKey::MakeFromTo<FBoneTransformAnimationAttribute, FBoneTransformAnimationAttribute>();
			TBoundValueMap<FBoneTransformAnimationAttribute>* TransformToTransformMap = Collection.Add<FBoneTransformAnimationAttribute>(TransformToTransformKey);
			UE_RETURN_ON_ERROR(TransformToTransformMap != nullptr, TEXT("Transform->Transform map should be valid"));

			FAttributeMappingKey TransformToFloatKey = FAttributeMappingKey::MakeFromTo<FBoneTransformAnimationAttribute, FFloatAnimationAttribute>();
			TBoundValueMap<FFloatAnimationAttribute>* TransformToFloatMap = Cast<FFloatAnimationAttribute>(Collection.Add(TransformToFloatKey));
			UE_RETURN_ON_ERROR(TransformToFloatMap != nullptr, TEXT("Transform->Bone map should be valid"));

			AddErrorIfFalse(!Collection.IsEmpty(), TEXT("Collection should not be empty"));
			AddErrorIfFalse(Collection.Num() == 5, TEXT("Unexpected map count"));

			AddErrorIfFalse(FloatToFloatMap == Collection.Find<FFloatAnimationAttribute>(FloatToFloatKey), TEXT("Unexpected result from Find"));
			AddErrorIfFalse(FloatToIntegerMap == Collection.Find<FIntegerAnimationAttribute>(FloatToIntegerKey), TEXT("Unexpected result from Find"));
			AddErrorIfFalse(FloatToStringMap == Collection.Find<FStringAnimationAttribute>(FloatToStringKey), TEXT("Unexpected result from Find"));
			AddErrorIfFalse(TransformToTransformMap == Collection.Find<FBoneTransformAnimationAttribute>(TransformToTransformKey), TEXT("Unexpected result from Find"));
			AddErrorIfFalse(TransformToFloatMap == Collection.Find(TransformToFloatKey), TEXT("Unexpected result from Find"));

			AddErrorIfFalse(Collection.Remove(FloatToFloatMap), TEXT("Failed to remove map"));
			AddErrorIfFalse(Collection.Remove(FloatToIntegerKey), TEXT("Failed to remove map"));
			AddErrorIfFalse(Collection.Remove(FloatToStringMap), TEXT("Failed to remove map"));
			AddErrorIfFalse(Collection.Remove(TransformToTransformKey), TEXT("Failed to remove map"));
			AddErrorIfFalse(Collection.Remove(TransformToFloatKey), TEXT("Failed to remove map"));

			AddErrorIfFalse(nullptr == Collection.Find<FFloatAnimationAttribute>(FloatToFloatKey), TEXT("Unexpected result from Find"));
			AddErrorIfFalse(nullptr == Collection.Find<FIntegerAnimationAttribute>(FloatToIntegerKey), TEXT("Unexpected result from Find"));
			AddErrorIfFalse(nullptr == Collection.Find<FStringAnimationAttribute>(FloatToStringKey), TEXT("Unexpected result from Find"));
			AddErrorIfFalse(nullptr == Collection.Find<FBoneTransformAnimationAttribute>(TransformToTransformKey), TEXT("Unexpected result from Find"));
			AddErrorIfFalse(nullptr == Collection.Find(TransformToFloatKey), TEXT("Unexpected result from Find"));

			AddErrorIfFalse(Collection.IsEmpty(), TEXT("Collection should be empty"));
			AddErrorIfFalse(Collection.Num() == 0, TEXT("Unexpected map count"));

			FloatToFloatMap = Collection.FindOrAdd<FFloatAnimationAttribute>(FloatToFloatKey);
			UE_RETURN_ON_ERROR(FloatToFloatMap != nullptr, TEXT("Float->Float map should be valid"));

			FloatToIntegerMap = Collection.FindOrAdd<FIntegerAnimationAttribute>(FloatToIntegerKey);
			UE_RETURN_ON_ERROR(FloatToIntegerMap != nullptr, TEXT("Float->Integer map should be valid"));

			FloatToStringMap = Collection.FindOrAdd<FStringAnimationAttribute>(FloatToStringKey);
			UE_RETURN_ON_ERROR(FloatToStringMap != nullptr, TEXT("Float->String map should be valid"));

			TransformToTransformMap = Collection.FindOrAdd<FBoneTransformAnimationAttribute>(TransformToTransformKey);
			UE_RETURN_ON_ERROR(TransformToTransformMap != nullptr, TEXT("Transform->Transform map should be valid"));

			TransformToFloatMap = Cast<FFloatAnimationAttribute>(Collection.FindOrAdd(TransformToFloatKey));
			UE_RETURN_ON_ERROR(TransformToFloatMap != nullptr, TEXT("Transform->Bone map should be valid"));

			AddErrorIfFalse(!Collection.IsEmpty(), TEXT("Collection should not be empty"));
			AddErrorIfFalse(Collection.Num() == 5, TEXT("Unexpected map count"));

			AddErrorIfFalse(FloatToFloatMap == Collection.FindOrAdd<FFloatAnimationAttribute>(FloatToFloatKey), TEXT("Unexpected result from FindOrAdd"));
			AddErrorIfFalse(FloatToIntegerMap == Collection.FindOrAdd<FIntegerAnimationAttribute>(FloatToIntegerKey), TEXT("Unexpected result from FindOrAdd"));
			AddErrorIfFalse(FloatToStringMap == Collection.FindOrAdd<FStringAnimationAttribute>(FloatToStringKey), TEXT("Unexpected result from FindOrAdd"));
			AddErrorIfFalse(TransformToTransformMap == Collection.FindOrAdd<FBoneTransformAnimationAttribute>(TransformToTransformKey), TEXT("Unexpected result from FindOrAdd"));
			AddErrorIfFalse(TransformToFloatMap == Collection.FindOrAdd(TransformToFloatKey), TEXT("Unexpected result from FindOrAdd"));

			AddErrorIfFalse(!Collection.IsEmpty(), TEXT("Collection should not be empty"));
			AddErrorIfFalse(Collection.Num() == 5, TEXT("Unexpected map count"));

			// Append
			{
				FBoundMapCollectionHeap CollectionCopy(FullBodySet);

				for (auto It = Collection.CreateConstIterator(); It; ++It)
				{
					FBoundValueMap* SetMapCopy = It.GetMap()->Duplicate(CollectionCopy.GetAllocator());
					const bool bAppended = CollectionCopy.Append(SetMapCopy);
					AddErrorIfFalse(bAppended, TEXT("Map should be appended"));
				}

				AddErrorIfFalse(CollectionCopy.IsEmpty() == Collection.IsEmpty(), TEXT("Collection should not be empty"));
				AddErrorIfFalse(CollectionCopy.Num() == Collection.Num(), TEXT("Unexpected map count"));

				TBoundValueMap<FFloatAnimationAttribute>* FloatToFloatMapCopy = CollectionCopy.Find<FFloatAnimationAttribute>(FloatToFloatKey);
				UE_RETURN_ON_ERROR(FloatToFloatMapCopy != nullptr, TEXT("Float->Float map should be valid"));

				TBoundValueMap<FIntegerAnimationAttribute>* FloatToIntegerMapCopy = CollectionCopy.Find<FIntegerAnimationAttribute>(FloatToIntegerKey);
				UE_RETURN_ON_ERROR(FloatToIntegerMapCopy != nullptr, TEXT("Float->Integer map should be valid"));

				TBoundValueMap<FStringAnimationAttribute>* FloatToStringMapCopy = CollectionCopy.Find<FStringAnimationAttribute>(FloatToStringKey);
				UE_RETURN_ON_ERROR(FloatToStringMapCopy != nullptr, TEXT("Float->String map should be valid"));

				TBoundValueMap<FBoneTransformAnimationAttribute>* TransformToTransformMapCopy = CollectionCopy.Find<FBoneTransformAnimationAttribute>(TransformToTransformKey);
				UE_RETURN_ON_ERROR(TransformToTransformMapCopy != nullptr, TEXT("Transform->Transform map should be valid"));

				TBoundValueMap<FFloatAnimationAttribute>* TransformToFloatMapCopy = Cast<FFloatAnimationAttribute>(CollectionCopy.Find(TransformToFloatKey));
				UE_RETURN_ON_ERROR(TransformToFloatMapCopy != nullptr, TEXT("Transform->Bone map should be valid"));

				AddErrorIfFalse(FloatToFloatMap != FloatToFloatMapCopy, TEXT("Map instances should be duplicated"));
				AddErrorIfFalse(FloatToIntegerMap != FloatToIntegerMapCopy, TEXT("Map instances should be duplicated"));
				AddErrorIfFalse(FloatToStringMap != FloatToStringMapCopy, TEXT("Map instances should be duplicated"));
				AddErrorIfFalse(TransformToTransformMap != TransformToTransformMapCopy, TEXT("Map instances should be duplicated"));
				AddErrorIfFalse(TransformToFloatMap != TransformToFloatMapCopy, TEXT("Map instances should be duplicated"));

				AddErrorIfFalse(FloatToFloatMap->GetTypedSet() == FloatToFloatMapCopy->GetTypedSet(), TEXT("Typed set mismatched"));
				AddErrorIfFalse(FloatToIntegerMap->GetTypedSet() == FloatToIntegerMapCopy->GetTypedSet(), TEXT("Typed set mismatched"));
				AddErrorIfFalse(FloatToStringMap->GetTypedSet() == FloatToStringMapCopy->GetTypedSet(), TEXT("Typed set mismatched"));
				AddErrorIfFalse(TransformToTransformMap->GetTypedSet() == TransformToTransformMapCopy->GetTypedSet(), TEXT("Typed set mismatched"));
				AddErrorIfFalse(TransformToFloatMap->GetTypedSet() == TransformToFloatMapCopy->GetTypedSet(), TEXT("Typed set mismatched"));
			}

			// Copy constructor
			{
				FBoundMapCollectionHeap CollectionCopy(Collection);

				AddErrorIfFalse(CollectionCopy.IsEmpty() == Collection.IsEmpty(), TEXT("Collection should not be empty"));
				AddErrorIfFalse(CollectionCopy.Num() == Collection.Num(), TEXT("Unexpected map count"));

				TBoundValueMap<FFloatAnimationAttribute>* FloatToFloatMapCopy = CollectionCopy.Find<FFloatAnimationAttribute>(FloatToFloatKey);
				UE_RETURN_ON_ERROR(FloatToFloatMapCopy != nullptr, TEXT("Float->Float map should be valid"));

				TBoundValueMap<FIntegerAnimationAttribute>* FloatToIntegerMapCopy = CollectionCopy.Find<FIntegerAnimationAttribute>(FloatToIntegerKey);
				UE_RETURN_ON_ERROR(FloatToIntegerMapCopy != nullptr, TEXT("Float->Integer map should be valid"));

				TBoundValueMap<FStringAnimationAttribute>* FloatToStringMapCopy = CollectionCopy.Find<FStringAnimationAttribute>(FloatToStringKey);
				UE_RETURN_ON_ERROR(FloatToStringMapCopy != nullptr, TEXT("Float->String map should be valid"));

				TBoundValueMap<FBoneTransformAnimationAttribute>* TransformToTransformMapCopy = CollectionCopy.Find<FBoneTransformAnimationAttribute>(TransformToTransformKey);
				UE_RETURN_ON_ERROR(TransformToTransformMapCopy != nullptr, TEXT("Transform->Transform map should be valid"));

				TBoundValueMap<FFloatAnimationAttribute>* TransformToFloatMapCopy = Cast<FFloatAnimationAttribute>(CollectionCopy.Find(TransformToFloatKey));
				UE_RETURN_ON_ERROR(TransformToFloatMapCopy != nullptr, TEXT("Transform->Bone map should be valid"));

				AddErrorIfFalse(FloatToFloatMap != FloatToFloatMapCopy, TEXT("Map instances should be duplicated"));
				AddErrorIfFalse(FloatToIntegerMap != FloatToIntegerMapCopy, TEXT("Map instances should be duplicated"));
				AddErrorIfFalse(FloatToStringMap != FloatToStringMapCopy, TEXT("Map instances should be duplicated"));
				AddErrorIfFalse(TransformToTransformMap != TransformToTransformMapCopy, TEXT("Map instances should be duplicated"));
				AddErrorIfFalse(TransformToFloatMap != TransformToFloatMapCopy, TEXT("Map instances should be duplicated"));

				AddErrorIfFalse(FloatToFloatMap->GetTypedSet() == FloatToFloatMapCopy->GetTypedSet(), TEXT("Typed set mismatched"));
				AddErrorIfFalse(FloatToIntegerMap->GetTypedSet() == FloatToIntegerMapCopy->GetTypedSet(), TEXT("Typed set mismatched"));
				AddErrorIfFalse(FloatToStringMap->GetTypedSet() == FloatToStringMapCopy->GetTypedSet(), TEXT("Typed set mismatched"));
				AddErrorIfFalse(TransformToTransformMap->GetTypedSet() == TransformToTransformMapCopy->GetTypedSet(), TEXT("Typed set mismatched"));
				AddErrorIfFalse(TransformToFloatMap->GetTypedSet() == TransformToFloatMapCopy->GetTypedSet(), TEXT("Typed set mismatched"));

				// Move constructor
				{
					FBoundMapCollectionHeap CollectionMoved(MoveTemp(CollectionCopy));

					AddErrorIfFalse(CollectionCopy.IsEmpty(), TEXT("Collection should be empty"));
					AddErrorIfFalse(!CollectionMoved.IsEmpty(), TEXT("Collection should not be empty"));
					AddErrorIfFalse(CollectionMoved.Num() == Collection.Num(), TEXT("Unexpected map count"));

					TBoundValueMap<FFloatAnimationAttribute>* FloatToFloatMapMoved = CollectionMoved.Find<FFloatAnimationAttribute>(FloatToFloatKey);
					UE_RETURN_ON_ERROR(FloatToFloatMapMoved != nullptr, TEXT("Float->Float map should be valid"));

					TBoundValueMap<FIntegerAnimationAttribute>* FloatToIntegerMapMoved = CollectionMoved.Find<FIntegerAnimationAttribute>(FloatToIntegerKey);
					UE_RETURN_ON_ERROR(FloatToIntegerMapMoved != nullptr, TEXT("Float->Integer map should be valid"));

					TBoundValueMap<FStringAnimationAttribute>* FloatToStringMapMoved = CollectionMoved.Find<FStringAnimationAttribute>(FloatToStringKey);
					UE_RETURN_ON_ERROR(FloatToStringMapMoved != nullptr, TEXT("Float->String map should be valid"));

					TBoundValueMap<FBoneTransformAnimationAttribute>* TransformToTransformMapMoved = CollectionMoved.Find<FBoneTransformAnimationAttribute>(TransformToTransformKey);
					UE_RETURN_ON_ERROR(TransformToTransformMapMoved != nullptr, TEXT("Transform->Transform map should be valid"));

					TBoundValueMap<FFloatAnimationAttribute>* TransformToFloatMapMoved = Cast<FFloatAnimationAttribute>(CollectionMoved.Find(TransformToFloatKey));
					UE_RETURN_ON_ERROR(TransformToFloatMapMoved != nullptr, TEXT("Transform->Bone map should be valid"));

					AddErrorIfFalse(FloatToFloatMapMoved == FloatToFloatMapCopy, TEXT("Map instances should be moved"));
					AddErrorIfFalse(FloatToIntegerMapMoved == FloatToIntegerMapCopy, TEXT("Map instances should be moved"));
					AddErrorIfFalse(FloatToStringMapMoved == FloatToStringMapCopy, TEXT("Map instances should be moved"));
					AddErrorIfFalse(TransformToTransformMapMoved == TransformToTransformMapCopy, TEXT("Map instances should be moved"));
					AddErrorIfFalse(TransformToFloatMapMoved == TransformToFloatMapCopy, TEXT("Map instances should be moved"));
				}
			}

			// Copy assignment
			{
				FBoundMapCollectionHeap CollectionCopy;
				CollectionCopy = Collection;

				AddErrorIfFalse(CollectionCopy.IsEmpty() == Collection.IsEmpty(), TEXT("Collection should not be empty"));
				AddErrorIfFalse(CollectionCopy.Num() == Collection.Num(), TEXT("Unexpected map count"));

				TBoundValueMap<FFloatAnimationAttribute>* FloatToFloatMapCopy = CollectionCopy.Find<FFloatAnimationAttribute>(FloatToFloatKey);
				UE_RETURN_ON_ERROR(FloatToFloatMapCopy != nullptr, TEXT("Float->Float map should be valid"));

				TBoundValueMap<FIntegerAnimationAttribute>* FloatToIntegerMapCopy = CollectionCopy.Find<FIntegerAnimationAttribute>(FloatToIntegerKey);
				UE_RETURN_ON_ERROR(FloatToIntegerMapCopy != nullptr, TEXT("Float->Integer map should be valid"));

				TBoundValueMap<FStringAnimationAttribute>* FloatToStringMapCopy = CollectionCopy.Find<FStringAnimationAttribute>(FloatToStringKey);
				UE_RETURN_ON_ERROR(FloatToStringMapCopy != nullptr, TEXT("Float->String map should be valid"));

				TBoundValueMap<FBoneTransformAnimationAttribute>* TransformToTransformMapCopy = CollectionCopy.Find<FBoneTransformAnimationAttribute>(TransformToTransformKey);
				UE_RETURN_ON_ERROR(TransformToTransformMapCopy != nullptr, TEXT("Transform->Transform map should be valid"));

				TBoundValueMap<FFloatAnimationAttribute>* TransformToFloatMapCopy = Cast<FFloatAnimationAttribute>(CollectionCopy.Find(TransformToFloatKey));
				UE_RETURN_ON_ERROR(TransformToFloatMapCopy != nullptr, TEXT("Transform->Bone map should be valid"));

				AddErrorIfFalse(FloatToFloatMap != FloatToFloatMapCopy, TEXT("Map instances should be duplicated"));
				AddErrorIfFalse(FloatToIntegerMap != FloatToIntegerMapCopy, TEXT("Map instances should be duplicated"));
				AddErrorIfFalse(FloatToStringMap != FloatToStringMapCopy, TEXT("Map instances should be duplicated"));
				AddErrorIfFalse(TransformToTransformMap != TransformToTransformMapCopy, TEXT("Map instances should be duplicated"));
				AddErrorIfFalse(TransformToFloatMap != TransformToFloatMapCopy, TEXT("Map instances should be duplicated"));

				AddErrorIfFalse(FloatToFloatMap->GetTypedSet() == FloatToFloatMapCopy->GetTypedSet(), TEXT("Typed set mismatched"));
				AddErrorIfFalse(FloatToIntegerMap->GetTypedSet() == FloatToIntegerMapCopy->GetTypedSet(), TEXT("Typed set mismatched"));
				AddErrorIfFalse(FloatToStringMap->GetTypedSet() == FloatToStringMapCopy->GetTypedSet(), TEXT("Typed set mismatched"));
				AddErrorIfFalse(TransformToTransformMap->GetTypedSet() == TransformToTransformMapCopy->GetTypedSet(), TEXT("Typed set mismatched"));
				AddErrorIfFalse(TransformToFloatMap->GetTypedSet() == TransformToFloatMapCopy->GetTypedSet(), TEXT("Typed set mismatched"));

				// Move assignment
				{
					FBoundMapCollectionHeap CollectionMoved;
					CollectionMoved = MoveTemp(CollectionCopy);

					AddErrorIfFalse(CollectionCopy.IsEmpty(), TEXT("Collection should be empty"));
					AddErrorIfFalse(!CollectionMoved.IsEmpty(), TEXT("Collection should not be empty"));
					AddErrorIfFalse(CollectionMoved.Num() == Collection.Num(), TEXT("Unexpected map count"));

					TBoundValueMap<FFloatAnimationAttribute>* FloatToFloatMapMoved = CollectionMoved.Find<FFloatAnimationAttribute>(FloatToFloatKey);
					UE_RETURN_ON_ERROR(FloatToFloatMapMoved != nullptr, TEXT("Float->Float map should be valid"));

					TBoundValueMap<FIntegerAnimationAttribute>* FloatToIntegerMapMoved = CollectionMoved.Find<FIntegerAnimationAttribute>(FloatToIntegerKey);
					UE_RETURN_ON_ERROR(FloatToIntegerMapMoved != nullptr, TEXT("Float->Integer map should be valid"));

					TBoundValueMap<FStringAnimationAttribute>* FloatToStringMapMoved = CollectionMoved.Find<FStringAnimationAttribute>(FloatToStringKey);
					UE_RETURN_ON_ERROR(FloatToStringMapMoved != nullptr, TEXT("Float->String map should be valid"));

					TBoundValueMap<FBoneTransformAnimationAttribute>* TransformToTransformMapMoved = CollectionMoved.Find<FBoneTransformAnimationAttribute>(TransformToTransformKey);
					UE_RETURN_ON_ERROR(TransformToTransformMapMoved != nullptr, TEXT("Transform->Transform map should be valid"));

					TBoundValueMap<FFloatAnimationAttribute>* TransformToFloatMapMoved = Cast<FFloatAnimationAttribute>(CollectionMoved.Find(TransformToFloatKey));
					UE_RETURN_ON_ERROR(TransformToFloatMapMoved != nullptr, TEXT("Transform->Bone map should be valid"));

					AddErrorIfFalse(FloatToFloatMapMoved == FloatToFloatMapCopy, TEXT("Map instances should be moved"));
					AddErrorIfFalse(FloatToIntegerMapMoved == FloatToIntegerMapCopy, TEXT("Map instances should be moved"));
					AddErrorIfFalse(FloatToStringMapMoved == FloatToStringMapCopy, TEXT("Map instances should be moved"));
					AddErrorIfFalse(TransformToTransformMapMoved == TransformToTransformMapCopy, TEXT("Map instances should be moved"));
					AddErrorIfFalse(TransformToFloatMapMoved == TransformToFloatMapCopy, TEXT("Map instances should be moved"));
				}
			}

			// Iterators
			{
				const auto GetSetMapKeyProjection = [](const typename FBoundMapCollection::FIterator& It) { return It.GetKey(); };
				const auto GetSetMapKeyProjectionConst = [](const typename FBoundMapCollection::FConstIterator& It) { return It.GetKey(); };
				const auto GetSetMapAttributeTypeProjection = [](const typename FBoundMapCollection::FIterator& It) { return It.GetAttributeType(); };
				const auto GetSetMapAttributeTypeProjectionConst = [](const typename FBoundMapCollection::FConstIterator& It) { return It.GetAttributeType(); };
				const auto GetSetMapProjection = [](const typename FBoundMapCollection::FIterator& It) { return It.GetMap(); };
				const auto GetSetMapProjectionConst = [](const typename FBoundMapCollection::FConstIterator& It) { return It.GetMap(); };

				AddErrorIfFalse(IteratorSize(Collection.CreateIterator()) == 5, TEXT("Iterator did not have expected size"));
				AddErrorIfFalse(IteratorSize(Collection.CreateConstIterator()) == 5, TEXT("Iterator did not have expected size"));

				AddErrorIfFalse(IteratorSortedByPredicate(Collection.CreateIterator(), GetSetMapKeyProjection, TLess<FAttributeMappingKey>()), TEXT("Iterator should be sorted"));
				AddErrorIfFalse(IteratorSortedByPredicate(Collection.CreateConstIterator(), GetSetMapKeyProjectionConst, TLess<FAttributeMappingKey>()), TEXT("Iterator should be sorted"));

				AddErrorIfFalse(FindWithBy(Collection.CreateIterator(), FloatToFloatMap->GetTypedSet()->GetType(), GetSetMapAttributeTypeProjection), TEXT("Iterator did not contain expected attribute type"));
				AddErrorIfFalse(FindWithBy(Collection.CreateIterator(), FloatToIntegerMap->GetTypedSet()->GetType(), GetSetMapAttributeTypeProjection), TEXT("Iterator did not contain expected attribute type"));
				AddErrorIfFalse(FindWithBy(Collection.CreateIterator(), FloatToStringMap->GetTypedSet()->GetType(), GetSetMapAttributeTypeProjection), TEXT("Iterator did not contain expected attribute type"));
				AddErrorIfFalse(FindWithBy(Collection.CreateIterator(), TransformToTransformMap->GetTypedSet()->GetType(), GetSetMapAttributeTypeProjection), TEXT("Iterator did not contain expected attribute type"));
				AddErrorIfFalse(FindWithBy(Collection.CreateIterator(), TransformToFloatMap->GetTypedSet()->GetType(), GetSetMapAttributeTypeProjection), TEXT("Iterator did not contain expected attribute type"));

				AddErrorIfFalse(FindWithBy(Collection.CreateIterator(), FloatToFloatMap, GetSetMapProjection), TEXT("Iterator did not contain expected set map"));
				AddErrorIfFalse(FindWithBy(Collection.CreateIterator(), FloatToIntegerMap, GetSetMapProjection), TEXT("Iterator did not contain expected set map"));
				AddErrorIfFalse(FindWithBy(Collection.CreateIterator(), FloatToStringMap, GetSetMapProjection), TEXT("Iterator did not contain expected set map"));
				AddErrorIfFalse(FindWithBy(Collection.CreateIterator(), TransformToTransformMap, GetSetMapProjection), TEXT("Iterator did not contain expected set map"));
				AddErrorIfFalse(FindWithBy(Collection.CreateIterator(), TransformToFloatMap, GetSetMapProjection), TEXT("Iterator did not contain expected set map"));

				AddErrorIfFalse(FindWithBy(Collection.CreateConstIterator(), FloatToFloatMap->GetTypedSet()->GetType(), GetSetMapAttributeTypeProjectionConst), TEXT("Iterator did not contain expected attribute type"));
				AddErrorIfFalse(FindWithBy(Collection.CreateConstIterator(), FloatToIntegerMap->GetTypedSet()->GetType(), GetSetMapAttributeTypeProjectionConst), TEXT("Iterator did not contain expected attribute type"));
				AddErrorIfFalse(FindWithBy(Collection.CreateConstIterator(), FloatToStringMap->GetTypedSet()->GetType(), GetSetMapAttributeTypeProjectionConst), TEXT("Iterator did not contain expected attribute type"));
				AddErrorIfFalse(FindWithBy(Collection.CreateConstIterator(), TransformToTransformMap->GetTypedSet()->GetType(), GetSetMapAttributeTypeProjectionConst), TEXT("Iterator did not contain expected attribute type"));
				AddErrorIfFalse(FindWithBy(Collection.CreateConstIterator(), TransformToFloatMap->GetTypedSet()->GetType(), GetSetMapAttributeTypeProjectionConst), TEXT("Iterator did not contain expected attribute type"));

				AddErrorIfFalse(FindWithBy(Collection.CreateConstIterator(), FloatToFloatMap, GetSetMapProjectionConst), TEXT("Iterator did not contain expected set map"));
				AddErrorIfFalse(FindWithBy(Collection.CreateConstIterator(), FloatToIntegerMap, GetSetMapProjectionConst), TEXT("Iterator did not contain expected set map"));
				AddErrorIfFalse(FindWithBy(Collection.CreateConstIterator(), FloatToStringMap, GetSetMapProjectionConst), TEXT("Iterator did not contain expected set map"));
				AddErrorIfFalse(FindWithBy(Collection.CreateConstIterator(), TransformToTransformMap, GetSetMapProjectionConst), TEXT("Iterator did not contain expected set map"));
				AddErrorIfFalse(FindWithBy(Collection.CreateConstIterator(), TransformToFloatMap, GetSetMapProjectionConst), TEXT("Iterator did not contain expected set map"));
			}
		}

		return true;
	}
}

#endif	// WITH_DEV_AUTOMATION_TESTS
