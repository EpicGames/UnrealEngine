// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "AnimNextTest.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "UAF/AbstractSkeleton/SetBindingFactory.h"
#include "UAF/AbstractSkeleton/SetCollectionFactory.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetCollection.h"
#include "UAF/Attributes/AttributeBindingData.h"
#include "UAF/Attributes/EngineAttributes.h"
#include "UAF/ValueRuntime/BoundValueMap.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

namespace UE::UAF::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBoundValueMapTest, "Animation.UAF.ValueRuntime.BoundValueMap", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

		bool FBoundValueMapTest::RunTest(const FString& InParameters)
	{
		ON_SCOPE_EXIT{ FUtils::CleanupAfterTests(); };

		FAttributeBindingDataPtr BindingData;
		{
			UFactory* BindingFactory = NewObject<UFactory>(GetTransientPackage(), UAbstractSkeletonSetBindingFactory::StaticClass());
			UE_RETURN_ON_ERROR(BindingFactory != nullptr, "Failed to create factory");

			UAbstractSkeletonSetBinding* Binding = CastChecked<UAbstractSkeletonSetBinding>(BindingFactory->FactoryCreateNew(UAbstractSkeletonSetBinding::StaticClass(), GetTransientPackage(), TEXT("TestBindingAsset"), RF_Transient, nullptr, nullptr, NAME_None));
			UE_RETURN_ON_ERROR(Binding != nullptr, "Failed to create binding asset");

			USkeleton* Skeleton = NewObject<USkeleton>(GetTransientPackage(), USkeleton::StaticClass(), TEXT("TestSkeletonAsset"), RF_Transient);
			UE_RETURN_ON_ERROR(Skeleton != nullptr, "Failed to create skeleton asset");

			{
				// Not terribly clean, we cast away the 'const' to modify the skeleton
				FReferenceSkeleton& RefSkeleton = const_cast<FReferenceSkeleton&>(Skeleton->GetReferenceSkeleton());
				FReferenceSkeletonModifier SkeletonModifier(RefSkeleton, Skeleton);

				SkeletonModifier.Add(FMeshBoneInfo(TEXT("Root"), TEXT("Root"), INDEX_NONE), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("Pelvis"), TEXT("Root"), 0), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("Spine_01"), TEXT("Spine_01"), 1), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("Spine_02"), TEXT("Spine_02"), 2), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("Head"), TEXT("Head"), 3), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("LeftArm"), TEXT("LeftArm"), 3), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("RightArm"), TEXT("RightArm"), 3), FTransform::Identity);

				// When our modifier is destroyed here, it will rebuild the skeleton
			}

			UE_RETURN_ON_ERROR(Skeleton->GetReferenceSkeleton().GetNum() == 7, "Failed to build skeleton asset");

			bool bSuccess = Binding->SetSkeleton(Skeleton);
			UE_RETURN_ON_ERROR(bSuccess, "Failed to set skeleton on our binding");

			UFactory* SecCollectionFactory = NewObject<UFactory>(GetTransientPackage(), UAbstractSkeletonSetCollectionFactory::StaticClass());
			UE_RETURN_ON_ERROR(SecCollectionFactory != nullptr, "Failed to create factory");

			UAbstractSkeletonSetCollection* SetCollection = CastChecked<UAbstractSkeletonSetCollection>(SecCollectionFactory->FactoryCreateNew(UAbstractSkeletonSetCollection::StaticClass(), GetTransientPackage(), TEXT("TestSetCollectionAsset"), RF_Transient, nullptr, nullptr, NAME_None));
			UE_RETURN_ON_ERROR(SetCollection != nullptr, "Failed to create set collection asset");

			SetCollection->AddSet(TEXT("FullBody"), NAME_None);
			SetCollection->AddSet(TEXT("UpperBody"), TEXT("FullBody"));

			bSuccess = Binding->SetSetCollection(SetCollection);
			UE_RETURN_ON_ERROR(bSuccess, "Failed to set set collection on our binding");

			Binding->AddBoneToSet(TEXT("Root"), TEXT("FullBody"));
			Binding->AddBoneToSet(TEXT("Pelvis"), TEXT("FullBody"));
			Binding->AddBoneToSet(TEXT("Spine_01"), TEXT("FullBody"));
			Binding->AddBoneToSet(TEXT("Spine_02"), TEXT("FullBody"));
			Binding->AddAttributeToSet(FAnimationAttributeIdentifier(TEXT("AttrFloat0"), INDEX_NONE, NAME_None, FFloatAnimationAttribute::StaticStruct()), TEXT("FullBody"));

			Binding->AddBoneToSet(TEXT("Head"), TEXT("UpperBody"));
			Binding->AddBoneToSet(TEXT("LeftArm"), TEXT("UpperBody"));
			Binding->AddBoneToSet(TEXT("RightArm"), TEXT("UpperBody"));
			Binding->AddAttributeToSet(FAnimationAttributeIdentifier(TEXT("AttrFloat1"), INDEX_NONE, NAME_None, FFloatAnimationAttribute::StaticStruct()), TEXT("UpperBody"));

			BindingData = MakeAttributeBindingData(Binding, nullptr);
			UE_RETURN_ON_ERROR(BindingData.IsValid(), TEXT("Attribute binding data should be valid"));
		}

		{
			FReallocFun Allocator = &FAllocatorTypeTrait<FDefaultAllocator>::Realloc;

			FAttributeNamedSetPtr FullBodySet = BindingData->FindNamedSet(TEXT("FullBody"));
			UE_RETURN_ON_ERROR(FullBodySet.IsValid(), TEXT("FullBody named set should be valid"));

			FAttributeTypedSetPtr FullBodyBoneSet = FullBodySet->FindTypedSet<FBoneTransformAnimationAttribute>();
			UE_RETURN_ON_ERROR(FullBodyBoneSet.IsValid(), TEXT("FullBody bone typed set should be valid"));

			// Bone transform -> Float value
			{
				TBoundValueMap<FFloatAnimationAttribute>& BoneToFloatMap = *MakeBoundValueMap<FFloatAnimationAttribute>(FBoundValueMap::FConstructArgs(FullBodyBoneSet, FFloatAnimationAttribute::StaticStruct(), Allocator));

				AddErrorIfFalse(!BoneToFloatMap.IsEmpty(), TEXT("Attribute set map should not be empty"));
				AddErrorIfFalse(BoneToFloatMap.Num() == FullBodyBoneSet->Num(), TEXT("Unexpected set map size"));
				AddErrorIfFalse(BoneToFloatMap.GetValueType() == FFloatAnimationAttribute::StaticStruct(), TEXT("Unexpected set map value type"));

				FFloatAnimationAttribute Value0{ 123.44f };
				FFloatAnimationAttribute Value1{ -123.44f };
				BoneToFloatMap[FAttributeSetIndex(0)] = Value0;
				BoneToFloatMap[FAttributeSetIndex(1)] = Value1;

				AddErrorIfFalse(BoneToFloatMap[FAttributeSetIndex(0)].Value == Value0.Value, TEXT("Unexpected mapped value"));
				AddErrorIfFalse(BoneToFloatMap[FAttributeSetIndex(1)].Value == Value1.Value, TEXT("Unexpected mapped value"));

				FFloatAnimationAttribute Value2{ -0.13544f };
				BoneToFloatMap.SetValue(FAttributeSetIndex(2), Value2);

				AddErrorIfFalse(BoneToFloatMap.GetValue(FAttributeSetIndex(2)).Value == Value2.Value, TEXT("Unexpected mapped value"));

				// Copy
				{
					TBoundValueMap<FFloatAnimationAttribute>& BoneToFloatMapCopy = *MakeBoundValueMap<FFloatAnimationAttribute>(FBoundValueMap::FConstructArgs(FullBodyBoneSet, FFloatAnimationAttribute::StaticStruct(), Allocator));
					BoneToFloatMap.CopyTo(BoneToFloatMapCopy);

					AddErrorIfFalse(!BoneToFloatMapCopy.IsEmpty(), TEXT("Attribute set map should not be empty"));
					AddErrorIfFalse(BoneToFloatMapCopy.Num() == BoneToFloatMap.Num(), TEXT("Unexpected set map size"));
					AddErrorIfFalse(BoneToFloatMapCopy.GetValueType() == BoneToFloatMap.GetValueType(), TEXT("Unexpected set map value type"));

					AddErrorIfFalse(BoneToFloatMapCopy[FAttributeSetIndex(0)].Value == Value0.Value, TEXT("Unexpected mapped value"));
					AddErrorIfFalse(BoneToFloatMapCopy[FAttributeSetIndex(1)].Value == Value1.Value, TEXT("Unexpected mapped value"));
					AddErrorIfFalse(BoneToFloatMapCopy[FAttributeSetIndex(2)].Value == Value2.Value, TEXT("Unexpected mapped value"));

					ReleaseBoundValueMap(&BoneToFloatMapCopy);
				}

				// Duplicate
				{
					TBoundValueMap<FFloatAnimationAttribute>& BoneToFloatMapCopy = *reinterpret_cast<TBoundValueMap<FFloatAnimationAttribute>*>(BoneToFloatMap.Duplicate(Allocator));

					AddErrorIfFalse(!BoneToFloatMapCopy.IsEmpty(), TEXT("Attribute set map should not be empty"));
					AddErrorIfFalse(BoneToFloatMapCopy.Num() == BoneToFloatMap.Num(), TEXT("Unexpected set map size"));
					AddErrorIfFalse(BoneToFloatMapCopy.GetValueType() == BoneToFloatMap.GetValueType(), TEXT("Unexpected set map value type"));

					AddErrorIfFalse(BoneToFloatMapCopy[FAttributeSetIndex(0)].Value == Value0.Value, TEXT("Unexpected mapped value"));
					AddErrorIfFalse(BoneToFloatMapCopy[FAttributeSetIndex(1)].Value == Value1.Value, TEXT("Unexpected mapped value"));
					AddErrorIfFalse(BoneToFloatMapCopy[FAttributeSetIndex(2)].Value == Value2.Value, TEXT("Unexpected mapped value"));

					ReleaseBoundValueMap(&BoneToFloatMapCopy);
				}

				ReleaseBoundValueMap(&BoneToFloatMap);
			}

			// Bone transform -> Bone transform value
			{
				TBoundValueMap<FBoneTransformAnimationAttribute>& BoneToTransformMap = *MakeBoundValueMap<FBoneTransformAnimationAttribute>(FBoundValueMap::FConstructArgs(FullBodyBoneSet, FBoneTransformAnimationAttribute::StaticStruct(), Allocator));

				AddErrorIfFalse(!BoneToTransformMap.IsEmpty(), TEXT("Attribute set map should not be empty"));
				AddErrorIfFalse(BoneToTransformMap.Num() == FullBodyBoneSet->Num(), TEXT("Unexpected set map size"));
				AddErrorIfFalse(BoneToTransformMap.GetValueType() == FBoneTransformAnimationAttribute::StaticStruct(), TEXT("Unexpected set map value type"));

				FBoneTransformAnimationAttribute Value0{ FTransform::Identity };
				FBoneTransformAnimationAttribute Value1{ FTransform(FVector(1.2, 3.4, 5.6)) };
				BoneToTransformMap[FAttributeSetIndex(0)] = Value0;
				BoneToTransformMap[FAttributeSetIndex(1)] = Value1;

				AddErrorIfFalse(BoneToTransformMap[FAttributeSetIndex(0)].Value.Equals(Value0.Value), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(BoneToTransformMap[FAttributeSetIndex(1)].Value.Equals(Value1.Value), TEXT("Unexpected mapped value"));

				FBoneTransformAnimationAttribute Value2{ FTransform(FVector(0.233, 331.54, 571.146)) };
				BoneToTransformMap.SetValue(FAttributeSetIndex(2), Value2);

				AddErrorIfFalse(BoneToTransformMap.GetValue(FAttributeSetIndex(2)).Value.Equals(Value2.Value), TEXT("Unexpected mapped value"));

				// Copy
				{
					TBoundValueMap<FBoneTransformAnimationAttribute>& BoneToTransformMapCopy = *MakeBoundValueMap<FBoneTransformAnimationAttribute>(FBoundValueMap::FConstructArgs(FullBodyBoneSet, FBoneTransformAnimationAttribute::StaticStruct(), Allocator));
					BoneToTransformMap.CopyTo(BoneToTransformMapCopy);

					AddErrorIfFalse(!BoneToTransformMapCopy.IsEmpty(), TEXT("Attribute set map should not be empty"));
					AddErrorIfFalse(BoneToTransformMapCopy.Num() == BoneToTransformMap.Num(), TEXT("Unexpected set map size"));
					AddErrorIfFalse(BoneToTransformMapCopy.GetValueType() == BoneToTransformMap.GetValueType(), TEXT("Unexpected set map value type"));

					AddErrorIfFalse(BoneToTransformMapCopy[FAttributeSetIndex(0)].Value.Equals(Value0.Value), TEXT("Unexpected mapped value"));
					AddErrorIfFalse(BoneToTransformMapCopy[FAttributeSetIndex(1)].Value.Equals(Value1.Value), TEXT("Unexpected mapped value"));
					AddErrorIfFalse(BoneToTransformMapCopy[FAttributeSetIndex(2)].Value.Equals(Value2.Value), TEXT("Unexpected mapped value"));

					ReleaseBoundValueMap(&BoneToTransformMapCopy);
				}

				// Duplicate
				{
					TBoundValueMap<FBoneTransformAnimationAttribute>& BoneToTransformMapCopy = *reinterpret_cast<TBoundValueMap<FBoneTransformAnimationAttribute>*>(BoneToTransformMap.Duplicate(Allocator));

					AddErrorIfFalse(!BoneToTransformMapCopy.IsEmpty(), TEXT("Attribute set map should not be empty"));
					AddErrorIfFalse(BoneToTransformMapCopy.Num() == BoneToTransformMap.Num(), TEXT("Unexpected set map size"));
					AddErrorIfFalse(BoneToTransformMapCopy.GetValueType() == BoneToTransformMap.GetValueType(), TEXT("Unexpected set map value type"));

					AddErrorIfFalse(BoneToTransformMapCopy[FAttributeSetIndex(0)].Value.Equals(Value0.Value), TEXT("Unexpected mapped value"));
					AddErrorIfFalse(BoneToTransformMapCopy[FAttributeSetIndex(1)].Value.Equals(Value1.Value), TEXT("Unexpected mapped value"));
					AddErrorIfFalse(BoneToTransformMapCopy[FAttributeSetIndex(2)].Value.Equals(Value2.Value), TEXT("Unexpected mapped value"));

					ReleaseBoundValueMap(&BoneToTransformMapCopy);
				}

				ReleaseBoundValueMap(&BoneToTransformMap);
			}

			// Bone transform -> String value
			{
				TBoundValueMap<FStringAnimationAttribute>& BoneToStringMap = *MakeBoundValueMap<FStringAnimationAttribute>(FBoundValueMap::FConstructArgs(FullBodyBoneSet, FStringAnimationAttribute::StaticStruct(), Allocator));

				AddErrorIfFalse(!BoneToStringMap.IsEmpty(), TEXT("Attribute set map should not be empty"));
				AddErrorIfFalse(BoneToStringMap.Num() == FullBodyBoneSet->Num(), TEXT("Unexpected set map size"));
				AddErrorIfFalse(BoneToStringMap.GetValueType() == FStringAnimationAttribute::StaticStruct(), TEXT("Unexpected set map value type"));

				FStringAnimationAttribute Value0{ TEXT("FooBar") };
				FStringAnimationAttribute Value1{ TEXT("Sweet123") };
				BoneToStringMap[FAttributeSetIndex(0)] = Value0;
				BoneToStringMap[FAttributeSetIndex(1)] = Value1;

				AddErrorIfFalse(BoneToStringMap[FAttributeSetIndex(0)].Value.Equals(Value0.Value), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(BoneToStringMap[FAttributeSetIndex(1)].Value.Equals(Value1.Value), TEXT("Unexpected mapped value"));

				FStringAnimationAttribute Value2{ TEXT("01Wow") };
				BoneToStringMap.SetValue(FAttributeSetIndex(2), Value2);

				AddErrorIfFalse(BoneToStringMap.GetValue(FAttributeSetIndex(2)).Value.Equals(Value2.Value), TEXT("Unexpected mapped value"));

				// Copy
				{
					TBoundValueMap<FStringAnimationAttribute>& BoneToStringMapCopy = *MakeBoundValueMap<FStringAnimationAttribute>(FBoundValueMap::FConstructArgs(FullBodyBoneSet, FStringAnimationAttribute::StaticStruct(), Allocator));
					BoneToStringMap.CopyTo(BoneToStringMapCopy);

					AddErrorIfFalse(!BoneToStringMapCopy.IsEmpty(), TEXT("Attribute set map should not be empty"));
					AddErrorIfFalse(BoneToStringMapCopy.Num() == BoneToStringMap.Num(), TEXT("Unexpected set map size"));
					AddErrorIfFalse(BoneToStringMapCopy.GetValueType() == BoneToStringMap.GetValueType(), TEXT("Unexpected set map value type"));

					AddErrorIfFalse(BoneToStringMapCopy[FAttributeSetIndex(0)].Value.Equals(Value0.Value), TEXT("Unexpected mapped value"));
					AddErrorIfFalse(BoneToStringMapCopy[FAttributeSetIndex(1)].Value.Equals(Value1.Value), TEXT("Unexpected mapped value"));
					AddErrorIfFalse(BoneToStringMapCopy[FAttributeSetIndex(2)].Value.Equals(Value2.Value), TEXT("Unexpected mapped value"));

					ReleaseBoundValueMap(&BoneToStringMapCopy);
				}

				// Duplicate
				{
					TBoundValueMap<FStringAnimationAttribute>& BoneToStringMapCopy = *reinterpret_cast<TBoundValueMap<FStringAnimationAttribute>*>(BoneToStringMap.Duplicate(Allocator));

					AddErrorIfFalse(!BoneToStringMapCopy.IsEmpty(), TEXT("Attribute set map should not be empty"));
					AddErrorIfFalse(BoneToStringMapCopy.Num() == BoneToStringMap.Num(), TEXT("Unexpected set map size"));
					AddErrorIfFalse(BoneToStringMapCopy.GetValueType() == BoneToStringMap.GetValueType(), TEXT("Unexpected set map value type"));

					AddErrorIfFalse(BoneToStringMapCopy[FAttributeSetIndex(0)].Value.Equals(Value0.Value), TEXT("Unexpected mapped value"));
					AddErrorIfFalse(BoneToStringMapCopy[FAttributeSetIndex(1)].Value.Equals(Value1.Value), TEXT("Unexpected mapped value"));
					AddErrorIfFalse(BoneToStringMapCopy[FAttributeSetIndex(2)].Value.Equals(Value2.Value), TEXT("Unexpected mapped value"));

					ReleaseBoundValueMap(&BoneToStringMapCopy);
				}

				ReleaseBoundValueMap(&BoneToStringMap);
			}
		}

		return true;
	}
}

#endif	// WITH_DEV_AUTOMATION_TESTS
