// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::UAF::Tests
{
	// Tests if an iterator contains the specified value using the iterator projection functor
	template <typename IteratorType, typename ValueType, typename ProjectionType>
	inline bool FindWithBy(IteratorType&& It, const ValueType& Value, ProjectionType Proj)
	{
		while (It)
		{
			if (Invoke(Proj, It) == Value)
			{
				return true;
			}

			++It;
		}

		return false;
	}

	// Tests if an iterator contains the specified value using the iterator projection functor
	template <typename IteratorType, typename ValueType, typename ProjectionType, typename PredicateType>
	inline bool FindWithByPredicate(IteratorType&& It, const ValueType& Value, ProjectionType Proj, PredicateType Predicate)
	{
		while (It)
		{
			if (Predicate(Invoke(Proj, It), Value))
			{
				return true;
			}

			++It;
		}

		return false;
	}

	// Returns the size of the iterator
	template <typename IteratorType>
	inline int32 IteratorSize(IteratorType&& It)
	{
		int32 Size = 0;

		while (It)
		{
			++Size;
			++It;
		}

		return Size;
	}

	// Returns whether the iterator is sorted or not
	template <typename IteratorType, typename ProjectionType, typename PredicateType>
	inline bool IteratorSortedByPredicate(IteratorType&& It, ProjectionType Proj, PredicateType Predicate)
	{
		if (!It)
		{
			// Empty iterators are sorted
			return true;
		}

		auto&& PrevValue = Invoke(Proj, It);
		++It;

		while (It)
		{
			auto&& Value = Invoke(Proj, It);

			if (!Invoke(Predicate, PrevValue, Value))
			{
				return false;
			}

			PrevValue = MoveTemp(Value);
			++It;
		}

		return true;
	}
}
