// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "AnimNextTest.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "HAL/Platform.h"
#include "UAF/AbstractSkeleton/SetBindingFactory.h"
#include "UAF/AbstractSkeleton/SetCollectionFactory.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetCollection.h"
#include "UAF/Attributes/AttributeBindingData.h"
#include "UAF/Attributes/EngineAttributes.h"
#include "UAF/ValueRuntime/ValueBundle.h"
#include "UAF/ValueRuntime/IteratorUtils.h"
#include "UAF/ValueRuntime/Transformers/Transformer.h"
#include "UAF/ValueRuntime/Transformers/Sanitize.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

namespace UE::UAF::Tests
{
	static bool CollectionContains(const FBoundMapCollection& Collection, const FBoundValueMap* Map)
	{
		if (Map == nullptr)
		{
			return false;
		}

		const FBoundValueMap* Other = Collection.Find(Map->GetMappingKey());
		return Map == Other;
	}

	static bool CollectionContains(const FUnboundMapCollection& Collection, const FUnboundValueMap* Map)
	{
		if (Map == nullptr)
		{
			return false;
		}

		const FUnboundValueMap* Other = Collection.Find(Map->GetValueType());
		return Map == Other;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FIteratorUtilsInnerJoinTest, "Animation.UAF.ValueRuntime.IteratorUtils.InnerJoin", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool FIteratorUtilsInnerJoinTest::RunTest(const FString& InParameters)
	{
		ON_SCOPE_EXIT{ FUtils::CleanupAfterTests(); };

		FAttributeBindingDataPtr BindingData;
		{
			UFactory* BindingFactory = NewObject<UFactory>(GetTransientPackage(), UAbstractSkeletonSetBindingFactory::StaticClass());
			UE_RETURN_ON_ERROR(BindingFactory != nullptr, "Failed to create factory");

			UAbstractSkeletonSetBinding* Binding = CastChecked<UAbstractSkeletonSetBinding>(BindingFactory->FactoryCreateNew(UAbstractSkeletonSetBinding::StaticClass(), GetTransientPackage(), TEXT("TestBindingAsset"), RF_Transient, nullptr, nullptr, NAME_None));
			UE_RETURN_ON_ERROR(Binding != nullptr, "Failed to create binding asset");

			USkeleton* Skeleton = NewObject<USkeleton>(GetTransientPackage(), USkeleton::StaticClass(), TEXT("TestSkeletonAsset"), RF_Transient);
			UE_RETURN_ON_ERROR(Skeleton != nullptr, "Failed to create skeleton asset");

			{
				// Not terribly clean, we cast away the 'const' to modify the skeleton
				FReferenceSkeleton& RefSkeleton = const_cast<FReferenceSkeleton&>(Skeleton->GetReferenceSkeleton());
				FReferenceSkeletonModifier SkeletonModifier(RefSkeleton, Skeleton);

				SkeletonModifier.Add(FMeshBoneInfo(TEXT("Root"), TEXT("Root"), INDEX_NONE), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("Pelvis"), TEXT("Root"), 0), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("Spine_01"), TEXT("Spine_01"), 1), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("Spine_02"), TEXT("Spine_02"), 2), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("Head"), TEXT("Head"), 3), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("LeftArm"), TEXT("LeftArm"), 3), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("RightArm"), TEXT("RightArm"), 3), FTransform::Identity);

				// When our modifier is destroyed here, it will rebuild the skeleton
			}

			UE_RETURN_ON_ERROR(Skeleton->GetReferenceSkeleton().GetNum() == 7, "Failed to build skeleton asset");

			bool bSuccess = Binding->SetSkeleton(Skeleton);
			UE_RETURN_ON_ERROR(bSuccess, "Failed to set skeleton on our binding");

			UFactory* SecCollectionFactory = NewObject<UFactory>(GetTransientPackage(), UAbstractSkeletonSetCollectionFactory::StaticClass());
			UE_RETURN_ON_ERROR(SecCollectionFactory != nullptr, "Failed to create factory");

			UAbstractSkeletonSetCollection* SetCollection = CastChecked<UAbstractSkeletonSetCollection>(SecCollectionFactory->FactoryCreateNew(UAbstractSkeletonSetCollection::StaticClass(), GetTransientPackage(), TEXT("TestSetCollectionAsset"), RF_Transient, nullptr, nullptr, NAME_None));
			UE_RETURN_ON_ERROR(SetCollection != nullptr, "Failed to create set collection asset");

			SetCollection->AddSet(TEXT("FullBody"), NAME_None);
			SetCollection->AddSet(TEXT("UpperBody"), TEXT("FullBody"));

			bSuccess = Binding->SetSetCollection(SetCollection);
			UE_RETURN_ON_ERROR(bSuccess, "Failed to set set collection on our binding");

			Binding->AddBoneToSet(TEXT("Root"), TEXT("FullBody"));
			Binding->AddBoneToSet(TEXT("Pelvis"), TEXT("FullBody"));
			Binding->AddBoneToSet(TEXT("Spine_01"), TEXT("FullBody"));
			Binding->AddBoneToSet(TEXT("Spine_02"), TEXT("FullBody"));
			Binding->AddAttributeToSet(FAnimationAttributeIdentifier(TEXT("AttrFloat0"), INDEX_NONE, NAME_None, FFloatAnimationAttribute::StaticStruct()), TEXT("FullBody"));

			Binding->AddBoneToSet(TEXT("Head"), TEXT("UpperBody"));
			Binding->AddBoneToSet(TEXT("LeftArm"), TEXT("UpperBody"));
			Binding->AddBoneToSet(TEXT("RightArm"), TEXT("UpperBody"));
			Binding->AddAttributeToSet(FAnimationAttributeIdentifier(TEXT("AttrFloat1"), INDEX_NONE, NAME_None, FFloatAnimationAttribute::StaticStruct()), TEXT("UpperBody"));

			BindingData = MakeAttributeBindingData(Binding, nullptr);
			UE_RETURN_ON_ERROR(BindingData.IsValid(), TEXT("Attribute binding data should be valid"));
		}

		// Bound value map collection
		{
			FAttributeNamedSetPtr FullBodySet = BindingData->FindNamedSet(TEXT("FullBody"));
			UE_RETURN_ON_ERROR(FullBodySet.IsValid(), TEXT("FullBody named set should be valid"));

			FBoundMapCollectionHeap Collection0(FullBodySet);

			AddErrorIfFalse(Collection0.IsEmpty(), TEXT("Collection should be empty"));
			AddErrorIfFalse(Collection0.GetNamedSet() == FullBodySet, TEXT("Unexpected named set"));

			FAttributeMappingKey FloatToFloatKey = FAttributeMappingKey::MakeFromTo<FFloatAnimationAttribute, FFloatAnimationAttribute>();
			TBoundValueMap<FFloatAnimationAttribute>* FloatToFloatMap0 = Collection0.Add<FFloatAnimationAttribute>(FloatToFloatKey);
			UE_RETURN_ON_ERROR(FloatToFloatMap0 != nullptr, TEXT("Float->Float map should be valid"));

			FAttributeMappingKey FloatToIntegerKey = FAttributeMappingKey::MakeFromTo<FFloatAnimationAttribute, FIntegerAnimationAttribute>();
			TBoundValueMap<FIntegerAnimationAttribute>* FloatToIntegerMap0 = Collection0.Add<FIntegerAnimationAttribute>(FloatToIntegerKey);
			UE_RETURN_ON_ERROR(FloatToIntegerMap0 != nullptr, TEXT("Float->Integer map should be valid"));

			FAttributeMappingKey FloatToStringKey = FAttributeMappingKey::MakeFromTo<FFloatAnimationAttribute, FStringAnimationAttribute>();
			TBoundValueMap<FStringAnimationAttribute>* FloatToStringMap0 = Collection0.Add<FStringAnimationAttribute>(FloatToStringKey);
			UE_RETURN_ON_ERROR(FloatToStringMap0 != nullptr, TEXT("Float->String map should be valid"));

			FAttributeMappingKey TransformToTransformKey = FAttributeMappingKey::MakeFromTo<FBoneTransformAnimationAttribute, FBoneTransformAnimationAttribute>();
			TBoundValueMap<FBoneTransformAnimationAttribute>* TransformToTransformMap0 = Collection0.Add<FBoneTransformAnimationAttribute>(TransformToTransformKey);
			UE_RETURN_ON_ERROR(TransformToTransformMap0 != nullptr, TEXT("Transform->Transform map should be valid"));

			FAttributeMappingKey TransformToFloatKey = FAttributeMappingKey::MakeFromTo<FBoneTransformAnimationAttribute, FFloatAnimationAttribute>();
			TBoundValueMap<FFloatAnimationAttribute>* TransformToFloatMap0 = Collection0.Add<FFloatAnimationAttribute>(TransformToFloatKey);
			UE_RETURN_ON_ERROR(TransformToFloatMap0 != nullptr, TEXT("Transform->Bone map should be valid"));

			AddErrorIfFalse(!Collection0.IsEmpty(), TEXT("Collection should not be empty"));
			AddErrorIfFalse(Collection0.Num() == 5, TEXT("Unexpected map count"));

			FBoundMapCollectionHeap Collection1(FullBodySet);

			AddErrorIfFalse(Collection1.IsEmpty(), TEXT("Collection should be empty"));
			AddErrorIfFalse(Collection1.GetNamedSet() == FullBodySet, TEXT("Unexpected named set"));

			TBoundValueMap<FFloatAnimationAttribute>* FloatToFloatMap1 = Collection1.Add<FFloatAnimationAttribute>(FloatToFloatKey);
			UE_RETURN_ON_ERROR(FloatToFloatMap1 != nullptr, TEXT("Float->Float map should be valid"));

			TBoundValueMap<FIntegerAnimationAttribute>* FloatToIntegerMap1 = Collection1.Add<FIntegerAnimationAttribute>(FloatToIntegerKey);
			UE_RETURN_ON_ERROR(FloatToIntegerMap1 != nullptr, TEXT("Float->Integer map should be valid"));

			TBoundValueMap<FStringAnimationAttribute>* FloatToStringMap1 = Collection1.Add<FStringAnimationAttribute>(FloatToStringKey);
			UE_RETURN_ON_ERROR(FloatToStringMap1 != nullptr, TEXT("Float->String map should be valid"));

			TBoundValueMap<FBoneTransformAnimationAttribute>* TransformToTransformMap1 = Collection1.Add<FBoneTransformAnimationAttribute>(TransformToTransformKey);
			UE_RETURN_ON_ERROR(TransformToTransformMap1 != nullptr, TEXT("Transform->Transform map should be valid"));

			TBoundValueMap<FFloatAnimationAttribute>* TransformToFloatMap1 = Collection1.Add<FFloatAnimationAttribute>(TransformToFloatKey);
			UE_RETURN_ON_ERROR(TransformToFloatMap1 != nullptr, TEXT("Transform->Bone map should be valid"));

			AddErrorIfFalse(!Collection1.IsEmpty(), TEXT("Collection should not be empty"));
			AddErrorIfFalse(Collection1.Num() == 5, TEXT("Unexpected map count"));

			// Both inputs perfectly match
			{
				int32 NumPredicateCalls = 0;
				InnerJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1](FBoundValueMap* Map0, FBoundValueMap* Map1)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetAttributeType() == Map1->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection0.CreateIterator(),
					Collection1.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 5, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1](const FBoundValueMap* Map0, const FBoundValueMap* Map1)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetAttributeType() == Map1->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection0.CreateConstIterator(),
					Collection1.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 5, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1](const FBoundValueMap* Map0, FBoundValueMap* Map1)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetAttributeType() == Map1->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection0.CreateConstIterator(),
					Collection1.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 5, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1](FBoundValueMap* Map0, const FBoundValueMap* Map1)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetAttributeType() == Map1->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection0.CreateIterator(),
					Collection1.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 5, TEXT("Unexpected InnerJoin iteration result"));
			}

			FBoundMapCollectionHeap Collection2(FullBodySet);

			AddErrorIfFalse(Collection2.IsEmpty(), TEXT("Collection should be empty"));
			AddErrorIfFalse(Collection2.GetNamedSet() == FullBodySet, TEXT("Unexpected named set"));

			TBoundValueMap<FFloatAnimationAttribute>* FloatToFloatMap2 = Collection2.Add<FFloatAnimationAttribute>(FloatToFloatKey);
			UE_RETURN_ON_ERROR(FloatToFloatMap2 != nullptr, TEXT("Float->Float map should be valid"));

			TBoundValueMap<FBoneTransformAnimationAttribute>* TransformToTransformMap2 = Collection2.Add<FBoneTransformAnimationAttribute>(TransformToTransformKey);
			UE_RETURN_ON_ERROR(TransformToTransformMap2 != nullptr, TEXT("Transform->Transform map should be valid"));

			AddErrorIfFalse(!Collection2.IsEmpty(), TEXT("Collection should not be empty"));
			AddErrorIfFalse(Collection2.Num() == 2, TEXT("Unexpected map count"));

			// First input is shorter and its entries are skipped when missing
			{
				int32 NumPredicateCalls = 0;
				InnerJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection2](FBoundValueMap* Map2, FBoundValueMap* Map0)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetAttributeType() == Map2->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection2.CreateIterator(),
					Collection0.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection2](const FBoundValueMap* Map2, const FBoundValueMap* Map0)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetAttributeType() == Map2->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection2.CreateConstIterator(),
					Collection0.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection2](const FBoundValueMap* Map2, FBoundValueMap* Map0)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetAttributeType() == Map2->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection2.CreateConstIterator(),
					Collection0.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection2](FBoundValueMap* Map2, const FBoundValueMap* Map0)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetAttributeType() == Map2->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection2.CreateIterator(),
					Collection0.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected InnerJoin iteration result"));
			}

			// Second input is shorter and its entries are skipped when missing
			{
				int32 NumPredicateCalls = 0;
				InnerJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection2](FBoundValueMap* Map0, FBoundValueMap* Map2)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetAttributeType() == Map2->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection0.CreateIterator(),
					Collection2.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection2](const FBoundValueMap* Map0, const FBoundValueMap* Map2)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetAttributeType() == Map2->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection0.CreateConstIterator(),
					Collection2.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection2](const FBoundValueMap* Map0, FBoundValueMap* Map2)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetAttributeType() == Map2->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection0.CreateConstIterator(),
					Collection2.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection2](FBoundValueMap* Map0, const FBoundValueMap* Map2)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetAttributeType() == Map2->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection0.CreateIterator(),
					Collection2.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected InnerJoin iteration result"));
			}

			// 3 input variant
			{
				int32 NumPredicateCalls = 0;
				InnerJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1, &Collection2](FBoundValueMap* Map0, FBoundValueMap* Map1, FBoundValueMap* Map2)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetAttributeType() == Map1->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetAttributeType() == Map2->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection0.CreateIterator(),
					Collection1.CreateIterator(),
					Collection2.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1, &Collection2](FBoundValueMap* Map0, FBoundValueMap* Map2, FBoundValueMap* Map1)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetAttributeType() == Map1->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetAttributeType() == Map2->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection0.CreateIterator(),
					Collection2.CreateIterator(),
					Collection1.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1, &Collection2](FBoundValueMap* Map1, FBoundValueMap* Map0, FBoundValueMap* Map2)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetAttributeType() == Map1->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetAttributeType() == Map2->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection1.CreateIterator(),
					Collection0.CreateIterator(),
					Collection2.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1, &Collection2](FBoundValueMap* Map1, FBoundValueMap* Map2, FBoundValueMap* Map0)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetAttributeType() == Map1->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetAttributeType() == Map2->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection1.CreateIterator(),
					Collection2.CreateIterator(),
					Collection0.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1, &Collection2](FBoundValueMap* Map2, FBoundValueMap* Map0, FBoundValueMap* Map1)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetAttributeType() == Map1->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetAttributeType() == Map2->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection2.CreateIterator(),
					Collection0.CreateIterator(),
					Collection1.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1, &Collection2](FBoundValueMap* Map2, FBoundValueMap* Map1, FBoundValueMap* Map0)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetAttributeType() == Map1->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetAttributeType() == Map2->GetAttributeType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection2.CreateIterator(),
					Collection1.CreateIterator(),
					Collection0.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected InnerJoin iteration result"));
			}

			// Join with a transformer list
			{
				// Use a float transformer
				// We set up Collection0 such that it has two mappings with float values: Transform -> Float and Float -> Float
				// InnerJoin should execute the predicate twice, once for each value type that matches the transformer's value type

				// Dummy float sanitize transformer
				struct FFloatAttributeTransformer_SanitizeTest
				{
					using TransformerType = Transformers::FSanitize;
					using ValueType = FFloatAnimationAttribute;

					static void TransformBoundValueMap(FBoundValueMap* SetMap) {}	// No-op
				};

				// Dummy vector sanitize transformer
				struct FVectorAttributeTransformer_SanitizeTest
				{
					using TransformerType = Transformers::FSanitize;
					using ValueType = FVectorAnimationAttribute;

					static void TransformBoundValueMap(FBoundValueMap* SetMap) {}	// No-op
				};

PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

				FValueTransformerList TransformerList(Transformers::FSanitize::TransformerName);
				TransformerList.AddBoundValueMapTransformer(FFloatAnimationAttribute::StaticStruct(), reinterpret_cast<FRawTransformerFunc>(&FFloatAttributeTransformer_SanitizeTest::TransformBoundValueMap));
				TransformerList.AddBoundValueMapTransformer(FVectorAnimationAttribute::StaticStruct(), reinterpret_cast<FRawTransformerFunc>(&FVectorAttributeTransformer_SanitizeTest::TransformBoundValueMap));

PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

				int32 NumPredicateCalls = 0;
				InnerJoinBy(
					FValueTransformerListWithBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0](FRawTransformerFunc Transformer, FBoundValueMap* SetMap)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, SetMap), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(SetMap->GetValueType() == FFloatAnimationAttribute::StaticStruct(), TEXT("Unexpected InnerJoin iteration result"));
					},
					TransformerList.CreateBoundValueMapTransformerIterator(),
					Collection0.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected InnerJoin iteration result"));
			}
		}

		// Unbound value map collection
		{
			FUnboundMapCollectionHeap Collection0;

			AddErrorIfFalse(Collection0.IsEmpty(), TEXT("Collection should be empty"));

			TUnboundValueMap<FFloatAnimationAttribute>* FloatMap0 = Collection0.Add<FFloatAnimationAttribute>();
			UE_RETURN_ON_ERROR(FloatMap0 != nullptr, TEXT("Float map should be valid"));

			TUnboundValueMap<FBoneTransformAnimationAttribute>* BoneMap0 = Collection0.Add<FBoneTransformAnimationAttribute>();
			UE_RETURN_ON_ERROR(BoneMap0 != nullptr, TEXT("Bone map should be valid"));

			AddErrorIfFalse(!Collection0.IsEmpty(), TEXT("Collection should not be empty"));
			AddErrorIfFalse(Collection0.Num() == 2, TEXT("Unexpected map count"));

			FUnboundMapCollectionHeap Collection1;

			AddErrorIfFalse(Collection1.IsEmpty(), TEXT("Collection should be empty"));

			TUnboundValueMap<FFloatAnimationAttribute>* FloatMap1 = Collection1.Add<FFloatAnimationAttribute>();
			UE_RETURN_ON_ERROR(FloatMap1 != nullptr, TEXT("Float map should be valid"));

			TUnboundValueMap<FBoneTransformAnimationAttribute>* BoneMap1 = Collection1.Add<FBoneTransformAnimationAttribute>();
			UE_RETURN_ON_ERROR(BoneMap1 != nullptr, TEXT("Bone map should be valid"));

			AddErrorIfFalse(!Collection1.IsEmpty(), TEXT("Collection should not be empty"));
			AddErrorIfFalse(Collection1.Num() == 2, TEXT("Unexpected map count"));

			// Both inputs perfectly match
			{
				int32 NumPredicateCalls = 0;
				InnerJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1](FUnboundValueMap* Map0, FUnboundValueMap* Map1)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection0.CreateIterator(),
					Collection1.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1](const FUnboundValueMap* Map0, const FUnboundValueMap* Map1)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection0.CreateConstIterator(),
					Collection1.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1](const FUnboundValueMap* Map0, FUnboundValueMap* Map1)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection0.CreateConstIterator(),
					Collection1.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1](FUnboundValueMap* Map0, const FUnboundValueMap* Map1)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection0.CreateIterator(),
					Collection1.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected InnerJoin iteration result"));
			}

			FUnboundMapCollectionHeap Collection2;

			AddErrorIfFalse(Collection2.IsEmpty(), TEXT("Collection should be empty"));

			TUnboundValueMap<FFloatAnimationAttribute>* FloatMap2 = Collection2.Add<FFloatAnimationAttribute>();
			UE_RETURN_ON_ERROR(FloatMap2 != nullptr, TEXT("Float map should be valid"));

			AddErrorIfFalse(!Collection2.IsEmpty(), TEXT("Collection should not be empty"));
			AddErrorIfFalse(Collection2.Num() == 1, TEXT("Unexpected map count"));

			// First input is shorter and its entries are skipped when missing
			{
				int32 NumPredicateCalls = 0;
				InnerJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection2](FUnboundValueMap* Map2, FUnboundValueMap* Map0)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection2.CreateIterator(),
					Collection0.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 1, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection2](const FUnboundValueMap* Map2, const FUnboundValueMap* Map0)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection2.CreateConstIterator(),
					Collection0.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 1, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection2](const FUnboundValueMap* Map2, FUnboundValueMap* Map0)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection2.CreateConstIterator(),
					Collection0.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 1, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection2](FUnboundValueMap* Map2, const FUnboundValueMap* Map0)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection2.CreateIterator(),
					Collection0.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 1, TEXT("Unexpected InnerJoin iteration result"));
			}

			// Second input is shorter and its entries are skipped when missing
			{
				int32 NumPredicateCalls = 0;
				InnerJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection2](FUnboundValueMap* Map0, FUnboundValueMap* Map2)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection0.CreateIterator(),
					Collection2.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 1, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection2](const FUnboundValueMap* Map0, const FUnboundValueMap* Map2)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection0.CreateConstIterator(),
					Collection2.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 1, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection2](const FUnboundValueMap* Map0, FUnboundValueMap* Map2)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection0.CreateConstIterator(),
					Collection2.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 1, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection2](FUnboundValueMap* Map0, const FUnboundValueMap* Map2)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection0.CreateIterator(),
					Collection2.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 1, TEXT("Unexpected InnerJoin iteration result"));
			}

			// 3 input variant
			{
				int32 NumPredicateCalls = 0;
				InnerJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1, &Collection2](FUnboundValueMap* Map0, FUnboundValueMap* Map1, FUnboundValueMap* Map2)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection0.CreateIterator(),
					Collection1.CreateIterator(),
					Collection2.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 1, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1, &Collection2](FUnboundValueMap* Map0, FUnboundValueMap* Map2, FUnboundValueMap* Map1)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection0.CreateIterator(),
					Collection2.CreateIterator(),
					Collection1.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 1, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1, &Collection2](FUnboundValueMap* Map1, FUnboundValueMap* Map0, FUnboundValueMap* Map2)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection1.CreateIterator(),
					Collection0.CreateIterator(),
					Collection2.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 1, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1, &Collection2](FUnboundValueMap* Map1, FUnboundValueMap* Map2, FUnboundValueMap* Map0)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection1.CreateIterator(),
					Collection2.CreateIterator(),
					Collection0.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 1, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1, &Collection2](FUnboundValueMap* Map2, FUnboundValueMap* Map0, FUnboundValueMap* Map1)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection2.CreateIterator(),
					Collection0.CreateIterator(),
					Collection1.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 1, TEXT("Unexpected InnerJoin iteration result"));

				NumPredicateCalls = 0;
				InnerJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1, &Collection2](FUnboundValueMap* Map2, FUnboundValueMap* Map1, FUnboundValueMap* Map0)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map2 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected InnerJoin iteration result"));
					},
					Collection2.CreateIterator(),
					Collection1.CreateIterator(),
					Collection0.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 1, TEXT("Unexpected InnerJoin iteration result"));
			}

			// Join with a transformer list
			{
				// Use a float transformer
				// We set up Collection0 such that it has two mappings with these value types: Float and Bone Transform
				// InnerJoin should execute the predicate twice, once for each value type that matches the transformer's value type

				// Dummy float sanitize transformer
				struct FFloatAttributeTransformer_SanitizeTest
				{
					using TransformerType = Transformers::FSanitize;
					using ValueType = FFloatAnimationAttribute;

					static void TransformMap(FUnboundValueMap* Map) {}	// No-op
				};

				// Dummy vector sanitize transformer
				struct FVectorAttributeTransformer_SanitizeTest
				{
					using TransformerType = Transformers::FSanitize;
					using ValueType = FVectorAnimationAttribute;

					static void TransformMap(FUnboundValueMap* Map) {}	// No-op
				};

PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

				FValueTransformerList TransformerList(Transformers::FSanitize::TransformerName);
				TransformerList.AddUnboundValueMapTransformer(FFloatAnimationAttribute::StaticStruct(), reinterpret_cast<FRawTransformerFunc>(&FFloatAttributeTransformer_SanitizeTest::TransformMap));
				TransformerList.AddUnboundValueMapTransformer(FVectorAnimationAttribute::StaticStruct(), reinterpret_cast<FRawTransformerFunc>(&FVectorAttributeTransformer_SanitizeTest::TransformMap));

PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

				int32 NumPredicateCalls = 0;
				InnerJoinBy(
					FValueTransformerListWithUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0](FRawTransformerFunc Transformer, FUnboundValueMap* Map)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map), TEXT("Unexpected InnerJoin iteration result"));
						AddErrorIfFalse(Map->GetValueType() == FFloatAnimationAttribute::StaticStruct(), TEXT("Unexpected InnerJoin iteration result"));
					},
					TransformerList.CreateUnboundValueMapTransformerIterator(),
					Collection0.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 1, TEXT("Unexpected InnerJoin iteration result"));
			}
		}

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FIteratorUtilsOuterJoinTest, "Animation.UAF.ValueRuntime.IteratorUtils.OuterJoin", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool FIteratorUtilsOuterJoinTest::RunTest(const FString& InParameters)
	{
		ON_SCOPE_EXIT{ FUtils::CleanupAfterTests(); };

		FAttributeBindingDataPtr BindingData;
		{
			UFactory* BindingFactory = NewObject<UFactory>(GetTransientPackage(), UAbstractSkeletonSetBindingFactory::StaticClass());
			UE_RETURN_ON_ERROR(BindingFactory != nullptr, "Failed to create factory");

			UAbstractSkeletonSetBinding* Binding = CastChecked<UAbstractSkeletonSetBinding>(BindingFactory->FactoryCreateNew(UAbstractSkeletonSetBinding::StaticClass(), GetTransientPackage(), TEXT("TestBindingAsset"), RF_Transient, nullptr, nullptr, NAME_None));
			UE_RETURN_ON_ERROR(Binding != nullptr, "Failed to create binding asset");

			USkeleton* Skeleton = NewObject<USkeleton>(GetTransientPackage(), USkeleton::StaticClass(), TEXT("TestSkeletonAsset"), RF_Transient);
			UE_RETURN_ON_ERROR(Skeleton != nullptr, "Failed to create skeleton asset");

			{
				// Not terribly clean, we cast away the 'const' to modify the skeleton
				FReferenceSkeleton& RefSkeleton = const_cast<FReferenceSkeleton&>(Skeleton->GetReferenceSkeleton());
				FReferenceSkeletonModifier SkeletonModifier(RefSkeleton, Skeleton);

				SkeletonModifier.Add(FMeshBoneInfo(TEXT("Root"), TEXT("Root"), INDEX_NONE), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("Pelvis"), TEXT("Root"), 0), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("Spine_01"), TEXT("Spine_01"), 1), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("Spine_02"), TEXT("Spine_02"), 2), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("Head"), TEXT("Head"), 3), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("LeftArm"), TEXT("LeftArm"), 3), FTransform::Identity);
				SkeletonModifier.Add(FMeshBoneInfo(TEXT("RightArm"), TEXT("RightArm"), 3), FTransform::Identity);

				// When our modifier is destroyed here, it will rebuild the skeleton
			}

			UE_RETURN_ON_ERROR(Skeleton->GetReferenceSkeleton().GetNum() == 7, "Failed to build skeleton asset");

			bool bSuccess = Binding->SetSkeleton(Skeleton);
			UE_RETURN_ON_ERROR(bSuccess, "Failed to set skeleton on our binding");

			UFactory* SecCollectionFactory = NewObject<UFactory>(GetTransientPackage(), UAbstractSkeletonSetCollectionFactory::StaticClass());
			UE_RETURN_ON_ERROR(SecCollectionFactory != nullptr, "Failed to create factory");

			UAbstractSkeletonSetCollection* SetCollection = CastChecked<UAbstractSkeletonSetCollection>(SecCollectionFactory->FactoryCreateNew(UAbstractSkeletonSetCollection::StaticClass(), GetTransientPackage(), TEXT("TestSetCollectionAsset"), RF_Transient, nullptr, nullptr, NAME_None));
			UE_RETURN_ON_ERROR(SetCollection != nullptr, "Failed to create set collection asset");

			SetCollection->AddSet(TEXT("FullBody"), NAME_None);
			SetCollection->AddSet(TEXT("UpperBody"), TEXT("FullBody"));

			bSuccess = Binding->SetSetCollection(SetCollection);
			UE_RETURN_ON_ERROR(bSuccess, "Failed to set set collection on our binding");

			Binding->AddBoneToSet(TEXT("Root"), TEXT("FullBody"));
			Binding->AddBoneToSet(TEXT("Pelvis"), TEXT("FullBody"));
			Binding->AddBoneToSet(TEXT("Spine_01"), TEXT("FullBody"));
			Binding->AddBoneToSet(TEXT("Spine_02"), TEXT("FullBody"));
			Binding->AddAttributeToSet(FAnimationAttributeIdentifier(TEXT("AttrFloat0"), INDEX_NONE, NAME_None, FFloatAnimationAttribute::StaticStruct()), TEXT("FullBody"));

			Binding->AddBoneToSet(TEXT("Head"), TEXT("UpperBody"));
			Binding->AddBoneToSet(TEXT("LeftArm"), TEXT("UpperBody"));
			Binding->AddBoneToSet(TEXT("RightArm"), TEXT("UpperBody"));
			Binding->AddAttributeToSet(FAnimationAttributeIdentifier(TEXT("AttrFloat1"), INDEX_NONE, NAME_None, FFloatAnimationAttribute::StaticStruct()), TEXT("UpperBody"));

			BindingData = MakeAttributeBindingData(Binding, nullptr);
			UE_RETURN_ON_ERROR(BindingData.IsValid(), TEXT("Attribute binding data should be valid"));
		}

		// Bound value map collection
		{
			FAttributeNamedSetPtr FullBodySet = BindingData->FindNamedSet(TEXT("FullBody"));
			UE_RETURN_ON_ERROR(FullBodySet.IsValid(), TEXT("FullBody named set should be valid"));

			FBoundMapCollectionHeap Collection0(FullBodySet);

			AddErrorIfFalse(Collection0.IsEmpty(), TEXT("Collection should be empty"));
			AddErrorIfFalse(Collection0.GetNamedSet() == FullBodySet, TEXT("Unexpected named set"));

			FAttributeMappingKey FloatToFloatKey = FAttributeMappingKey::MakeFromTo<FFloatAnimationAttribute, FFloatAnimationAttribute>();
			TBoundValueMap<FFloatAnimationAttribute>* FloatToFloatMap0 = Collection0.Add<FFloatAnimationAttribute>(FloatToFloatKey);
			UE_RETURN_ON_ERROR(FloatToFloatMap0 != nullptr, TEXT("Float->Float map should be valid"));

			FAttributeMappingKey FloatToIntegerKey = FAttributeMappingKey::MakeFromTo<FFloatAnimationAttribute, FIntegerAnimationAttribute>();
			TBoundValueMap<FIntegerAnimationAttribute>* FloatToIntegerMap0 = Collection0.Add<FIntegerAnimationAttribute>(FloatToIntegerKey);
			UE_RETURN_ON_ERROR(FloatToIntegerMap0 != nullptr, TEXT("Float->Integer map should be valid"));

			FAttributeMappingKey FloatToStringKey = FAttributeMappingKey::MakeFromTo<FFloatAnimationAttribute, FStringAnimationAttribute>();
			TBoundValueMap<FStringAnimationAttribute>* FloatToStringMap0 = Collection0.Add<FStringAnimationAttribute>(FloatToStringKey);
			UE_RETURN_ON_ERROR(FloatToStringMap0 != nullptr, TEXT("Float->String map should be valid"));

			FAttributeMappingKey TransformToTransformKey = FAttributeMappingKey::MakeFromTo<FBoneTransformAnimationAttribute, FBoneTransformAnimationAttribute>();
			TBoundValueMap<FBoneTransformAnimationAttribute>* TransformToTransformMap0 = Collection0.Add<FBoneTransformAnimationAttribute>(TransformToTransformKey);
			UE_RETURN_ON_ERROR(TransformToTransformMap0 != nullptr, TEXT("Transform->Transform map should be valid"));

			FAttributeMappingKey TransformToFloatKey = FAttributeMappingKey::MakeFromTo<FBoneTransformAnimationAttribute, FFloatAnimationAttribute>();
			TBoundValueMap<FFloatAnimationAttribute>* TransformToFloatMap0 = Collection0.Add<FFloatAnimationAttribute>(TransformToFloatKey);
			UE_RETURN_ON_ERROR(TransformToFloatMap0 != nullptr, TEXT("Transform->Bone map should be valid"));

			AddErrorIfFalse(!Collection0.IsEmpty(), TEXT("Collection should not be empty"));
			AddErrorIfFalse(Collection0.Num() == 5, TEXT("Unexpected map count"));

			FBoundMapCollectionHeap Collection1(FullBodySet);

			AddErrorIfFalse(Collection1.IsEmpty(), TEXT("Collection should be empty"));
			AddErrorIfFalse(Collection1.GetNamedSet() == FullBodySet, TEXT("Unexpected named set"));

			TBoundValueMap<FFloatAnimationAttribute>* FloatToFloatMap1 = Collection1.Add<FFloatAnimationAttribute>(FloatToFloatKey);
			UE_RETURN_ON_ERROR(FloatToFloatMap1 != nullptr, TEXT("Float->Float map should be valid"));

			TBoundValueMap<FIntegerAnimationAttribute>* FloatToIntegerMap1 = Collection1.Add<FIntegerAnimationAttribute>(FloatToIntegerKey);
			UE_RETURN_ON_ERROR(FloatToIntegerMap1 != nullptr, TEXT("Float->Integer map should be valid"));

			TBoundValueMap<FStringAnimationAttribute>* FloatToStringMap1 = Collection1.Add<FStringAnimationAttribute>(FloatToStringKey);
			UE_RETURN_ON_ERROR(FloatToStringMap1 != nullptr, TEXT("Float->String map should be valid"));

			TBoundValueMap<FBoneTransformAnimationAttribute>* TransformToTransformMap1 = Collection1.Add<FBoneTransformAnimationAttribute>(TransformToTransformKey);
			UE_RETURN_ON_ERROR(TransformToTransformMap1 != nullptr, TEXT("Transform->Transform map should be valid"));

			TBoundValueMap<FFloatAnimationAttribute>* TransformToFloatMap1 = Collection1.Add<FFloatAnimationAttribute>(TransformToFloatKey);
			UE_RETURN_ON_ERROR(TransformToFloatMap1 != nullptr, TEXT("Transform->Bone map should be valid"));

			AddErrorIfFalse(!Collection1.IsEmpty(), TEXT("Collection should not be empty"));
			AddErrorIfFalse(Collection1.Num() == 5, TEXT("Unexpected map count"));

			// Both inputs perfectly match
			{
				int32 NumPredicateCalls = 0;
				OuterJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1](FBoundValueMap* Map0, FBoundValueMap* Map1)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected OuterJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetAttributeType() == Map1->GetAttributeType(), TEXT("Unexpected OuterJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
					},
					Collection0.CreateIterator(),
					Collection1.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 5, TEXT("Unexpected OuterJoin iteration result"));

				NumPredicateCalls = 0;
				OuterJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1](const FBoundValueMap* Map0, const FBoundValueMap* Map1)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected OuterJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetAttributeType() == Map1->GetAttributeType(), TEXT("Unexpected OuterJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
					},
					Collection0.CreateConstIterator(),
					Collection1.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 5, TEXT("Unexpected OuterJoin iteration result"));

				NumPredicateCalls = 0;
				OuterJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1](const FBoundValueMap* Map0, FBoundValueMap* Map1)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected OuterJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetAttributeType() == Map1->GetAttributeType(), TEXT("Unexpected OuterJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
					},
					Collection0.CreateConstIterator(),
					Collection1.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 5, TEXT("Unexpected OuterJoin iteration result"));

				NumPredicateCalls = 0;
				OuterJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1](FBoundValueMap* Map0, const FBoundValueMap* Map1)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected OuterJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetAttributeType() == Map1->GetAttributeType(), TEXT("Unexpected OuterJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
					},
					Collection0.CreateIterator(),
					Collection1.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 5, TEXT("Unexpected OuterJoin iteration result"));
			}

			FBoundMapCollectionHeap Collection2(FullBodySet);

			AddErrorIfFalse(Collection2.IsEmpty(), TEXT("Collection should be empty"));
			AddErrorIfFalse(Collection2.GetNamedSet() == FullBodySet, TEXT("Unexpected named set"));

			TBoundValueMap<FFloatAnimationAttribute>* FloatToFloatMap2 = Collection2.Add<FFloatAnimationAttribute>(FloatToFloatKey);
			UE_RETURN_ON_ERROR(FloatToFloatMap2 != nullptr, TEXT("Float->Float map should be valid"));

			TBoundValueMap<FBoneTransformAnimationAttribute>* TransformToTransformMap2 = Collection2.Add<FBoneTransformAnimationAttribute>(TransformToTransformKey);
			UE_RETURN_ON_ERROR(TransformToTransformMap2 != nullptr, TEXT("Transform->Transform map should be valid"));

			AddErrorIfFalse(!Collection2.IsEmpty(), TEXT("Collection should not be empty"));
			AddErrorIfFalse(Collection2.Num() == 2, TEXT("Unexpected map count"));

			// First input is shorter and will be padded with null entries
			{
				int32 NumPredicateCalls = 0;
				int32 NumNullEntries = 0;
				OuterJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &NumNullEntries, &Collection0, &Collection2](FBoundValueMap* Map2, FBoundValueMap* Map0)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));

						if (Map2 != nullptr)
						{
							AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetAttributeType() == Map2->GetAttributeType(), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
						}
						else
						{
							NumNullEntries++;
							AddErrorIfFalse(Map0 != nullptr, TEXT("Unexpected OuterJoin iteration result"));
						}
					},
					Collection2.CreateIterator(),
					Collection0.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 5, TEXT("Unexpected OuterJoin iteration result"));
				AddErrorIfFalse(NumNullEntries == 3, TEXT("Unexpected OuterJoin iteration result"));

				NumPredicateCalls = 0;
				NumNullEntries = 0;
				OuterJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &NumNullEntries, &Collection0, &Collection2](const FBoundValueMap* Map2, const FBoundValueMap* Map0)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));

						if (Map2 != nullptr)
						{
							AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetAttributeType() == Map2->GetAttributeType(), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
						}
						else
						{
							NumNullEntries++;
							AddErrorIfFalse(Map0 != nullptr, TEXT("Unexpected OuterJoin iteration result"));
						}
					},
					Collection2.CreateConstIterator(),
					Collection0.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 5, TEXT("Unexpected OuterJoin iteration result"));
				AddErrorIfFalse(NumNullEntries == 3, TEXT("Unexpected OuterJoin iteration result"));

				NumPredicateCalls = 0;
				NumNullEntries = 0;
				OuterJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &NumNullEntries, &Collection0, &Collection2](const FBoundValueMap* Map2, FBoundValueMap* Map0)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));

						if (Map2 != nullptr)
						{
							AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetAttributeType() == Map2->GetAttributeType(), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
						}
						else
						{
							NumNullEntries++;
							AddErrorIfFalse(Map0 != nullptr, TEXT("Unexpected OuterJoin iteration result"));
						}
					},
					Collection2.CreateConstIterator(),
					Collection0.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 5, TEXT("Unexpected OuterJoin iteration result"));
				AddErrorIfFalse(NumNullEntries == 3, TEXT("Unexpected OuterJoin iteration result"));

				NumPredicateCalls = 0;
				NumNullEntries = 0;
				OuterJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &NumNullEntries, &Collection0, &Collection2](FBoundValueMap* Map2, const FBoundValueMap* Map0)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));

						if (Map2 != nullptr)
						{
							AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetAttributeType() == Map2->GetAttributeType(), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
						}
						else
						{
							NumNullEntries++;
							AddErrorIfFalse(Map0 != nullptr, TEXT("Unexpected OuterJoin iteration result"));
						}
					},
					Collection2.CreateIterator(),
					Collection0.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 5, TEXT("Unexpected OuterJoin iteration result"));
				AddErrorIfFalse(NumNullEntries == 3, TEXT("Unexpected OuterJoin iteration result"));
			}

			// Second input is shorter and will be padded with null entries
			{
				int32 NumPredicateCalls = 0;
				int32 NumNullEntries = 0;
				OuterJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &NumNullEntries, &Collection0, &Collection2](FBoundValueMap* Map0, FBoundValueMap* Map2)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));

						if (Map2 != nullptr)
						{
							AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetAttributeType() == Map2->GetAttributeType(), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
						}
						else
						{
							NumNullEntries++;
							AddErrorIfFalse(Map0 != nullptr, TEXT("Unexpected OuterJoin iteration result"));
						}
					},
					Collection0.CreateIterator(),
					Collection2.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 5, TEXT("Unexpected OuterJoin iteration result"));
				AddErrorIfFalse(NumNullEntries == 3, TEXT("Unexpected OuterJoin iteration result"));

				NumPredicateCalls = 0;
				NumNullEntries = 0;
				OuterJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &NumNullEntries, &Collection0, &Collection2](const FBoundValueMap* Map0, const FBoundValueMap* Map2)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));

						if (Map2 != nullptr)
						{
							AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetAttributeType() == Map2->GetAttributeType(), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
						}
						else
						{
							NumNullEntries++;
							AddErrorIfFalse(Map0 != nullptr, TEXT("Unexpected OuterJoin iteration result"));
						}
					},
					Collection0.CreateConstIterator(),
					Collection2.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 5, TEXT("Unexpected OuterJoin iteration result"));
				AddErrorIfFalse(NumNullEntries == 3, TEXT("Unexpected OuterJoin iteration result"));

				NumPredicateCalls = 0;
				NumNullEntries = 0;
				OuterJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &NumNullEntries, &Collection0, &Collection2](const FBoundValueMap* Map0, FBoundValueMap* Map2)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));

						if (Map2 != nullptr)
						{
							AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetAttributeType() == Map2->GetAttributeType(), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
						}
						else
						{
							NumNullEntries++;
							AddErrorIfFalse(Map0 != nullptr, TEXT("Unexpected OuterJoin iteration result"));
						}
					},
					Collection0.CreateConstIterator(),
					Collection2.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 5, TEXT("Unexpected OuterJoin iteration result"));
				AddErrorIfFalse(NumNullEntries == 3, TEXT("Unexpected OuterJoin iteration result"));

				NumPredicateCalls = 0;
				NumNullEntries = 0;
				OuterJoinBy(
					FBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &NumNullEntries, &Collection0, &Collection2](FBoundValueMap* Map0, const FBoundValueMap* Map2)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));

						if (Map2 != nullptr)
						{
							AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetAttributeType() == Map2->GetAttributeType(), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
						}
						else
						{
							NumNullEntries++;
							AddErrorIfFalse(Map0 != nullptr, TEXT("Unexpected OuterJoin iteration result"));
						}
					},
					Collection0.CreateIterator(),
					Collection2.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 5, TEXT("Unexpected OuterJoin iteration result"));
				AddErrorIfFalse(NumNullEntries == 3, TEXT("Unexpected OuterJoin iteration result"));
			}

			// Join with a transformer list
			{
				// Use a float transformer
				// We set up Collection0 such that it has two mappings with float values: Transform -> Float and Float -> Float
				// InnerJoin should execute the predicate twice, once for each value type that matches the transformer's value type

				// Dummy float sanitize transformer
				struct FFloatAttributeTransformer_SanitizeTest
				{
					using TransformerType = Transformers::FSanitize;
					using ValueType = FFloatAnimationAttribute;

					static void TransformBoundValueMap(FBoundValueMap* SetMap) {}	// No-op
				};

				// Dummy vector sanitize transformer
				struct FVectorAttributeTransformer_SanitizeTest
				{
					using TransformerType = Transformers::FSanitize;
					using ValueType = FVectorAnimationAttribute;

					static void TransformBoundValueMap(FBoundValueMap* SetMap) {}	// No-op
				};

PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

				FValueTransformerList TransformerList(Transformers::FSanitize::TransformerName);
				TransformerList.AddBoundValueMapTransformer(FFloatAnimationAttribute::StaticStruct(), reinterpret_cast<FRawTransformerFunc>(&FFloatAttributeTransformer_SanitizeTest::TransformBoundValueMap));
				TransformerList.AddBoundValueMapTransformer(FVectorAnimationAttribute::StaticStruct(), reinterpret_cast<FRawTransformerFunc>(&FVectorAttributeTransformer_SanitizeTest::TransformBoundValueMap));

PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

				int32 NumPredicateCalls = 0;
				int32 NumNullCollectionEntries = 0;
				int32 NumNullTransformEntries = 0;
				OuterJoinBy(
					FValueTransformerListWithBoundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &NumNullCollectionEntries, &NumNullTransformEntries, &Collection0](FRawTransformerFunc Transformer, FBoundValueMap* SetMap)
					{
						NumPredicateCalls++;

						if (SetMap != nullptr)
						{
							AddErrorIfFalse(CollectionContains(Collection0, SetMap), TEXT("Unexpected OuterJoin iteration result"));

							if (Transformer != nullptr)
							{
								AddErrorIfFalse(SetMap->GetValueType() == FFloatAnimationAttribute::StaticStruct(), TEXT("Unexpected OuterJoin iteration result"));
							}
						}
						else
						{
							NumNullCollectionEntries++;
						}

						if (Transformer == nullptr)
						{
							NumNullTransformEntries++;
						}
					},
					TransformerList.CreateBoundValueMapTransformerIterator(),
					Collection0.CreateIterator());

				// [Transformer Value Type, Collection Set Map Value Type]
				// ---
				// [Float, Float->Float]
				// [Float, Bone->Float]
				// [Float, nullptr]
				// [nullptr, Float->Int]
				// [nullptr, Float->String]
				// [nullptr, Bone->Bone]
				// [Vector, nullptr]

				AddErrorIfFalse(NumPredicateCalls == 7, TEXT("Unexpected OuterJoin iteration result"));
				AddErrorIfFalse(NumNullCollectionEntries == 2, TEXT("Unexpected OuterJoin iteration result"));
				AddErrorIfFalse(NumNullTransformEntries == 3, TEXT("Unexpected OuterJoin iteration result"));
			}
		}

		// Unbound value map collection
		{
			FUnboundMapCollectionHeap Collection0;

			AddErrorIfFalse(Collection0.IsEmpty(), TEXT("Collection should be empty"));

			TUnboundValueMap<FFloatAnimationAttribute>* FloatMap0 = Collection0.Add<FFloatAnimationAttribute>();
			UE_RETURN_ON_ERROR(FloatMap0 != nullptr, TEXT("Float map should be valid"));

			TUnboundValueMap<FBoneTransformAnimationAttribute>* BoneMap0 = Collection0.Add<FBoneTransformAnimationAttribute>();
			UE_RETURN_ON_ERROR(BoneMap0 != nullptr, TEXT("Bone map should be valid"));

			AddErrorIfFalse(!Collection0.IsEmpty(), TEXT("Collection should not be empty"));
			AddErrorIfFalse(Collection0.Num() == 2, TEXT("Unexpected map count"));

			FUnboundMapCollectionHeap Collection1;

			AddErrorIfFalse(Collection1.IsEmpty(), TEXT("Collection should be empty"));

			TUnboundValueMap<FFloatAnimationAttribute>* FloatMap1 = Collection1.Add<FFloatAnimationAttribute>();
			UE_RETURN_ON_ERROR(FloatMap1 != nullptr, TEXT("Float map should be valid"));

			TUnboundValueMap<FBoneTransformAnimationAttribute>* BoneMap1 = Collection1.Add<FBoneTransformAnimationAttribute>();
			UE_RETURN_ON_ERROR(BoneMap1 != nullptr, TEXT("Bone map should be valid"));

			AddErrorIfFalse(!Collection1.IsEmpty(), TEXT("Collection should not be empty"));
			AddErrorIfFalse(Collection1.Num() == 2, TEXT("Unexpected map count"));

			// Both inputs perfectly match
			{
				int32 NumPredicateCalls = 0;
				OuterJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1](FUnboundValueMap* Map0, FUnboundValueMap* Map1)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected OuterJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
					},
					Collection0.CreateIterator(),
					Collection1.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected OuterJoin iteration result"));

				NumPredicateCalls = 0;
				OuterJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1](const FUnboundValueMap* Map0, const FUnboundValueMap* Map1)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected OuterJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
					},
					Collection0.CreateConstIterator(),
					Collection1.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected OuterJoin iteration result"));

				NumPredicateCalls = 0;
				OuterJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1](const FUnboundValueMap* Map0, FUnboundValueMap* Map1)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected OuterJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
					},
					Collection0.CreateConstIterator(),
					Collection1.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected OuterJoin iteration result"));

				NumPredicateCalls = 0;
				OuterJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &Collection0, &Collection1](FUnboundValueMap* Map0, const FUnboundValueMap* Map1)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));
						AddErrorIfFalse(CollectionContains(Collection1, Map1), TEXT("Unexpected OuterJoin iteration result"));
						AddErrorIfFalse(Map0 != nullptr && Map1 != nullptr && Map0->GetValueType() == Map1->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
					},
					Collection0.CreateIterator(),
					Collection1.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected OuterJoin iteration result"));
			}

			FUnboundMapCollectionHeap Collection2;

			AddErrorIfFalse(Collection2.IsEmpty(), TEXT("Collection should be empty"));

			TUnboundValueMap<FFloatAnimationAttribute>* FloatMap2 = Collection2.Add<FFloatAnimationAttribute>();
			UE_RETURN_ON_ERROR(FloatMap2 != nullptr, TEXT("Float map should be valid"));

			AddErrorIfFalse(!Collection2.IsEmpty(), TEXT("Collection should not be empty"));
			AddErrorIfFalse(Collection2.Num() == 1, TEXT("Unexpected map count"));

			// First input is shorter and will be padded with null entries
			{
				int32 NumPredicateCalls = 0;
				int32 NumNullEntries = 0;
				OuterJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &NumNullEntries, &Collection0, &Collection2](FUnboundValueMap* Map2, FUnboundValueMap* Map0)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));

						if (Map2 != nullptr)
						{
							AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
						}
						else
						{
							NumNullEntries++;
							AddErrorIfFalse(Map0 != nullptr, TEXT("Unexpected OuterJoin iteration result"));
						}
					},
					Collection2.CreateIterator(),
					Collection0.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected OuterJoin iteration result"));
				AddErrorIfFalse(NumNullEntries == 1, TEXT("Unexpected OuterJoin iteration result"));

				NumPredicateCalls = 0;
				NumNullEntries = 0;
				OuterJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &NumNullEntries, &Collection0, &Collection2](const FUnboundValueMap* Map2, const FUnboundValueMap* Map0)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));

						if (Map2 != nullptr)
						{
							AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
						}
						else
						{
							NumNullEntries++;
							AddErrorIfFalse(Map0 != nullptr, TEXT("Unexpected OuterJoin iteration result"));
						}
					},
					Collection2.CreateConstIterator(),
					Collection0.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected OuterJoin iteration result"));
				AddErrorIfFalse(NumNullEntries == 1, TEXT("Unexpected OuterJoin iteration result"));

				NumPredicateCalls = 0;
				NumNullEntries = 0;
				OuterJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &NumNullEntries, &Collection0, &Collection2](const FUnboundValueMap* Map2, FUnboundValueMap* Map0)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));

						if (Map2 != nullptr)
						{
							AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
						}
						else
						{
							NumNullEntries++;
							AddErrorIfFalse(Map0 != nullptr, TEXT("Unexpected OuterJoin iteration result"));
						}
					},
					Collection2.CreateConstIterator(),
					Collection0.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected OuterJoin iteration result"));
				AddErrorIfFalse(NumNullEntries == 1, TEXT("Unexpected OuterJoin iteration result"));

				NumPredicateCalls = 0;
				NumNullEntries = 0;
				OuterJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &NumNullEntries, &Collection0, &Collection2](FUnboundValueMap* Map2, const FUnboundValueMap* Map0)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));

						if (Map2 != nullptr)
						{
							AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
						}
						else
						{
							NumNullEntries++;
							AddErrorIfFalse(Map0 != nullptr, TEXT("Unexpected OuterJoin iteration result"));
						}
					},
					Collection2.CreateIterator(),
					Collection0.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected OuterJoin iteration result"));
				AddErrorIfFalse(NumNullEntries == 1, TEXT("Unexpected OuterJoin iteration result"));
			}

			// Second input is shorter and will be padded with null entries
			{
				int32 NumPredicateCalls = 0;
				int32 NumNullEntries = 0;
				OuterJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &NumNullEntries, &Collection0, &Collection2](FUnboundValueMap* Map0, FUnboundValueMap* Map2)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));

						if (Map2 != nullptr)
						{
							AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
						}
						else
						{
							NumNullEntries++;
							AddErrorIfFalse(Map0 != nullptr, TEXT("Unexpected OuterJoin iteration result"));
						}
					},
					Collection0.CreateIterator(),
					Collection2.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected OuterJoin iteration result"));
				AddErrorIfFalse(NumNullEntries == 1, TEXT("Unexpected OuterJoin iteration result"));

				NumPredicateCalls = 0;
				NumNullEntries = 0;
				OuterJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &NumNullEntries, &Collection0, &Collection2](const FUnboundValueMap* Map0, const FUnboundValueMap* Map2)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));

						if (Map2 != nullptr)
						{
							AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
						}
						else
						{
							NumNullEntries++;
							AddErrorIfFalse(Map0 != nullptr, TEXT("Unexpected OuterJoin iteration result"));
						}
					},
					Collection0.CreateConstIterator(),
					Collection2.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected OuterJoin iteration result"));
				AddErrorIfFalse(NumNullEntries == 1, TEXT("Unexpected OuterJoin iteration result"));

				NumPredicateCalls = 0;
				NumNullEntries = 0;
				OuterJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &NumNullEntries, &Collection0, &Collection2](const FUnboundValueMap* Map0, FUnboundValueMap* Map2)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));

						if (Map2 != nullptr)
						{
							AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
						}
						else
						{
							NumNullEntries++;
							AddErrorIfFalse(Map0 != nullptr, TEXT("Unexpected OuterJoin iteration result"));
						}
					},
					Collection0.CreateConstIterator(),
					Collection2.CreateIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected OuterJoin iteration result"));
				AddErrorIfFalse(NumNullEntries == 1, TEXT("Unexpected OuterJoin iteration result"));

				NumPredicateCalls = 0;
				NumNullEntries = 0;
				OuterJoinBy(
					FUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &NumNullEntries, &Collection0, &Collection2](FUnboundValueMap* Map0, const FUnboundValueMap* Map2)
					{
						NumPredicateCalls++;
						AddErrorIfFalse(CollectionContains(Collection0, Map0), TEXT("Unexpected OuterJoin iteration result"));

						if (Map2 != nullptr)
						{
							AddErrorIfFalse(CollectionContains(Collection2, Map2), TEXT("Unexpected OuterJoin iteration result"));
							AddErrorIfFalse(Map0 != nullptr && Map0->GetValueType() == Map2->GetValueType(), TEXT("Unexpected OuterJoin iteration result"));
						}
						else
						{
							NumNullEntries++;
							AddErrorIfFalse(Map0 != nullptr, TEXT("Unexpected OuterJoin iteration result"));
						}
					},
					Collection0.CreateIterator(),
					Collection2.CreateConstIterator());
				AddErrorIfFalse(NumPredicateCalls == 2, TEXT("Unexpected OuterJoin iteration result"));
				AddErrorIfFalse(NumNullEntries == 1, TEXT("Unexpected OuterJoin iteration result"));
			}

			// Join with a transformer list
			{
				// Use a float transformer
				// We set up Collection0 such that it has two mappings with these value types: Float and Bone Transform
				// InnerJoin should execute the predicate twice, once for each value type that matches the transformer's value type

				// Dummy float sanitize transformer
				struct FFloatAttributeTransformer_SanitizeTest
				{
					using TransformerType = Transformers::FSanitize;
					using ValueType = FFloatAnimationAttribute;

					static void TransformMap(FUnboundValueMap* Map) {}	// No-op
				};

				// Dummy vector sanitize transformer
				struct FVectorAttributeTransformer_SanitizeTest
				{
					using TransformerType = Transformers::FSanitize;
					using ValueType = FVectorAnimationAttribute;

					static void TransformMap(FUnboundValueMap* Map) {}	// No-op
				};

PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

				FValueTransformerList TransformerList(Transformers::FSanitize::TransformerName);
				TransformerList.AddUnboundValueMapTransformer(FFloatAnimationAttribute::StaticStruct(), reinterpret_cast<FRawTransformerFunc>(&FFloatAttributeTransformer_SanitizeTest::TransformMap));
				TransformerList.AddUnboundValueMapTransformer(FVectorAnimationAttribute::StaticStruct(), reinterpret_cast<FRawTransformerFunc>(&FVectorAttributeTransformer_SanitizeTest::TransformMap));

PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

				int32 NumPredicateCalls = 0;
				int32 NumNullCollectionEntries = 0;
				int32 NumNullTransformEntries = 0;
				OuterJoinBy(
					FValueTransformerListWithUnboundMapCollectionJoinOp(),
					[this, &NumPredicateCalls, &NumNullCollectionEntries, &NumNullTransformEntries, &Collection0](FRawTransformerFunc Transformer, FUnboundValueMap* Map)
					{
						NumPredicateCalls++;

						if (Map != nullptr)
						{
							AddErrorIfFalse(CollectionContains(Collection0, Map), TEXT("Unexpected OuterJoin iteration result"));

							if (Transformer != nullptr)
							{
								AddErrorIfFalse(Map->GetValueType() == FFloatAnimationAttribute::StaticStruct(), TEXT("Unexpected OuterJoin iteration result"));
							}
						}
						else
						{
							NumNullCollectionEntries++;
						}

						if (Transformer == nullptr)
						{
							NumNullTransformEntries++;
						}
					},
					TransformerList.CreateUnboundValueMapTransformerIterator(),
					Collection0.CreateIterator());

				// [Transformer Value Type, Collection Map Value Type]
				// ---
				// [Float, Float]
				// [nullptr, Bone]
				// [Vector, nullptr]

				AddErrorIfFalse(NumPredicateCalls == 3, TEXT("Unexpected OuterJoin iteration result"));
				AddErrorIfFalse(NumNullCollectionEntries == 1, TEXT("Unexpected OuterJoin iteration result"));
				AddErrorIfFalse(NumNullTransformEntries == 1, TEXT("Unexpected OuterJoin iteration result"));
			}
		}

		return true;
	}
}

#endif	// WITH_DEV_AUTOMATION_TESTS
