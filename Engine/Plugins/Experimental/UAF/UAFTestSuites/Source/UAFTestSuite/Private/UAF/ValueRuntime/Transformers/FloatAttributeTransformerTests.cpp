// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "AnimNextTest.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "UAF/Attributes/AttributeBindingDataCache.h"
#include "UAF/ValueRuntime/ValueRuntimeRegistry.h"
#include "UAF/ValueRuntime/Transformers/Accumulate.h"
#include "UAF/ValueRuntime/Transformers/AdditiveSpace.h"
#include "UAF/ValueRuntime/Transformers/Interpolate.h"
#include "UAF/ValueRuntime/Transformers/Overwrite.h"
#include "UAF/ValueRuntime/Transformers/TransformerTestUtil.h"
#include "UObject/Package.h"

namespace UE::UAF::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTransformerTests_FloatAttribute, "Animation.UAF.ValueRuntime.Transformers.FloatAttribute", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool FTransformerTests_FloatAttribute::RunTest(const FString& InParameters)
	{
		ON_SCOPE_EXIT{ FUtils::CleanupAfterTests(); };

		TValueOrError<TNonNullPtr<UAbstractSkeletonSetBinding>, FString> SetBindingResult = BuildSetBinding({});

		if (SetBindingResult.HasError())
		{
			AddError(SetBindingResult.GetError());
			return false;
		}
		
		TNonNullPtr<UAbstractSkeletonSetBinding> SetBinding = SetBindingResult.GetValue();

		SetBinding->AddAttributeToSet(FAnimationAttributeIdentifier(TEXT("AttrA"), INDEX_NONE, NAME_None, FFloatAnimationAttribute::StaticStruct()), "SetA");
		SetBinding->AddAttributeToSet(FAnimationAttributeIdentifier(TEXT("AttrB"), INDEX_NONE, NAME_None, FFloatAnimationAttribute::StaticStruct()), "SetB");
		SetBinding->AddAttributeToSet(FAnimationAttributeIdentifier(TEXT("AttrC"), INDEX_NONE, NAME_None, FFloatAnimationAttribute::StaticStruct()), "SetC");
		
		const FAttributeBindingDataPtr& BindingData = GAttributeBindingDataCache.GetOrAdd(SetBinding, nullptr);
		FAttributeNamedSetPtr SetA = BindingData->FindNamedSet("SetA");
		FAttributeNamedSetPtr SetB = BindingData->FindNamedSet("SetB");
		FAttributeNamedSetPtr SetC = BindingData->FindNamedSet("SetC");

		// Overwrite
		AddInfo("Overwrite");
		{
			// 0.25 weight
			{
				FTransformerUnaryTest<FFloatAnimationAttribute, false> FloatTest;
				FloatTest.AttributeName = "AttrA";
				FloatTest.Input.Value = 5.0f;
				FloatTest.ApplyTransformer = [](const FValueBundleHeap& InputValues, FValueBundleHeap& OutputValues)
					{
						const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();
						Transformers::FOverwrite::Apply(TransformerMap, InputValues, 0.25f, OutputValues);
					};
				FloatTest.IsExpectedResult = [&](TOptional<FFloatAnimationAttribute> Result) -> TOptional<FString>
					{
						if (!Result.IsSet())
						{
							return FString("Overwrite - 0.25 weight - expected to produce a value but received none");
						}

						constexpr float ExpectedValue = 0.25f * 5.0f;
						if (!FMath::IsNearlyEqual(Result->Value, ExpectedValue))
						{
							return FString::Format(TEXT("Overwrite - 0.25 weight - expected {0}, Received {1}"), { ExpectedValue, Result->Value });
						}

						return TOptional<FString>();
					};
			}
		}
		
		// Accumulate
		AddInfo("Accumulate");
		{
			// 0.25 weight
			{
				FTransformerBinaryTest<Transformers::FAccumulate, FFloatAnimationAttribute, EInplaceTransformationSupport::Allow> FloatTest;
				FloatTest.AttributeName = "AttrA";
				FloatTest.InputA.Value = 5.0f;
				FloatTest.InputB.Value = 8.0f;
				FloatTest.ApplyTransformer = [](const FValueBundleHeap& InputAValues, const FValueBundleHeap& InputBValues, FValueBundleHeap& OutputValues)
					{
						const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();
						Transformers::FAccumulate::Apply(TransformerMap, InputAValues, InputBValues, 0.25f, OutputValues);
					};
				FloatTest.IsExpectedResult = [&](TOptional<FFloatAnimationAttribute> Result) -> TOptional<FString>
					{
						if (!Result.IsSet())
						{
							return FString("Accumulate - 0.25 weight - expected to produce a value but received none");
						}
						constexpr float ExpectedValue = 5.0f + (0.25f * 8.0f);
						if (!FMath::IsNearlyEqual(Result->Value, ExpectedValue))
						{
							return FString::Format(TEXT("Accumulate - 0.25 weight - expected {0}, received {1}"), { ExpectedValue, Result->Value });
						}
						return TOptional<FString>();
					};

				FloatTest.Run(*this, SetA);
			}
		}

		// Interpolate
		AddInfo("Interpolate");
		{
			// 0.25 weight
			{
				FTransformerBinaryTest<Transformers::FInterpolate, FFloatAnimationAttribute, EInplaceTransformationSupport::Allow> FloatTest;
				FloatTest.AttributeName = "AttrA";
				FloatTest.InputA.Value = 5.0f;
				FloatTest.InputB.Value = 8.0f;
				FloatTest.ApplyTransformer = [](const FValueBundleHeap& InputAValues, const FValueBundleHeap& InputBValues, FValueBundleHeap& OutputValues)
					{
						const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();
						Transformers::FInterpolate::Apply(TransformerMap, InputAValues, InputBValues, 0.25f, OutputValues);
					};
				FloatTest.IsExpectedResult = [&](TOptional<FFloatAnimationAttribute> Result) -> TOptional<FString>
					{
						if (!Result.IsSet())
						{
							return FString("Interpolate - 0.25 weight -expected to produce a value but received none");
						}
						constexpr float ExpectedValue = (0.75f * 5.0f) + (0.25f * 8.0f);
						if (!FMath::IsNearlyEqual(Result->Value, ExpectedValue))
						{
							return FString::Format(TEXT("Interpolate - 0.25 weight - expected {0}, received {1}"), { ExpectedValue, Result->Value });
						}
						return TOptional<FString>();
					};

				FloatTest.Run(*this, SetA);
			}
		}

		// Layer
		AddInfo("Layer");
		{
			// Value in base but not in layer
			{
				// Weight 0.25
				{
					FTransformerBinaryTest<Transformers::FLayer, FFloatAnimationAttribute, EInplaceTransformationSupport::Allow> FloatTest;
					FloatTest.AttributeName = "AttrA";
					FloatTest.WeightB = 0.25f;
					FloatTest.InputA = FFloatAnimationAttribute({ 5.0f });
					FloatTest.InputB.Reset();
					FloatTest.IsExpectedResult = [&](TOptional<FFloatAnimationAttribute> Result) -> TOptional<FString>
						{
							if (!Result.IsSet())
							{
								return FString("Layer - 0.25 weight, case 1 - expected to produce a value but received none");
							}
							
							constexpr float ExpectedValue = 5.0f;
							if (!FMath::IsNearlyEqual(Result->Value, ExpectedValue))
							{
								return FString::Format(TEXT("Layer - 0.25 weight, case 1 - expected {0}, received {1}"), { ExpectedValue, Result->Value });
							}
							
							return TOptional<FString>();
						};

					FloatTest.Run(*this, SetA, SetB);
				}
			}

			// Value in layer but not in base
			{
				// Weight 0.25
				{
					FTransformerBinaryTest<Transformers::FLayer, FFloatAnimationAttribute, EInplaceTransformationSupport::Allow> FloatTest;
					FloatTest.AttributeName = "AttrC";
					FloatTest.WeightB = 0.25f;
					FloatTest.InputA.Reset();
					FloatTest.InputB = FFloatAnimationAttribute({ 8.0f });
					FloatTest.IsExpectedResult = [&](TOptional<FFloatAnimationAttribute> Result) -> TOptional<FString>
						{
							if (Result.IsSet())
							{
								return FString("Layer - 0.25 weight, case 2 - expected to not produce a value but received one");
							}
														
							return TOptional<FString>();
						};

					FloatTest.Run(*this, SetB, SetC);
				}
			}
			
			// Value in both base and layer
			{
				// Weight 0.25
				{
					FTransformerBinaryTest<Transformers::FLayer, FFloatAnimationAttribute, EInplaceTransformationSupport::Allow> FloatTest;
					FloatTest.AttributeName = "AttrB";
					FloatTest.WeightB = 0.25f;
					FloatTest.InputA = FFloatAnimationAttribute({ 5.0f });
					FloatTest.InputB = FFloatAnimationAttribute({ 8.0f });
					FloatTest.IsExpectedResult = [&](TOptional<FFloatAnimationAttribute> Result) -> TOptional<FString>
						{
							if (!Result.IsSet())
							{
								return FString("Layer - 0.25 weight, case 3 - expected to produce a value but received none");
							}
							
							constexpr float ExpectedValue = (0.75f * 5.0f) + (0.25F * 8.0f);
							if (!FMath::IsNearlyEqual(Result->Value, ExpectedValue))
							{
								return FString::Format(TEXT("Layer - 0.25 weight, case 3 - expected {0}, received {1}"), { ExpectedValue, Result->Value });
							}
							
							return TOptional<FString>();
						};

					FloatTest.Run(*this, SetA, SetB);
				}
			}
		}

		// Make Additive Space
		AddInfo("MakeAdditiveSpace");
		{
			// 0.25 weight
			{
				FTransformerBinaryTest<Transformers::FMakeAdditiveSpace, FFloatAnimationAttribute, EInplaceTransformationSupport::Disallow> FloatTest;
				FloatTest.AttributeName = "AttrA";
				FloatTest.InputA.Value = 5.0f;
				FloatTest.InputB.Value = 8.0f;
				FloatTest.ApplyTransformer = [](const FValueBundleHeap& InputAValues, const FValueBundleHeap& InputBValues, FValueBundleHeap& OutputValues)
					{
						const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();
						Transformers::FMakeAdditiveSpace::Apply(TransformerMap, InputAValues, InputBValues, OutputValues);
					};
				FloatTest.IsExpectedResult = [&](TOptional<FFloatAnimationAttribute> Result) -> TOptional<FString>
					{
						if (!Result.IsSet())
						{
							return FString("MakeAdditiveSpace - expected to produce a value but received none");
						}
						constexpr float ExpectedValue = 8.0f - 5.0f;
						if (!FMath::IsNearlyEqual(Result->Value, ExpectedValue))
						{
							return FString::Format(TEXT("MakeAdditiveSpace - expected {0}, received {1}"), { ExpectedValue, Result->Value });
						}
						return TOptional<FString>();
					};

				FloatTest.Run(*this, SetA);
			}
		}

		// Apply Additive Space
		AddInfo("ApplyAdditiveSpace");
		{
			// 0.25 weight
			{
				FTransformerBinaryTest<Transformers::FMakeAdditiveSpace, FFloatAnimationAttribute, EInplaceTransformationSupport::Allow> FloatTest;
				FloatTest.AttributeName = "AttrA";
				FloatTest.InputA.Value = 5.0f;
				FloatTest.InputB.Value = 8.0f;
				FloatTest.ApplyTransformer = [](const FValueBundleHeap& InputAValues, const FValueBundleHeap& InputBValues, FValueBundleHeap& OutputValues)
					{
						const_cast<FValueBundleHeap&>(InputBValues).SetValueSpace(FValueSpace(EValueSpaceType::Local, true));
						
						const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();
						Transformers::FApplyAdditiveSpace::Apply(TransformerMap, InputAValues, InputBValues, 0.25f, OutputValues);
					};
				FloatTest.IsExpectedResult = [&](TOptional<FFloatAnimationAttribute> Result) -> TOptional<FString>
					{
						if (!Result.IsSet())
						{
							return FString("ApplyAdditiveSpace - 0.25 weight - expected to produce a value but received none");
						}
						constexpr float ExpectedValue = 5.0f + (0.25f * 8.0f);
						if (!FMath::IsNearlyEqual(Result->Value, ExpectedValue))
						{
							return FString::Format(TEXT("ApplyAdditiveSpace - 0.25 weight - expected {0}, received {1}"), { ExpectedValue, Result->Value });
						}
						return TOptional<FString>();
					};

				FloatTest.Run(*this, SetA);
			}
		}
		
		// Sanitize
		// Nothing to do
		
		return true;
	}
}

#endif // WITH_DEV_AUTOMATION_TESTS