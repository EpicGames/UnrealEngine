// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "AnimNextTest.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "UAF/Attributes/AttributeBindingDataCache.h"
#include "UAF/ValueRuntime/ValueRuntimeRegistry.h"
#include "UAF/ValueRuntime/Transformers/Accumulate.h"
#include "UAF/ValueRuntime/Transformers/AdditiveSpace.h"
#include "UAF/ValueRuntime/Transformers/Interpolate.h"
#include "UAF/ValueRuntime/Transformers/Overwrite.h"
#include "UAF/ValueRuntime/Transformers/Sanitize.h"
#include "UAF/ValueRuntime/Transformers/TransformerTestUtil.h"
#include "UObject/Package.h"

namespace UE::UAF::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTransformerTests_QuaternionAttribute, "Animation.UAF.ValueRuntime.Transformers.QuaternionAttribute", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool FTransformerTests_QuaternionAttribute::RunTest(const FString& InParameters)
	{
		ON_SCOPE_EXIT{ FUtils::CleanupAfterTests(); };

		TValueOrError<TNonNullPtr<UAbstractSkeletonSetBinding>, FString> SetBindingResult = BuildSetBinding({});

		if (SetBindingResult.HasError())
		{
			AddError(SetBindingResult.GetError());
			return false;
		}
		
		TNonNullPtr<UAbstractSkeletonSetBinding> SetBinding = SetBindingResult.GetValue();

		SetBinding->AddAttributeToSet(FAnimationAttributeIdentifier(TEXT("AttrA"), INDEX_NONE, NAME_None, FQuaternionAnimationAttribute::StaticStruct()), "SetA");
		SetBinding->AddAttributeToSet(FAnimationAttributeIdentifier(TEXT("AttrB"), INDEX_NONE, NAME_None, FQuaternionAnimationAttribute::StaticStruct()), "SetB");
		SetBinding->AddAttributeToSet(FAnimationAttributeIdentifier(TEXT("AttrC"), INDEX_NONE, NAME_None, FQuaternionAnimationAttribute::StaticStruct()), "SetC");

		const FAttributeBindingDataPtr& BindingData = GAttributeBindingDataCache.GetOrAdd(SetBinding, nullptr);
		FAttributeNamedSetPtr SetA = BindingData->FindNamedSet("SetA");
		FAttributeNamedSetPtr SetB = BindingData->FindNamedSet("SetB");
		FAttributeNamedSetPtr SetC = BindingData->FindNamedSet("SetC");

		const FQuat QuatValueA = FQuat(0.0f, 1.0f, 2.0f, 0.0f);
		const FQuat QuatValueB = FQuat(0.0f, 1.0f, 0.0f, 1.0f);

		// Overwrite
		AddInfo("Overwrite");
		{
			// 0.25 weight
			{
				FTransformerUnaryTest<FQuaternionAnimationAttribute, false> QuatTest;
				QuatTest.AttributeName = "AttrA";
				QuatTest.Input.Value = QuatValueA;
				QuatTest.ApplyTransformer = [](const FValueBundleHeap& InputValues, FValueBundleHeap& OutputValues)
					{
						const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();
						Transformers::FOverwrite::Apply(TransformerMap, InputValues, 0.25f, OutputValues);
					};
				QuatTest.IsExpectedResult = [&](TOptional<FQuaternionAnimationAttribute> Result) -> TOptional<FString>
					{
						if (!Result.IsSet())
						{
							return FString("Overwrite - 0.25 weight - expected to produce a value but received none");
						}

						const FQuat ExpectedValue(0.0f, 0.25f, 0.5f, 0.0f);
						if (!ExpectedValue.Equals(Result->Value))
						{
							return FString::Printf(TEXT("Overwrite - 0.25 weight - expected %s, Received %s"), *ExpectedValue.ToString(), *Result->Value.ToString());
						}

						return TOptional<FString>();
					};

				QuatTest.Run(*this, SetA);
			}
		}

		// Accumulate
		AddInfo("Accumulate");
		{
			// 0.25 weight
			{
				FTransformerBinaryTest<Transformers::FAccumulate, FQuaternionAnimationAttribute, EInplaceTransformationSupport::Allow> QuatTest;
				QuatTest.AttributeName = "AttrA";
				QuatTest.InputA.Value = QuatValueA;
				QuatTest.InputB.Value = QuatValueB;
				QuatTest.ApplyTransformer = [](const FValueBundleHeap& InputAValues, const FValueBundleHeap& InputBValues, FValueBundleHeap& OutputValues)
					{
						const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();
						Transformers::FAccumulate::Apply(TransformerMap, InputAValues, InputBValues, 0.25f, OutputValues);
					};
				QuatTest.IsExpectedResult = [&](TOptional<FQuaternionAnimationAttribute> Result) -> TOptional<FString>
					{
						if (!Result.IsSet())
						{
							return FString("Accumulate - 0.25 weight - expected to produce a value but received none");
						}
						const FQuat ExpectedValue(0.0f, 1.25f, 2.0f, 0.25f);
						if (!ExpectedValue.Equals(Result->Value))
						{
							return FString::Printf(TEXT("Accumulate - 0.25 weight - expected %s, received %s"), *ExpectedValue.ToString(), *Result->Value.ToString());
						}
						return TOptional<FString>();
					};

				QuatTest.Run(*this, SetA);
			}
		}

		// Interpolate
		AddInfo("Interpolate");
		{
			// 0.25 weight
			{
				FTransformerBinaryTest<Transformers::FInterpolate, FQuaternionAnimationAttribute, EInplaceTransformationSupport::Allow> QuatTest;
				QuatTest.AttributeName = "AttrA";
				QuatTest.InputA.Value = QuatValueA;
				QuatTest.InputB.Value = QuatValueB;
				QuatTest.ApplyTransformer = [](const FValueBundleHeap& InputAValues, const FValueBundleHeap& InputBValues, FValueBundleHeap& OutputValues)
					{
						const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();
						Transformers::FInterpolate::Apply(TransformerMap, InputAValues, InputBValues, 0.25f, OutputValues);
					};
				QuatTest.IsExpectedResult = [&](TOptional<FQuaternionAnimationAttribute> Result) -> TOptional<FString>
					{
						if (!Result.IsSet())
						{
							return FString("Interpolate - 0.25 weight - expected to produce a value but received none");
						}
						const FQuat ExpectedValue = FQuat::FastLerp(QuatValueA, QuatValueB, 0.25f);
						if (!ExpectedValue.Equals(Result->Value))
						{
							return FString::Printf(TEXT("Interpolate - 0.25 weight - expected %s, received %s"), *ExpectedValue.ToString(), *Result->Value.ToString());
						}
						return TOptional<FString>();
					};

				QuatTest.Run(*this, SetA);
			}
		}

		// Layer
		AddInfo("Layer");
		{
			// Value in layer but not in base
			{
				// Weight 0.25
				{
					FTransformerBinaryTest<Transformers::FLayer, FQuaternionAnimationAttribute, EInplaceTransformationSupport::Allow> QuatTest;
					QuatTest.AttributeName = "AttrC";
					QuatTest.WeightB = 0.25f;
					QuatTest.InputA.Reset();
					QuatTest.InputB = FQuaternionAnimationAttribute({ QuatValueB });
					QuatTest.IsExpectedResult = [&](TOptional<FQuaternionAnimationAttribute> Result) -> TOptional<FString>
						{
							if (Result.IsSet())
							{
								return FString("Layer - 0.25 weight, case 2 - expected to not produce a value but received one");
							}

							return TOptional<FString>();
						};

					QuatTest.Run(*this, SetB, SetC);
				}
			}

			// Value in both base and layer
			{
				// Weight 0.25
				{
					FTransformerBinaryTest<Transformers::FLayer, FQuaternionAnimationAttribute, EInplaceTransformationSupport::Allow> QuatTest;
					QuatTest.AttributeName = "AttrB";
					QuatTest.WeightB = 0.25f;
					QuatTest.InputA = FQuaternionAnimationAttribute({ QuatValueA });
					QuatTest.InputB = FQuaternionAnimationAttribute({ QuatValueB });
					QuatTest.IsExpectedResult = [&](TOptional<FQuaternionAnimationAttribute> Result) -> TOptional<FString>
						{
							if (!Result.IsSet())
							{
								return FString("Layer - 0.25 weight, case 3 - expected to produce a value but received none");
							}

							// = (1-0.25)*FQuat(0,0,0,1) + 0.25*FQuat(0,0,1,0) = FQuat(0,0,0.25,0.75)
							const FQuat ExpectedValue = FQuat::FastLerp(QuatValueA, QuatValueB, 0.25f);
							if (!ExpectedValue.Equals(Result->Value))
							{
								return FString::Printf(TEXT("Layer - 0.25 weight, case 3 - expected %s, received %s"), *ExpectedValue.ToString(), *Result->Value.ToString());
							}

							return TOptional<FString>();
						};

					QuatTest.Run(*this, SetA, SetB);
				}
			}
		}

		// Make Additive Space
		AddInfo("MakeAdditiveSpace");
		{
			{
				FTransformerBinaryTest<Transformers::FMakeAdditiveSpace, FQuaternionAnimationAttribute, EInplaceTransformationSupport::Disallow> QuatTest;
				QuatTest.AttributeName = "AttrA";
				QuatTest.InputA.Value = QuatValueA;
				QuatTest.InputB.Value = QuatValueB;
				QuatTest.ApplyTransformer = [](const FValueBundleHeap& InputAValues, const FValueBundleHeap& InputBValues, FValueBundleHeap& OutputValues)
					{
						const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();
						Transformers::FMakeAdditiveSpace::Apply(TransformerMap, InputAValues, InputBValues, OutputValues);
					};
				QuatTest.IsExpectedResult = [&](TOptional<FQuaternionAnimationAttribute> Result) -> TOptional<FString>
					{
						if (!Result.IsSet())
						{
							return FString("MakeAdditiveSpace - expected to produce a value but received none");
						}
						const FQuat ExpectedValue = (QuatValueB * QuatValueA.Inverse()).GetNormalized();
						if (!ExpectedValue.Equals(Result->Value))
						{
							return FString::Printf(TEXT("MakeAdditiveSpace - expected %s, received %s"), *ExpectedValue.ToString(), *Result->Value.ToString());
						}
						return TOptional<FString>();
					};

				QuatTest.Run(*this, SetA);
			}
		}

		// Apply Additive Space
		AddInfo("ApplyAdditiveSpace");
		{
			// 0.25 weight
			{
				FTransformerBinaryTest<Transformers::FMakeAdditiveSpace, FQuaternionAnimationAttribute, EInplaceTransformationSupport::Allow> QuatTest;
				QuatTest.AttributeName = "AttrA";
				QuatTest.InputA.Value = QuatValueA;
				QuatTest.InputB.Value = QuatValueB;
				QuatTest.ApplyTransformer = [](const FValueBundleHeap& InputAValues, const FValueBundleHeap& InputBValues, FValueBundleHeap& OutputValues)
					{
						const_cast<FValueBundleHeap&>(InputBValues).SetValueSpace(FValueSpace(EValueSpaceType::Local, true));

						const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();
						Transformers::FApplyAdditiveSpace::Apply(TransformerMap, InputAValues, InputBValues, 0.25f, OutputValues);
					};
				QuatTest.IsExpectedResult = [QuatValueA, QuatValueB](TOptional<FQuaternionAnimationAttribute> Result) -> TOptional<FString>
					{
						if (!Result.IsSet())
						{
							return FString("ApplyAdditiveSpace - 0.25 weight - expected to produce a value but received none");
						}
						const FQuat WeightedRotation = FQuat::FastLerp(FQuat::Identity, QuatValueB, 0.25f).GetNormalized();
						const FQuat ExpectedValue = QuatValueA + WeightedRotation;
						if (!ExpectedValue.Equals(Result->Value))
						{
							return FString::Printf(TEXT("ApplyAdditiveSpace - 0.25 weight - expected %s, received %s"), *ExpectedValue.ToString(), *Result->Value.ToString());
						}
						return TOptional<FString>();
					};

				QuatTest.Run(*this, SetA);
			}
		}

		// Sanitize
		AddInfo("Sanitize");
		{
			FTransformerUnaryTest<FQuaternionAnimationAttribute, true> BoneTransformTest;
			BoneTransformTest.AttributeName = "AttrA";
			BoneTransformTest.Input.Value = FQuat(0.2f, 0.5f, 0.3f, 0.1f);
			BoneTransformTest.ApplyTransformer = [](FValueBundleHeap& InputValues)
				{
					const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();
					Transformers::FSanitize::Apply(TransformerMap, InputValues);
				};
			BoneTransformTest.IsExpectedResult = [&](TOptional<FQuaternionAnimationAttribute> Result) -> TOptional<FString>
				{
					if (!Result.IsSet())
					{
						return FString("Sanitize - expected to produce a value but received none");
					}
					const FQuat ExpectedValue = FQuat(0.2f, 0.5f, 0.3f, 0.1f).GetNormalized();
					if (!ExpectedValue.Equals(Result->Value))
					{
						return FString::Format(TEXT("Sanitize - expected {0}, Received {1}"), { ExpectedValue.ToString(), Result->Value.ToString() });
					}

					return TOptional<FString>();
				};

			BoneTransformTest.Run(*this, SetA);
		}

		return true;
	}
}

#endif // WITH_DEV_AUTOMATION_TESTS
