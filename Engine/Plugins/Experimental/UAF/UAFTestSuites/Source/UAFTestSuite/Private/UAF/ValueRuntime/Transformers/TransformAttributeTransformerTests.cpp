// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "AnimNextTest.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "UAF/Attributes/AttributeBindingDataCache.h"
#include "UAF/ValueRuntime/ValueRuntimeRegistry.h"
#include "UAF/ValueRuntime/Transformers/Accumulate.h"
#include "UAF/ValueRuntime/Transformers/AdditiveSpace.h"
#include "UAF/ValueRuntime/Transformers/Interpolate.h"
#include "UAF/ValueRuntime/Transformers/Overwrite.h"
#include "UAF/ValueRuntime/Transformers/Sanitize.h"
#include "UAF/ValueRuntime/Transformers/TransformerTestUtil.h"
#include "UObject/Package.h"

namespace UE::UAF::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTransformerTests_TransformAttribute, "Animation.UAF.ValueRuntime.Transformers.TransformAttribute", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool FTransformerTests_TransformAttribute::RunTest(const FString& InParameters)
	{
		ON_SCOPE_EXIT{ FUtils::CleanupAfterTests(); };

		TValueOrError<TNonNullPtr<UAbstractSkeletonSetBinding>, FString> SetBindingResult = BuildSetBinding({});

		if (SetBindingResult.HasError())
		{
			AddError(SetBindingResult.GetError());
			return false;
		}
		
		TNonNullPtr<UAbstractSkeletonSetBinding> SetBinding = SetBindingResult.GetValue();

		SetBinding->AddAttributeToSet(FAnimationAttributeIdentifier(TEXT("AttrA"), INDEX_NONE, NAME_None, FTransformAnimationAttribute::StaticStruct()), "SetA");
		SetBinding->AddAttributeToSet(FAnimationAttributeIdentifier(TEXT("AttrB"), INDEX_NONE, NAME_None, FTransformAnimationAttribute::StaticStruct()), "SetB");
		SetBinding->AddAttributeToSet(FAnimationAttributeIdentifier(TEXT("AttrC"), INDEX_NONE, NAME_None, FTransformAnimationAttribute::StaticStruct()), "SetC");

		const FAttributeBindingDataPtr& BindingData = GAttributeBindingDataCache.GetOrAdd(SetBinding, nullptr);
		FAttributeNamedSetPtr SetA = BindingData->FindNamedSet("SetA");
		FAttributeNamedSetPtr SetB = BindingData->FindNamedSet("SetB");
		FAttributeNamedSetPtr SetC = BindingData->FindNamedSet("SetC");

		const FTransform TransformValueA(FQuat::Identity, FVector(100.0f, 200.0f, 300.0f), FVector::OneVector);
		const FTransform TransformValueB(FQuat(0.0f, 1.0f, 0.0f, 0.0f), FVector(400.0f, 500.0f, 600.0f), FVector(2.0f, 2.0f, 2.0f));

		// Overwrite
		AddInfo("Overwrite");
		{
			// 0.25 weight
			{
				FTransformerUnaryTest<FTransformAnimationAttribute, false> TransformTest;
				TransformTest.AttributeName = "AttrA";
				TransformTest.Input.Value = TransformValueA;
				TransformTest.ApplyTransformer = [](const FValueBundleHeap& InputValues, FValueBundleHeap& OutputValues)
					{
						const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();
						Transformers::FOverwrite::Apply(TransformerMap, InputValues, 0.25f, OutputValues);
					};
				TransformTest.IsExpectedResult = [&](TOptional<FTransformAnimationAttribute> Result) -> TOptional<FString>
					{
						if (!Result.IsSet())
						{
							return FString("Overwrite - 0.25 weight - expected to produce a value but received none");
						}
						
						const FTransform ExpectedValue = TransformValueA * ScalarRegister(0.25f);
						if (!ExpectedValue.Equals(Result->Value))
						{
							return FString::Printf(TEXT("Overwrite - 0.25 weight - expected %s, Received %s"), *ExpectedValue.ToHumanReadableString(), *Result->Value.ToHumanReadableString());
						}

						return TOptional<FString>();
					};

				TransformTest.Run(*this, SetA);
			}
		}

		// Accumulate
		AddInfo("Accumulate");
		{
			// 0.25 weight
			{
				FTransformerBinaryTest<Transformers::FAccumulate, FTransformAnimationAttribute, EInplaceTransformationSupport::Allow> TransformTest;
				TransformTest.AttributeName = "AttrA";
				TransformTest.InputA.Value = TransformValueA;
				TransformTest.InputB.Value = TransformValueB;
				TransformTest.ApplyTransformer = [](const FValueBundleHeap& InputAValues, const FValueBundleHeap& InputBValues, FValueBundleHeap& OutputValues)
					{
						const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();
						Transformers::FAccumulate::Apply(TransformerMap, InputAValues, InputBValues, 0.25f, OutputValues);
					};
				TransformTest.IsExpectedResult = [&](TOptional<FTransformAnimationAttribute> Result) -> TOptional<FString>
					{
						if (!Result.IsSet())
						{
							return FString("Accumulate - 0.25 weight - expected to produce a value but received none");
						}
						
						const FTransform ExpectedValue = TransformValueA + (TransformValueB * ScalarRegister(0.25f));
						if (!ExpectedValue.Equals(Result->Value))
						{
							return FString::Printf(TEXT("Accumulate - 0.25 weight - expected %s, received %s"), *ExpectedValue.ToHumanReadableString(), *Result->Value.ToHumanReadableString());
						}
						
						return TOptional<FString>();
					};

				TransformTest.Run(*this, SetA);
			}
		}

		// Interpolate
		AddInfo("Interpolate");
		{
			// 0.25 weight
			{
				FTransformerBinaryTest<Transformers::FInterpolate, FTransformAnimationAttribute, EInplaceTransformationSupport::Allow> TransformTest;
				TransformTest.AttributeName = "AttrA";
				TransformTest.InputA.Value = TransformValueA;
				TransformTest.InputB.Value = TransformValueB;
				TransformTest.ApplyTransformer = [](const FValueBundleHeap& InputAValues, const FValueBundleHeap& InputBValues, FValueBundleHeap& OutputValues)
					{
						const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();
						Transformers::FInterpolate::Apply(TransformerMap, InputAValues, InputBValues, 0.25f, OutputValues);
					};
				TransformTest.IsExpectedResult = [&](TOptional<FTransformAnimationAttribute> Result) -> TOptional<FString>
					{
						if (!Result.IsSet())
						{
							return FString("Interpolate - 0.25 weight - expected to produce a value but received none");
						}
						
						FTransform ExpectedValue = TransformValueA;
						ExpectedValue.BlendWith(TransformValueB, 0.25f);
						if (!ExpectedValue.Equals(Result->Value))
						{
							return FString::Printf(TEXT("Interpolate - 0.25 weight - expected %s, received %s"), *ExpectedValue.ToHumanReadableString(), *Result->Value.ToHumanReadableString());
						}
						
						return TOptional<FString>();
					};

				TransformTest.Run(*this, SetA);
			}
		}

		// Layer
		AddInfo("Layer");
		{
			// Value in layer but not in base
			{
				// Weight 0.25
				{
					FTransformerBinaryTest<Transformers::FLayer, FTransformAnimationAttribute, EInplaceTransformationSupport::Allow> TransformTest;
					TransformTest.AttributeName = "AttrC";
					TransformTest.WeightB = 0.25f;
					TransformTest.InputA.Reset();
					TransformTest.InputB = FTransformAnimationAttribute({ TransformValueB });
					TransformTest.IsExpectedResult = [&](TOptional<FTransformAnimationAttribute> Result) -> TOptional<FString>
						{
							if (Result.IsSet())
							{
								return FString("Layer - 0.25 weight, case 2 - expected to not produce a value but received one");
							}

							return TOptional<FString>();
						};

					TransformTest.Run(*this, SetB, SetC);
				}
			}

			// Value in both base and layer
			{
				// Weight 0.25
				{
					FTransformerBinaryTest<Transformers::FLayer, FTransformAnimationAttribute, EInplaceTransformationSupport::Allow> TransformTest;
					TransformTest.AttributeName = "AttrB";
					TransformTest.WeightB = 0.25f;
					TransformTest.InputA = FTransformAnimationAttribute({ TransformValueA });
					TransformTest.InputB = FTransformAnimationAttribute({ TransformValueB });
					TransformTest.IsExpectedResult = [&](TOptional<FTransformAnimationAttribute> Result) -> TOptional<FString>
						{
							if (!Result.IsSet())
							{
								return FString("Layer - 0.25 weight, case 3 - expected to produce a value but received none");
							}
							
							FTransform ExpectedValue = TransformValueA;
							ExpectedValue.BlendWith(TransformValueB, 0.25f);
							if (!ExpectedValue.Equals(Result->Value))
							{
								return FString::Printf(TEXT("Layer - 0.25 weight, case 3 - expected %s, received %s"), *ExpectedValue.ToHumanReadableString(), *Result->Value.ToHumanReadableString());
							}

							return TOptional<FString>();
						};

					TransformTest.Run(*this, SetA, SetB);
				}
			}
		}

		// Make Additive Space
		AddInfo("MakeAdditiveSpace");
		{
			{
				FTransformerBinaryTest<Transformers::FMakeAdditiveSpace, FTransformAnimationAttribute, EInplaceTransformationSupport::Disallow> TransformTest;
				TransformTest.AttributeName = "AttrA";
				TransformTest.InputA.Value = TransformValueA;
				TransformTest.InputB.Value = TransformValueB;
				TransformTest.ApplyTransformer = [](const FValueBundleHeap& InputAValues, const FValueBundleHeap& InputBValues, FValueBundleHeap& OutputValues)
					{
						const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();
						Transformers::FMakeAdditiveSpace::Apply(TransformerMap, InputAValues, InputBValues, OutputValues);
					};
				TransformTest.IsExpectedResult = [&](TOptional<FTransformAnimationAttribute> Result) -> TOptional<FString>
					{
						if (!Result.IsSet())
						{
							return FString("MakeAdditiveSpace - expected to produce a value but received none");
						}
						
						FTransform ExpectedValue = TransformValueB;
						FAnimationRuntime::ConvertTransformToAdditive(ExpectedValue, TransformValueA);
						if (!ExpectedValue.Equals(Result->Value))
						{
							return FString::Printf(TEXT("MakeAdditiveSpace - expected %s, received %s"), *ExpectedValue.ToHumanReadableString(), *Result->Value.ToHumanReadableString());
						}
						
						return TOptional<FString>();
					};

				TransformTest.Run(*this, SetA);
			}
		}

		// Apply Additive Space
		AddInfo("ApplyAdditiveSpace");
		{
			// 0.25 weight
			{
				FTransformerBinaryTest<Transformers::FMakeAdditiveSpace, FTransformAnimationAttribute, EInplaceTransformationSupport::Allow> TransformTest;
				TransformTest.AttributeName = "AttrA";
				TransformTest.InputA.Value = TransformValueA;
				TransformTest.InputB.Value = TransformValueB;
				TransformTest.ApplyTransformer = [](const FValueBundleHeap& InputAValues, const FValueBundleHeap& InputBValues, FValueBundleHeap& OutputValues)
					{
						const_cast<FValueBundleHeap&>(InputBValues).SetValueSpace(FValueSpace(EValueSpaceType::Local, true));

						const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();
						Transformers::FApplyAdditiveSpace::Apply(TransformerMap, InputAValues, InputBValues, 0.25f, OutputValues);
					};
				TransformTest.IsExpectedResult = [&](TOptional<FTransformAnimationAttribute> Result) -> TOptional<FString>
					{
						if (!Result.IsSet())
						{
							return FString("ApplyAdditiveSpace - 0.25 weight - expected to produce a value but received none");
						}
						
						FTransform ExpectedValue = TransformValueA;
						FTransform::BlendFromIdentityAndAccumulate(ExpectedValue, TransformValueB, ScalarRegister(0.25f));
						if (!ExpectedValue.Equals(Result->Value))
						{
							return FString::Printf(TEXT("ApplyAdditiveSpace - 0.25 weight - expected %s, received %s"), *ExpectedValue.ToHumanReadableString(), *Result->Value.ToHumanReadableString());
						}
						
						return TOptional<FString>();
					};

				TransformTest.Run(*this, SetA);
			}
		}

		// Sanitize
		AddInfo("Sanitize");
		{
			FTransformerUnaryTest<FTransformAnimationAttribute, true> TransformTest;
			TransformTest.AttributeName = "AttrA";
			TransformTest.Input.Value = FTransform(FQuat(0.2f, 0.5f, 0.3f, 0.1f), FVector(100.0f, 50.0f, 25.0f), FVector(1.0f));
			TransformTest.ApplyTransformer = [](FValueBundleHeap& InputValues)
				{
					const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();
					Transformers::FSanitize::Apply(TransformerMap, InputValues);
				};
			TransformTest.IsExpectedResult = [&](TOptional<FTransformAnimationAttribute> Result) -> TOptional<FString>
				{
					if (!Result.IsSet())
					{
						return FString("Sanitize - expected to produce a value but received none");
					}

					FTransform ExpectedValue = FTransform(FQuat(0.2f, 0.5f, 0.3f, 0.1f), FVector(100.0f, 50.0f, 25.0f), FVector(1.0f));
					ExpectedValue.NormalizeRotation();
					if (!ExpectedValue.Equals(Result->Value))
					{
						return FString::Format(TEXT("Sanitize - expected {0}, Received {1}"), { ExpectedValue.ToString(), Result->Value.ToString() });
					}

					return TOptional<FString>();
				};

			TransformTest.Run(*this, SetA);
		}

		return true;
	}
}

#endif // WITH_DEV_AUTOMATION_TESTS
