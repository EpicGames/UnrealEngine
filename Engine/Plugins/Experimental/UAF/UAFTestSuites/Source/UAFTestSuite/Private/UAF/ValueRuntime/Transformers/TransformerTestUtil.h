// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "Misc/AutomationTest.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetCollection.h"
#include "UAF/AbstractSkeleton/SetBindingFactory.h"
#include "UAF/AbstractSkeleton/SetCollectionFactory.h"
#include "UAF/ValueRuntime/ValueRuntimeRegistry.h"
#include "UAF/ValueRuntime/Transformers/Layer.h"
#include "UObject/Package.h"

namespace UE::UAF::Tests
{
	// Creates a new set binding for testing
	// Optionally can provide a function to operate on the skeleton to apply any modifications if needed
	// SetBinding will be created with the following set hierarchy:
	// (Everything Set)
	// \_ SetA
	//    \_ SetB
	//    \_ SetC
	inline TValueOrError<TNonNullPtr<UAbstractSkeletonSetBinding>, FString> BuildSetBinding(TFunction<void(TNonNullPtr<USkeleton>)> SkeletonModifier)
	{
		UFactory* BindingFactory = NewObject<UFactory>(GetTransientPackage(), UAbstractSkeletonSetBindingFactory::StaticClass());
		if (!BindingFactory)
		{
			return MakeError("Failed to create factory");
		}

		UAbstractSkeletonSetBinding* SetBinding = CastChecked<UAbstractSkeletonSetBinding>(BindingFactory->FactoryCreateNew(UAbstractSkeletonSetBinding::StaticClass(), GetTransientPackage(), TEXT("TestBindingAsset"), RF_Transient, nullptr, nullptr, NAME_None));
		if (!SetBinding)
		{
			return MakeError("Failed to create binding asset");
		}

		UFactory* SetCollectionFactory = NewObject<UFactory>(GetTransientPackage(), UAbstractSkeletonSetCollectionFactory::StaticClass());
		if (!SetCollectionFactory)
		{
			return MakeError("Failed to create factory");
		}

		UAbstractSkeletonSetCollection* SetCollection = CastChecked<UAbstractSkeletonSetCollection>(SetCollectionFactory->FactoryCreateNew(UAbstractSkeletonSetCollection::StaticClass(), GetTransientPackage(), TEXT("TestSetCollectionAsset"), RF_Transient, nullptr, nullptr, NAME_None));
		if (!SetCollection)
		{
			return MakeError("Failed to create set collection asset");
		}

		SetCollection->AddSet("SetA");
		SetCollection->AddSet("SetB", "SetA");
		SetCollection->AddSet("SetC", "SetA");
		
		bool bSuccess = SetBinding->SetSetCollection(SetCollection);
		if (!bSuccess)
		{
			return MakeError("Failed to set set collection on our binding");
		}

		USkeleton* Skeleton = NewObject<USkeleton>(GetTransientPackage(), USkeleton::StaticClass(), TEXT("TestSkeletonAsset"), RF_Transient);
		if (!Skeleton)
		{
			return MakeError("Failed to create skeleton asset");
		}

		if (SkeletonModifier.IsSet())
		{
			SkeletonModifier(Skeleton);
		}
		
		bSuccess = SetBinding->SetSkeleton(Skeleton);
		if (!bSuccess)
		{
			return MakeError("Failed to set skeleton on our binding");
		}

		return MakeValue(SetBinding);
	}
	
	// Util struct to perform a transformer on an attribute of a particular value and to validate its result.
	// FTransformUnaryTests is for transformers taking a single input.
	// The same operation is performed on a bound value and unbound value.
	// When calling Run we assume there is a valid attribute defined in the bound maps for the test to use.
	// The SingleBuffer template argument determines whether the transformer only takes a single value bundle
	// as input (which only supports in-place operations) or whether the transformer can output to a second
	// bundle (which supports in-place and out-of-place operations)
	template <typename AttributeType, bool SingleBuffer>
	struct FTransformerUnaryTest
	{
		FName AttributeName;
		AttributeType Input;

		using ApplyTransformerFunctionType = std::conditional_t<SingleBuffer,
		TFunction<void(FValueBundleHeap& InputOutputValues)>,
		TFunction<void(const FValueBundleHeap& InputValues, FValueBundleHeap& OutputValues)>>;
		
		ApplyTransformerFunctionType ApplyTransformer;
		TFunction<TOptional<FString>(TOptional<AttributeType> Result)> IsExpectedResult;

		void Run(FAutomationTestBase& Test, const FAttributeNamedSetPtr& NamedSet)
		{
			FValueBundleHeap InputValueBundle(NamedSet);

			// Setup bound attributes
			const FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo<AttributeType>();
			const FAttributeTypedSetPtr& TypedSet = NamedSet->FindTypedSet<AttributeType>();
			const FAttributeSetIndex AttributeSetIndex = TypedSet->FindIndex(AttributeName);
			TBoundValueMap<AttributeType>* InputABoundValues = InputValueBundle.GetBoundValueMaps().FindOrAdd<AttributeType>(MappingKey);
			InputABoundValues->SetValue(AttributeSetIndex, Input);

			// Setup unbound attributes
			InputValueBundle.GetUnboundValueMaps().FindOrAdd<AttributeType>()->Add(AttributeName, Input);

			if constexpr (SingleBuffer)
			{
				ApplyTransformer(InputValueBundle);
				ValidateResult(Test, NamedSet, "OutOfPlace", InputValueBundle);
			}
			else
			{
				// Transform input into output
				{
					FValueBundleHeap OutputValueBundle(NamedSet);
					ApplyTransformer(InputValueBundle, OutputValueBundle);
					ValidateResult(Test, NamedSet, "OutOfPlace", OutputValueBundle);
				}

				// Transform input inplace
				{
					FValueBundleHeap InputCopyValueBundle(NamedSet);
					InputCopyValueBundle = InputValueBundle;
					ApplyTransformer(InputCopyValueBundle, InputCopyValueBundle);
					ValidateResult(Test, NamedSet, "InplaceA", InputCopyValueBundle);
				}
			}
		};

		void ValidateResult(FAutomationTestBase& Test, const FAttributeNamedSetPtr& NamedSet, FString ErrorPrefix, const FValueBundleHeap& Result)
		{
			// Validate bound attribute
			{
				TOptional<AttributeType> ResultAttribute;
				
				const FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo<AttributeType>();
				if (const FAttributeTypedSetPtr& TypedSet = NamedSet->FindTypedSet<AttributeType>())
				{
					if (const FAttributeSetIndex AttributeSetIndex = TypedSet->FindIndex(AttributeName))
					{
						if (const TBoundValueMap<AttributeType>* OutputBoundValues = Result.GetBoundValueMaps().Find<AttributeType>(MappingKey))
						{
							ResultAttribute = OutputBoundValues->GetValue(AttributeSetIndex);
						}
					}
				}

				TOptional<FString> MaybeError = IsExpectedResult(ResultAttribute);

				if (MaybeError.IsSet())
				{
					Test.AddError(FString::Format(TEXT("{0} Bound: {1}"), { ErrorPrefix, *MaybeError }), 1);
				}
				else
				{
					Test.AddInfo(FString::Format(TEXT("{0} Bound: Pass!"), { ErrorPrefix }));
				}
			}
			
			{
				TOptional<AttributeType> ResultAttribute;

				if (const TUnboundValueMap<AttributeType>* UnboundMap = Result.GetUnboundValueMaps().Find<AttributeType>())
				{
					if (const AttributeType* ResultValue = UnboundMap->Find(AttributeName))
					{
						ResultAttribute = *ResultValue;
					}
				}
				
				TOptional<FString> MaybeError = IsExpectedResult(ResultAttribute);
				if (MaybeError.IsSet())
				{
					Test.AddError(FString::Format(TEXT("{0} Unbound: {1}"), { ErrorPrefix, *MaybeError }), 1);
				}
				else
				{
					Test.AddInfo(FString::Format(TEXT("{0} Unbound: Pass!"), { ErrorPrefix }));
				}
			}
		}
	};

	enum class EInplaceTransformationSupport
	{
		Allow,
		Disallow,
	};

	// Util struct to perform a transformer on an attribute of a particular value and to validate its result.
	// FTransformBinaryTest is for transformers taking two inputs.
	// The same operation is performed on a bound value and unbound value.
	// When calling Run we assume there is a valid attribute defined in the bound maps for the test to use.
	// The InplaceSupport template argument determines whether the transformer allows for inplace operations.
	template <typename TransformerType, typename AttributeType, EInplaceTransformationSupport InplaceSupport>
	struct FTransformerBinaryTest
	{
		FName AttributeName;
		AttributeType InputA;
		AttributeType InputB;
		
		using ApplyTransformerFunctionType = TFunction<void(const FValueBundleHeap& InputAValues, const FValueBundleHeap& InputBValues, FValueBundleHeap& OutputValues)>;
		ApplyTransformerFunctionType ApplyTransformer;
		
		TFunction<TOptional<FString>(TOptional<AttributeType> Result)> IsExpectedResult;

		void Run(FAutomationTestBase& Test, FAttributeNamedSetPtr NamedSet)
		{
			FValueBundleHeap InputAValueBundle(NamedSet);
			InputAValueBundle.SetValueSpace(FValueSpace(EValueSpaceType::Local));
			FValueBundleHeap InputBValueBundle(NamedSet);
			InputBValueBundle.SetValueSpace(FValueSpace(EValueSpaceType::Local));

			// Setup bound attributes
			const FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo<AttributeType>();
			const FAttributeTypedSetPtr& TypedSet = NamedSet->FindTypedSet<AttributeType>();
			const FAttributeSetIndex AttributeSetIndex = TypedSet->FindIndex(AttributeName);
			TBoundValueMap<AttributeType>* InputABoundValues = InputAValueBundle.GetBoundValueMaps().FindOrAdd<AttributeType>(MappingKey);
			TBoundValueMap<AttributeType>* InputBBoundValues = InputBValueBundle.GetBoundValueMaps().FindOrAdd<AttributeType>(MappingKey);
			InputABoundValues->SetValue(AttributeSetIndex, InputA);
			InputBBoundValues->SetValue(AttributeSetIndex, InputB);

			// Setup unbound attributes
			InputAValueBundle.GetUnboundValueMaps().FindOrAdd<AttributeType>()->Add(AttributeName, InputA);
			InputBValueBundle.GetUnboundValueMaps().FindOrAdd<AttributeType>()->Add(AttributeName, InputB);

			// Transform inputs A and B into Output
			FValueBundleHeap OutputValueBundle(NamedSet);
			ApplyTransformer(InputAValueBundle, InputBValueBundle, OutputValueBundle);
			ValidateResult(Test, NamedSet, "OutOfPlace", OutputValueBundle);

			if constexpr (InplaceSupport == EInplaceTransformationSupport::Allow)
			{
				// Transform inputs A and B inplace into A
				FValueBundleHeap InputACopyValueBundle(NamedSet);
				InputACopyValueBundle = InputAValueBundle;
				ApplyTransformer(InputACopyValueBundle, InputBValueBundle, InputACopyValueBundle);
				ValidateResult(Test, NamedSet, "InplaceA", InputACopyValueBundle);

				// Transform inputs A and B inplace into B
				FValueBundleHeap InputBCopyValueBundle(NamedSet);
				InputBCopyValueBundle = InputBValueBundle;
				ApplyTransformer(InputAValueBundle, InputBCopyValueBundle, InputBCopyValueBundle);
				ValidateResult(Test, NamedSet, "InplaceB", InputBCopyValueBundle);
			}
		};

		void ValidateResult(FAutomationTestBase& Test, const FAttributeNamedSetPtr& NamedSet, FString ErrorPrefix, const FValueBundleHeap& Result)
		{
			// Validate bound attribute
			{
				TOptional<AttributeType> ResultAttribute;
				
				const FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo<AttributeType>();
				if (const FAttributeTypedSetPtr& TypedSet = NamedSet->FindTypedSet<AttributeType>())
				{
					if (const FAttributeSetIndex AttributeSetIndex = TypedSet->FindIndex(AttributeName))
					{
						if (const TBoundValueMap<AttributeType>* OutputBoundValues = Result.GetBoundValueMaps().Find<AttributeType>(MappingKey))
						{
							ResultAttribute = OutputBoundValues->GetValue(AttributeSetIndex);
						}
					}
				}

				TOptional<FString> MaybeError = IsExpectedResult(ResultAttribute);

				if (MaybeError.IsSet())
				{
					Test.AddError(FString::Format(TEXT("{0} Bound: {1}"), { ErrorPrefix, *MaybeError }), 1);
				}
				else
				{
					Test.AddInfo(FString::Format(TEXT("{0} Bound: Pass!"), { ErrorPrefix }));
				}
			}
			
			{
				TOptional<AttributeType> ResultAttribute;

				if (const TUnboundValueMap<AttributeType>* UnboundMap = Result.GetUnboundValueMaps().Find<AttributeType>())
				{
					if (const AttributeType* ResultValue = UnboundMap->Find(AttributeName))
					{
						ResultAttribute = *ResultValue;
					}
				}
				
				TOptional<FString> MaybeError = IsExpectedResult(ResultAttribute);
				if (MaybeError.IsSet())
				{
					Test.AddError(FString::Format(TEXT("{0} Unbound: {1}"), { ErrorPrefix, *MaybeError }), 1);
				}
				else
				{
					Test.AddInfo(FString::Format(TEXT("{0} Unbound: Pass!"), { ErrorPrefix }));
				}
			}
		}
	};

	// Specialization for FLayer
	template <typename AttributeType>
	struct FTransformerBinaryTest<Transformers::FLayer, AttributeType, EInplaceTransformationSupport::Allow>
	{
		FName AttributeName;
		float WeightB;
		TOptional<AttributeType> InputA;
		TOptional<AttributeType> InputB;
		TFunction<TOptional<FString>(TOptional<AttributeType> Result)> IsExpectedResult;

		void Run(FAutomationTestBase& Test, const FAttributeNamedSetPtr& NamedSetBase, const FAttributeNamedSetPtr& NamedSetLayer)
		{
			FValueBundleHeap InputAValueBundle(NamedSetBase);
			InputAValueBundle.SetValueSpace(FValueSpace(EValueSpaceType::Local));
			FValueBundleHeap InputBValueBundle(NamedSetLayer);
			InputBValueBundle.SetValueSpace(FValueSpace(EValueSpaceType::Local));
			
			// Setup bound attributes
			const FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo<AttributeType>();
			if (InputA.IsSet())
			{
				const FAttributeTypedSetPtr& TypedSet = NamedSetBase->FindTypedSet<AttributeType>();
				const FAttributeSetIndex AttributeSetIndex = TypedSet->FindIndex(AttributeName);
				TBoundValueMap<AttributeType>* InputABoundValues = InputAValueBundle.GetBoundValueMaps().FindOrAdd<AttributeType>(MappingKey);
				InputABoundValues->SetValue(AttributeSetIndex, *InputA);
			}
			if (InputB.IsSet())
			{
				const FAttributeTypedSetPtr& TypedSet = NamedSetLayer->FindTypedSet<AttributeType>();
				const FAttributeSetIndex AttributeSetIndex = TypedSet->FindIndex(AttributeName);
				TBoundValueMap<AttributeType>* InputBBoundValues = InputBValueBundle.GetBoundValueMaps().FindOrAdd<AttributeType>(MappingKey);
				InputBBoundValues->SetValue(AttributeSetIndex, *InputB);
			}

			// Setup unbound attributes
			if (InputA.IsSet())
			{
				InputAValueBundle.GetUnboundValueMaps().FindOrAdd<AttributeType>()->Add(AttributeName, *InputA);
			}
			if (InputB.IsSet())
			{
				InputBValueBundle.GetUnboundValueMaps().FindOrAdd<AttributeType>()->Add(AttributeName, *InputB);
			}

			const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();

			// Transform inputs A and B into Output
			FValueBundleHeap OutputValueBundle(NamedSetBase);
			Transformers::FLayer::Apply(TransformerMap, InputAValueBundle, InputBValueBundle, WeightB, OutputValueBundle);
			ValidateResult(Test, NamedSetBase, "OutOfPlace", OutputValueBundle);

			// Transform inputs A and B inplace into A
			FValueBundleHeap InputACopyValueBundle(NamedSetBase);
			InputACopyValueBundle = InputAValueBundle;
			Transformers::FLayer::Apply(TransformerMap, InputACopyValueBundle, InputBValueBundle, WeightB, InputACopyValueBundle);
			ValidateResult(Test, NamedSetBase, "InplaceA", InputACopyValueBundle);
		};

		void ValidateResult(FAutomationTestBase& Test, const FAttributeNamedSetPtr& NamedSet, FString ErrorPrefix, const FValueBundleHeap& Result)
		{
			// Validate bound attribute
			{
				TOptional<AttributeType> ResultAttribute;
				
				const FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo<AttributeType>();
				if (const FAttributeTypedSetPtr& TypedSet = NamedSet->FindTypedSet<AttributeType>())
				{
					if (const FAttributeSetIndex AttributeSetIndex = TypedSet->FindIndex(AttributeName))
					{
						if (const TBoundValueMap<AttributeType>* OutputBoundValues = Result.GetBoundValueMaps().Find<AttributeType>(MappingKey))
						{
							ResultAttribute = OutputBoundValues->GetValue(AttributeSetIndex);
						}
					}
				}

				TOptional<FString> MaybeError = IsExpectedResult(ResultAttribute);

				if (MaybeError.IsSet())
				{
					Test.AddError(FString::Format(TEXT("{0} Bound: {1}"), { ErrorPrefix, *MaybeError }), 1);
				}
				else
				{
					Test.AddInfo(FString::Format(TEXT("{0} Bound: Pass!"), { ErrorPrefix }));
				}
			}
			
			// Validate unbound attribute
			{
				TOptional<AttributeType> ResultAttribute;

				if (const TUnboundValueMap<AttributeType>* UnboundMap = Result.GetUnboundValueMaps().Find<AttributeType>())
				{
					if (const AttributeType* ResultValue = UnboundMap->Find(AttributeName))
					{
						ResultAttribute = *ResultValue;
					}
				}
				
				TOptional<FString> MaybeError = IsExpectedResult(ResultAttribute);
				if (MaybeError.IsSet())
				{
					Test.AddError(FString::Format(TEXT("{0} Unbound: {1}"), { ErrorPrefix, *MaybeError }), 1);
				}
				else
				{
					Test.AddInfo(FString::Format(TEXT("{0} Unbound: Pass!"), { ErrorPrefix }));
				}
			}
		}
	};
}