// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "AnimNextTest.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "UAF/Attributes/AttributeBindingDataCache.h"
#include "UAF/ValueRuntime/ValueRuntimeRegistry.h"
#include "UAF/ValueRuntime/Transformers/Accumulate.h"
#include "UAF/ValueRuntime/Transformers/AdditiveSpace.h"
#include "UAF/ValueRuntime/Transformers/Interpolate.h"
#include "UAF/ValueRuntime/Transformers/Overwrite.h"
#include "UAF/ValueRuntime/Transformers/TransformerTestUtil.h"
#include "UObject/Package.h"

namespace UE::UAF::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTransformerTests_VectorAttribute, "Animation.UAF.ValueRuntime.Transformers.VectorAttribute", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool FTransformerTests_VectorAttribute::RunTest(const FString& InParameters)
	{
		ON_SCOPE_EXIT{ FUtils::CleanupAfterTests(); };

		TValueOrError<TNonNullPtr<UAbstractSkeletonSetBinding>, FString> SetBindingResult = BuildSetBinding({});

		if (SetBindingResult.HasError())
		{
			AddError(SetBindingResult.GetError());
			return false;
		}
		
		TNonNullPtr<UAbstractSkeletonSetBinding> SetBinding = SetBindingResult.GetValue();

		SetBinding->AddAttributeToSet(FAnimationAttributeIdentifier(TEXT("AttrA"), INDEX_NONE, NAME_None, FVectorAnimationAttribute::StaticStruct()), "SetA");
		SetBinding->AddAttributeToSet(FAnimationAttributeIdentifier(TEXT("AttrB"), INDEX_NONE, NAME_None, FVectorAnimationAttribute::StaticStruct()), "SetB");
		SetBinding->AddAttributeToSet(FAnimationAttributeIdentifier(TEXT("AttrC"), INDEX_NONE, NAME_None, FVectorAnimationAttribute::StaticStruct()), "SetC");
		
		const FAttributeBindingDataPtr& BindingData = GAttributeBindingDataCache.GetOrAdd(SetBinding, nullptr);
		FAttributeNamedSetPtr SetA = BindingData->FindNamedSet("SetA");
		FAttributeNamedSetPtr SetB = BindingData->FindNamedSet("SetB");
		FAttributeNamedSetPtr SetC = BindingData->FindNamedSet("SetC");

		// Overwrite
		AddInfo("Overwrite");
		{
			// 0.25 weight
			{
				FTransformerUnaryTest<FVectorAnimationAttribute, false> VectorTest;
				VectorTest.AttributeName = "AttrA";
				VectorTest.Input.Value = FVector(1.0f, 2.0f, 3.0f);
				VectorTest.ApplyTransformer = [](const FValueBundleHeap& InputValues, FValueBundleHeap& OutputValues)
					{
						const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();
						Transformers::FOverwrite::Apply(TransformerMap, InputValues, 0.25f, OutputValues);
					};
				VectorTest.IsExpectedResult = [&](TOptional<FVectorAnimationAttribute> Result) -> TOptional<FString>
					{
						if (!Result.IsSet())
						{
							return FString("Overwrite - 0.25 weight - expected to produce a value but received none");
						}

						const FVector ExpectedValue = FVector(1.0f, 2.0f, 3.0f) * 0.25f;
						if (!ExpectedValue.Equals(Result->Value))
						{
							return FString::Format(TEXT("Overwrite - 0.25 weight - expected {0}, Received {1}"), { ExpectedValue.ToString(), Result->Value.ToString() });
						}

						return TOptional<FString>();
					};
			}
		}
		
		// Accumulate
		AddInfo("Accumulate");
		{
			// 0.25 weight
			{
				FTransformerBinaryTest<Transformers::FAccumulate, FVectorAnimationAttribute, EInplaceTransformationSupport::Allow> VectorTest;
				VectorTest.AttributeName = "AttrA";
				VectorTest.InputA.Value = FVector(1.0f, 2.0f, 3.0f);
				VectorTest.InputB.Value = FVector(4.0f, 5.0f, 6.0f);
				VectorTest.ApplyTransformer = [](const FValueBundleHeap& InputAValues, const FValueBundleHeap& InputBValues, FValueBundleHeap& OutputValues)
					{
						const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();
						Transformers::FAccumulate::Apply(TransformerMap, InputAValues, InputBValues, 0.25f, OutputValues);
					};
				VectorTest.IsExpectedResult = [&](TOptional<FVectorAnimationAttribute> Result) -> TOptional<FString>
					{
						if (!Result.IsSet())
						{
							return FString("Accumulate - 0.25 weight - expected to produce a value but received none");
						}
						const FVector ExpectedValue = FVector(1.0f, 2.0f, 3.0f) + 0.25f * FVector(4.0f, 5.0f, 6.0f);
						if (!ExpectedValue.Equals(Result->Value))
						{
							return FString::Format(TEXT("Accumulate - 0.25 weight - expected {0}, received {1}"), { ExpectedValue.ToString(), Result->Value.ToString() });
						}
						return TOptional<FString>();
					};

				VectorTest.Run(*this, SetA);
			}
		}

		// Interpolate
		AddInfo("Interpolate");
		{
			// 0.25 weight
			{
				FTransformerBinaryTest<Transformers::FInterpolate, FVectorAnimationAttribute, EInplaceTransformationSupport::Allow> VectorTest;
				VectorTest.AttributeName = "AttrA";
				VectorTest.InputA.Value = FVector(1.0f, 2.0f, 3.0f);
				VectorTest.InputB.Value = FVector(4.0f, 5.0f, 6.0f);
				VectorTest.ApplyTransformer = [](const FValueBundleHeap& InputAValues, const FValueBundleHeap& InputBValues, FValueBundleHeap& OutputValues)
					{
						const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();
						Transformers::FInterpolate::Apply(TransformerMap, InputAValues, InputBValues, 0.25f, OutputValues);
					};
				VectorTest.IsExpectedResult = [&](TOptional<FVectorAnimationAttribute> Result) -> TOptional<FString>
					{
						if (!Result.IsSet())
						{
							return FString("Interpolate - 0.25 weight -expected to produce a value but received none");
						}
						
						const FVector ExpectedValue = 0.75f * FVector(1.0f, 2.0f, 3.0f) + 0.25f * FVector(4.0f, 5.0f, 6.0f);
						if (!ExpectedValue.Equals(Result->Value))
						{
							return FString::Format(TEXT("Interpolate - 0.25 weight - expected {0}, received {1}"), { ExpectedValue.ToString(), Result->Value.ToString() });
						}
						
						return TOptional<FString>();
					};

				VectorTest.Run(*this, SetA);
			}
		}

		// Layer
		AddInfo("Layer");
		{
			// Value in base but not in layer
			{
				// Weight 0.25
				{
					FTransformerBinaryTest<Transformers::FLayer, FVectorAnimationAttribute, EInplaceTransformationSupport::Allow> VectorTest;
					VectorTest.AttributeName = "AttrA";
					VectorTest.WeightB = 0.25f;
					VectorTest.InputA = FVectorAnimationAttribute({ FVector(1.0f, 2.0f, 3.0f) });
					VectorTest.InputB.Reset();
					VectorTest.IsExpectedResult = [&](TOptional<FVectorAnimationAttribute> Result) -> TOptional<FString>
						{
							if (!Result.IsSet())
							{
								return FString("Layer - 0.25 weight, case 1 - expected to produce a value but received none");
							}
							
							const FVector ExpectedValue = FVector(1.0f, 2.0f, 3.0f);
							if (!ExpectedValue.Equals(Result->Value))
							{
								return FString::Format(TEXT("Layer - 0.25 weight, case 1 - expected {0}, received {1}"), { ExpectedValue.ToString(), Result->Value.ToString() });
							}
							
							return TOptional<FString>();
						};

					VectorTest.Run(*this, SetA, SetB);
				}
			}

			// Value in layer but not in base
			{
				// Weight 0.25
				{
					FTransformerBinaryTest<Transformers::FLayer, FVectorAnimationAttribute, EInplaceTransformationSupport::Allow> VectorTest;
					VectorTest.AttributeName = "AttrC";
					VectorTest.WeightB = 0.25f;
					VectorTest.InputA.Reset();
					VectorTest.InputB = FVectorAnimationAttribute({ FVector(1.0f, 2.0f, 3.0f) });
					VectorTest.IsExpectedResult = [&](TOptional<FVectorAnimationAttribute> Result) -> TOptional<FString>
						{
							if (Result.IsSet())
							{
								return FString("Layer - 0.25 weight, case 2 - expected to not produce a value but received one");
							}
														
							return TOptional<FString>();
						};

					VectorTest.Run(*this, SetB, SetC);
				}
			}
			
			// Value in both base and layer
			{
				// Weight 0.25
				{
					FTransformerBinaryTest<Transformers::FLayer, FVectorAnimationAttribute, EInplaceTransformationSupport::Allow> VectorTest;
					VectorTest.AttributeName = "AttrB";
					VectorTest.WeightB = 0.25f;
					VectorTest.InputA = FVectorAnimationAttribute({ FVector(1.0f, 2.0f, 3.0f) });
					VectorTest.InputB = FVectorAnimationAttribute({ FVector(4.0f, 5.0f, 6.0f) });
					VectorTest.IsExpectedResult = [&](TOptional<FVectorAnimationAttribute> Result) -> TOptional<FString>
						{
							if (!Result.IsSet())
							{
								return FString("Layer - 0.25 weight, case 3 - expected to produce a value but received none");
							}
							
							const FVector ExpectedValue = 0.75f * FVector(1.0f, 2.0f, 3.0f) + 0.25f * FVector(4.0f, 5.0f, 6.0f);
							if (!ExpectedValue.Equals(Result->Value))
							{
								return FString::Format(TEXT("Layer - 0.25 weight, case 3 - expected {0}, received {1}"), { ExpectedValue.ToString(), Result->Value.ToString() });
							}
							
							return TOptional<FString>();
						};

					VectorTest.Run(*this, SetA, SetB);
				}
			}
		}

		// Make Additive Space
		AddInfo("MakeAdditiveSpace");
		{
			// 0.25 weight
			{
				FTransformerBinaryTest<Transformers::FMakeAdditiveSpace, FVectorAnimationAttribute, EInplaceTransformationSupport::Disallow> VectorTest;
				VectorTest.AttributeName = "AttrA";
				VectorTest.InputA = FVectorAnimationAttribute({ FVector(1.0f, 2.0f, 3.0f) });
				VectorTest.InputB = FVectorAnimationAttribute({ FVector(4.0f, 5.0f, 6.0f) });
				VectorTest.ApplyTransformer = [](const FValueBundleHeap& InputAValues, const FValueBundleHeap& InputBValues, FValueBundleHeap& OutputValues)
					{
						const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();
						Transformers::FMakeAdditiveSpace::Apply(TransformerMap, InputAValues, InputBValues, OutputValues);
					};
				VectorTest.IsExpectedResult = [&](TOptional<FVectorAnimationAttribute> Result) -> TOptional<FString>
					{
						if (!Result.IsSet())
						{
							return FString("MakeAdditiveSpace - expected to produce a value but received none");
						}
						
						const FVector ExpectedValue = FVector(4.0f, 5.0f, 6.0f) - FVector(1.0f, 2.0f, 3.0f);
						if (!ExpectedValue.Equals(Result->Value))
						{
							return FString::Format(TEXT("MakeAdditiveSpace - expected {0}, received {1}"), { ExpectedValue.ToString(), Result->Value.ToString() });
						}
						
						return TOptional<FString>();
					};

				VectorTest.Run(*this, SetA);
			}
		}

		// Apply Additive Space
		AddInfo("ApplyAdditiveSpace");
		{
			// 0.25 weight
			{
				FTransformerBinaryTest<Transformers::FMakeAdditiveSpace, FVectorAnimationAttribute, EInplaceTransformationSupport::Allow> VectorTest;
				VectorTest.AttributeName = "AttrA";
				VectorTest.InputA = FVectorAnimationAttribute({ FVector(1.0f, 2.0f, 3.0f) });
				VectorTest.InputB = FVectorAnimationAttribute({ FVector(4.0f, 5.0f, 6.0f) });
				VectorTest.ApplyTransformer = [](const FValueBundleHeap& InputAValues, const FValueBundleHeap& InputBValues, FValueBundleHeap& OutputValues)
					{
						const_cast<FValueBundleHeap&>(InputBValues).SetValueSpace(FValueSpace(EValueSpaceType::Local, true));
						
						const auto& TransformerMap = FValueRuntimeRegistry::Get().GetTransformerMap();
						Transformers::FApplyAdditiveSpace::Apply(TransformerMap, InputAValues, InputBValues, 0.25f, OutputValues);
					};
				VectorTest.IsExpectedResult = [&](TOptional<FVectorAnimationAttribute> Result) -> TOptional<FString>
					{
						if (!Result.IsSet())
						{
							return FString("ApplyAdditiveSpace - 0.25 weight - expected to produce a value but received none");
						}
						
						const FVector ExpectedValue = FVector(1.0f, 2.0f, 3.0f) + 0.25f * FVector(4.0f, 5.0f, 6.0f);
						if (!ExpectedValue.Equals(Result->Value))
						{
							return FString::Format(TEXT("ApplyAdditiveSpace - 0.25 weight - expected {0}, received {1}"), { ExpectedValue.ToString(), Result->Value.ToString() });
						}
						
						return TOptional<FString>();
					};

				VectorTest.Run(*this, SetA);
			}
		}
		
		return true;
	}
}

#endif // WITH_DEV_AUTOMATION_TESTS