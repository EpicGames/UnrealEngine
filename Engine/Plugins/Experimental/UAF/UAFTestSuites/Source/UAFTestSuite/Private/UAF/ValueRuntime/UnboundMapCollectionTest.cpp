// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "AnimNextTest.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "UAF/Attributes/EngineAttributes.h"
#include "UAF/ValueRuntime/UnboundMapCollection.h"
#include "UAF/ValueRuntime/IteratorTestUtils.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

namespace UE::UAF::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FUnboundMapCollectionTest, "Animation.UAF.ValueRuntime.UnboundMapCollection", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool FUnboundMapCollectionTest::RunTest(const FString& InParameters)
	{
		ON_SCOPE_EXIT{ FUtils::CleanupAfterTests(); };

		{
			FUnboundMapCollectionHeap Collection;

			AddErrorIfFalse(Collection.IsEmpty(), TEXT("Collection should be empty"));

			TUnboundValueMap<FFloatAnimationAttribute>* FloatMap = Collection.Add<FFloatAnimationAttribute>();
			UE_RETURN_ON_ERROR(FloatMap != nullptr, TEXT("Float map should be valid"));

			TUnboundValueMap<FBoneTransformAnimationAttribute>* BoneMap = Cast<FBoneTransformAnimationAttribute>(Collection.Add(FBoneTransformAnimationAttribute::StaticStruct()));
			UE_RETURN_ON_ERROR(BoneMap != nullptr, TEXT("Bone map should be valid"));

			TUnboundValueMap<FIntegerAnimationAttribute>* IntMap = MakeUnboundValueMap<FIntegerAnimationAttribute>(Collection.GetAllocator());
			UE_RETURN_ON_ERROR(IntMap != nullptr, TEXT("Integer map should be valid"));
			bool bAdded = Collection.Add(IntMap);
			UE_RETURN_ON_ERROR(bAdded, TEXT("Integer map should be added"));

			// Can't add twice
			bAdded = Collection.Add(IntMap);
			AddErrorIfFalse(!bAdded, TEXT("Integer map should not be added"));

			AddErrorIfFalse(!Collection.IsEmpty(), TEXT("Collection should not be empty"));
			AddErrorIfFalse(Collection.Num() == 3, TEXT("Unexpected map count"));

			AddErrorIfFalse(FloatMap == Collection.Find<FFloatAnimationAttribute>(), TEXT("Unexpected result from Find"));
			AddErrorIfFalse(BoneMap == Collection.Find(FBoneTransformAnimationAttribute::StaticStruct()), TEXT("Unexpected result from Find"));

			AddErrorIfFalse(Collection.Remove(FloatMap), TEXT("Failed to remove map"));
			AddErrorIfFalse(Collection.Remove(FBoneTransformAnimationAttribute::StaticStruct()), TEXT("Failed to remove map"));
			AddErrorIfFalse(Collection.Remove(IntMap), TEXT("Failed to remove map"));

			AddErrorIfFalse(nullptr == Collection.Find<FFloatAnimationAttribute>(), TEXT("Unexpected result from Find"));
			AddErrorIfFalse(nullptr == Collection.Find(FBoneTransformAnimationAttribute::StaticStruct()), TEXT("Unexpected result from Find"));

			AddErrorIfFalse(Collection.IsEmpty(), TEXT("Collection should be empty"));
			AddErrorIfFalse(Collection.Num() == 0, TEXT("Unexpected map count"));

			FloatMap = Collection.FindOrAdd<FFloatAnimationAttribute>();
			UE_RETURN_ON_ERROR(FloatMap != nullptr, TEXT("Float map should be valid"));

			BoneMap = Cast<FBoneTransformAnimationAttribute>(Collection.FindOrAdd(FBoneTransformAnimationAttribute::StaticStruct()));
			UE_RETURN_ON_ERROR(BoneMap != nullptr, TEXT("Bone map should be valid"));

			AddErrorIfFalse(!Collection.IsEmpty(), TEXT("Collection should not be empty"));
			AddErrorIfFalse(Collection.Num() == 2, TEXT("Unexpected map count"));

			AddErrorIfFalse(FloatMap == Collection.FindOrAdd<FFloatAnimationAttribute>(), TEXT("Unexpected result from FindOrAdd"));
			AddErrorIfFalse(BoneMap == Collection.FindOrAdd(FBoneTransformAnimationAttribute::StaticStruct()), TEXT("Unexpected result from FindOrAdd"));

			AddErrorIfFalse(!Collection.IsEmpty(), TEXT("Attribute collection should not be empty"));
			AddErrorIfFalse(Collection.Num() == 2, TEXT("Unexpected map count"));

			// Append
			{
				FUnboundMapCollectionHeap CollectionCopy;

				for (auto It = Collection.CreateConstIterator(); It; ++It)
				{
					FUnboundValueMap* MapCopy = It.GetMap()->Duplicate(CollectionCopy.GetAllocator());
					const bool bAppended = CollectionCopy.Append(MapCopy);
					AddErrorIfFalse(bAppended, TEXT("Map should be appended"));
				}

				AddErrorIfFalse(CollectionCopy.IsEmpty() == Collection.IsEmpty(), TEXT("Collection should not be empty"));
				AddErrorIfFalse(CollectionCopy.Num() == Collection.Num(), TEXT("Unexpected map count"));

				TUnboundValueMap<FFloatAnimationAttribute>* FloatMapCopy = CollectionCopy.Find<FFloatAnimationAttribute>();
				UE_RETURN_ON_ERROR(FloatMapCopy != nullptr, TEXT("Float map should be valid"));

				TUnboundValueMap<FBoneTransformAnimationAttribute>* BoneMapCopy = Cast<FBoneTransformAnimationAttribute>(CollectionCopy.Find(FBoneTransformAnimationAttribute::StaticStruct()));
				UE_RETURN_ON_ERROR(BoneMapCopy != nullptr, TEXT("Bone map should be valid"));

				AddErrorIfFalse(FloatMap != FloatMapCopy, TEXT("Map instances should be duplicated"));
				AddErrorIfFalse(BoneMap != BoneMapCopy, TEXT("Map instances should be duplicated"));
			}

			// Copy constructor
			{
				FUnboundMapCollectionHeap CollectionCopy(Collection);

				AddErrorIfFalse(CollectionCopy.IsEmpty() == Collection.IsEmpty(), TEXT("Collection should not be empty"));
				AddErrorIfFalse(CollectionCopy.Num() == Collection.Num(), TEXT("Unexpected map count"));

				TUnboundValueMap<FFloatAnimationAttribute>* FloatMapCopy = CollectionCopy.Find<FFloatAnimationAttribute>();
				UE_RETURN_ON_ERROR(FloatMapCopy != nullptr, TEXT("Float map should be valid"));

				TUnboundValueMap<FBoneTransformAnimationAttribute>* BoneMapCopy = Cast<FBoneTransformAnimationAttribute>(CollectionCopy.Find(FBoneTransformAnimationAttribute::StaticStruct()));
				UE_RETURN_ON_ERROR(BoneMapCopy != nullptr, TEXT("Bone map should be valid"));

				AddErrorIfFalse(FloatMap != FloatMapCopy, TEXT("Map instances should be duplicated"));
				AddErrorIfFalse(BoneMap != BoneMapCopy, TEXT("Map instances should be duplicated"));

				// Move constructor
				{
					FUnboundMapCollectionHeap CollectionMoved(MoveTemp(CollectionCopy));

					AddErrorIfFalse(CollectionCopy.IsEmpty(), TEXT("Collection should be empty"));
					AddErrorIfFalse(!CollectionMoved.IsEmpty(), TEXT("Collection should not be empty"));
					AddErrorIfFalse(CollectionMoved.Num() == Collection.Num(), TEXT("Unexpected map count"));

					TUnboundValueMap<FFloatAnimationAttribute>* FloatMapMoved = CollectionMoved.Find<FFloatAnimationAttribute>();
					UE_RETURN_ON_ERROR(FloatMapMoved != nullptr, TEXT("Float map should be valid"));

					TUnboundValueMap<FBoneTransformAnimationAttribute>* BoneMapMoved = Cast<FBoneTransformAnimationAttribute>(CollectionMoved.Find(FBoneTransformAnimationAttribute::StaticStruct()));
					UE_RETURN_ON_ERROR(BoneMapMoved != nullptr, TEXT("Bone map should be valid"));

					AddErrorIfFalse(FloatMapMoved == FloatMapCopy, TEXT("Set map instances should be moved"));
					AddErrorIfFalse(BoneMapMoved == BoneMapCopy, TEXT("Set map instances should be moved"));
				}
			}

			// Copy assignment
			{
				FUnboundMapCollectionHeap CollectionCopy;
				CollectionCopy = Collection;

				AddErrorIfFalse(CollectionCopy.IsEmpty() == Collection.IsEmpty(), TEXT("Collection should not be empty"));
				AddErrorIfFalse(CollectionCopy.Num() == Collection.Num(), TEXT("Unexpected map count"));

				TUnboundValueMap<FFloatAnimationAttribute>* FloatMapCopy = CollectionCopy.Find<FFloatAnimationAttribute>();
				UE_RETURN_ON_ERROR(FloatMapCopy != nullptr, TEXT("Float map should be valid"));

				TUnboundValueMap<FBoneTransformAnimationAttribute>* BoneMapCopy = Cast<FBoneTransformAnimationAttribute>(CollectionCopy.Find(FBoneTransformAnimationAttribute::StaticStruct()));
				UE_RETURN_ON_ERROR(BoneMapCopy != nullptr, TEXT("Bone map should be valid"));

				AddErrorIfFalse(FloatMap != FloatMapCopy, TEXT("Map instances should be duplicated"));
				AddErrorIfFalse(BoneMap != BoneMapCopy, TEXT("Map instances should be duplicated"));

				// Move assignment
				{
					FUnboundMapCollectionHeap CollectionMoved;
					CollectionMoved = MoveTemp(CollectionCopy);

					AddErrorIfFalse(CollectionCopy.IsEmpty(), TEXT("Collection should be empty"));
					AddErrorIfFalse(!CollectionMoved.IsEmpty(), TEXT("Collection should not be empty"));
					AddErrorIfFalse(CollectionMoved.Num() == Collection.Num(), TEXT("Unexpected map count"));

					TUnboundValueMap<FFloatAnimationAttribute>* FloatMapMoved = CollectionMoved.Find<FFloatAnimationAttribute>();
					UE_RETURN_ON_ERROR(FloatMapMoved != nullptr, TEXT("Float map should be valid"));

					TUnboundValueMap<FBoneTransformAnimationAttribute>* BoneMapMoved = Cast<FBoneTransformAnimationAttribute>(CollectionMoved.Find(FBoneTransformAnimationAttribute::StaticStruct()));
					UE_RETURN_ON_ERROR(BoneMapMoved != nullptr, TEXT("Bone map should be valid"));

					AddErrorIfFalse(FloatMapMoved == FloatMapCopy, TEXT("Map instances should be moved"));
					AddErrorIfFalse(BoneMapMoved == BoneMapCopy, TEXT("Map instances should be moved"));
				}
			}

			// Iterators
			{
				const auto GetMapValueTypeProjection = [](const typename FUnboundMapCollection::FIterator& It) { return It.GetValueType(); };
				const auto GetMapValueTypeProjectionConst = [](const typename FUnboundMapCollection::FConstIterator& It) { return It.GetValueType(); };
				const auto GetMapProjection = [](const typename FUnboundMapCollection::FIterator& It) { return It.GetMap(); };
				const auto GetMapProjectionConst = [](const typename FUnboundMapCollection::FConstIterator& It) { return It.GetMap(); };

				AddErrorIfFalse(IteratorSize(Collection.CreateIterator()) == 2, TEXT("Iterator did not have expected size"));
				AddErrorIfFalse(IteratorSize(Collection.CreateConstIterator()) == 2, TEXT("Iterator did not have expected size"));

				AddErrorIfFalse(IteratorSortedByPredicate(Collection.CreateIterator(), GetMapValueTypeProjection, TLess<UScriptStruct*>()), TEXT("Iterator should be sorted"));
				AddErrorIfFalse(IteratorSortedByPredicate(Collection.CreateConstIterator(), GetMapValueTypeProjectionConst, TLess<UScriptStruct*>()), TEXT("Iterator should be sorted"));

				AddErrorIfFalse(FindWithBy(Collection.CreateIterator(), FloatMap, GetMapProjection), TEXT("Iterator did not contain expected map"));
				AddErrorIfFalse(FindWithBy(Collection.CreateIterator(), BoneMap, GetMapProjection), TEXT("Iterator did not contain expected map"));

				AddErrorIfFalse(FindWithBy(Collection.CreateConstIterator(), FloatMap, GetMapProjectionConst), TEXT("Iterator did not contain expected map"));
				AddErrorIfFalse(FindWithBy(Collection.CreateConstIterator(), BoneMap, GetMapProjectionConst), TEXT("Iterator did not contain expected map"));
			}
		}

		return true;
	}
}

#endif	// WITH_DEV_AUTOMATION_TESTS
