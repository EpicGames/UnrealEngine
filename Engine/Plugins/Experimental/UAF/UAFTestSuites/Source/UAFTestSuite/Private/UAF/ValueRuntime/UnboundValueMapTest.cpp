// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "Algo/Find.h"
#include "AnimNextTest.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "UAF/Attributes/EngineAttributes.h"
#include "UAF/ValueRuntime/IteratorTestUtils.h"
#include "UAF/ValueRuntime/UnboundValueMap.h"

namespace UE::UAF::Tests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FUnboundValueMapTest, "Animation.UAF.ValueRuntime.UnboundValueMap", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool FUnboundValueMapTest::RunTest(const FString& InParameters)
	{
		FReallocFun Allocator = &FAllocatorTypeTrait<FDefaultAllocator>::Realloc;

		// Float value
		{
			TUnboundValueMap<FFloatAnimationAttribute>& FloatMap = *MakeUnboundValueMap<FFloatAnimationAttribute>(Allocator);

			AddErrorIfFalse(FloatMap.IsEmpty(), TEXT("Attribute map should be empty"));
			AddErrorIfFalse(FloatMap.Num() == 0, TEXT("Unexpected map size"));
			AddErrorIfFalse(FloatMap.Max() == 0, TEXT("Unexpected map capacity"));
			AddErrorIfFalse(FloatMap.GetValueType() == FFloatAnimationAttribute::StaticStruct(), TEXT("Unexpected map value type"));

			AddErrorIfFalse(!FloatMap.Contains(TEXT("TestValue0")), TEXT("Attribute map should be empty"));
			AddErrorIfFalse(FloatMap.IndexOf(TEXT("TestValue0")) == INDEX_NONE, TEXT("Attribute map should be empty"));
			AddErrorIfFalse(FloatMap.Find(TEXT("TestValue0")) == nullptr, TEXT("Attribute map should be empty"));

			bool bSuccess = true;

			FFloatAnimationAttribute Value0{ 123.44f };
			FFloatAnimationAttribute Value1{ -123.44f };
			bSuccess &= FloatMap.Add(TEXT("TestValue0"), Value0);
			bSuccess &= FloatMap.Add(TEXT("TestValue1"), Value1);

			// Add a few more to trigger reallocation
			FFloatAnimationAttribute ValueA{ 14.5f };
			FFloatAnimationAttribute ValueB{ 117.8f };
			FFloatAnimationAttribute ValueC{ 0.1345f };
			bSuccess &= FloatMap.Add(TEXT("TestValueA"), ValueA);
			bSuccess &= FloatMap.Add(TEXT("TestValueB"), ValueB);
			bSuccess &= FloatMap.Add(TEXT("TestValueC"), ValueC);

			AddErrorIfFalse(bSuccess, TEXT("Failed to add values to map"));
			AddErrorIfFalse(FloatMap.Contains(TEXT("TestValue0")), TEXT("Failed to add value to map"));
			AddErrorIfFalse(FloatMap.Contains(TEXT("TestValue1")), TEXT("Failed to add value to map"));
			AddErrorIfFalse(FloatMap.Contains(TEXT("TestValueA")), TEXT("Failed to add value to map"));
			AddErrorIfFalse(FloatMap.Contains(TEXT("TestValueB")), TEXT("Failed to add value to map"));
			AddErrorIfFalse(FloatMap.Contains(TEXT("TestValueC")), TEXT("Failed to add value to map"));
			UE_RETURN_ON_ERROR(FloatMap.IndexOf(TEXT("TestValue0")) != INDEX_NONE, TEXT("Failed to add value to map"));
			UE_RETURN_ON_ERROR(FloatMap.IndexOf(TEXT("TestValue1")) != INDEX_NONE, TEXT("Failed to add value to map"));
			UE_RETURN_ON_ERROR(FloatMap.IndexOf(TEXT("TestValueA")) != INDEX_NONE, TEXT("Failed to add value to map"));
			UE_RETURN_ON_ERROR(FloatMap.IndexOf(TEXT("TestValueB")) != INDEX_NONE, TEXT("Failed to add value to map"));
			UE_RETURN_ON_ERROR(FloatMap.IndexOf(TEXT("TestValueC")) != INDEX_NONE, TEXT("Failed to add value to map"));
			AddErrorIfFalse(FloatMap.Find(TEXT("TestValue0")) != nullptr, TEXT("Failed to add value to map"));
			AddErrorIfFalse(FloatMap.Find(TEXT("TestValue1")) != nullptr, TEXT("Failed to add value to map"));
			AddErrorIfFalse(FloatMap.Find(TEXT("TestValueA")) != nullptr, TEXT("Failed to add value to map"));
			AddErrorIfFalse(FloatMap.Find(TEXT("TestValueB")) != nullptr, TEXT("Failed to add value to map"));
			AddErrorIfFalse(FloatMap.Find(TEXT("TestValueC")) != nullptr, TEXT("Failed to add value to map"));
			AddErrorIfFalse(FloatMap.GetName(FloatMap.IndexOf(TEXT("TestValue0"))) == TEXT("TestValue0"), TEXT("Unexpected mapped value"));
			AddErrorIfFalse(FloatMap.GetName(FloatMap.IndexOf(TEXT("TestValue1"))) == TEXT("TestValue1"), TEXT("Unexpected mapped value"));
			AddErrorIfFalse(FloatMap.GetName(FloatMap.IndexOf(TEXT("TestValueA"))) == TEXT("TestValueA"), TEXT("Unexpected mapped value"));
			AddErrorIfFalse(FloatMap.GetName(FloatMap.IndexOf(TEXT("TestValueB"))) == TEXT("TestValueB"), TEXT("Unexpected mapped value"));
			AddErrorIfFalse(FloatMap.GetName(FloatMap.IndexOf(TEXT("TestValueC"))) == TEXT("TestValueC"), TEXT("Unexpected mapped value"));
			AddErrorIfFalse(FloatMap.GetValue(FloatMap.IndexOf(TEXT("TestValue0"))).Value == Value0.Value, TEXT("Unexpected mapped value"));
			AddErrorIfFalse(FloatMap.GetValue(FloatMap.IndexOf(TEXT("TestValue1"))).Value == Value1.Value, TEXT("Unexpected mapped value"));
			AddErrorIfFalse(FloatMap.GetValue(FloatMap.IndexOf(TEXT("TestValueA"))).Value == ValueA.Value, TEXT("Unexpected mapped value"));
			AddErrorIfFalse(FloatMap.GetValue(FloatMap.IndexOf(TEXT("TestValueB"))).Value == ValueB.Value, TEXT("Unexpected mapped value"));
			AddErrorIfFalse(FloatMap.GetValue(FloatMap.IndexOf(TEXT("TestValueC"))).Value == ValueC.Value, TEXT("Unexpected mapped value"));

			// Add duplicate
			bSuccess = FloatMap.Add(TEXT("TestValue0"), Value0);

			AddErrorIfFalse(!bSuccess, TEXT("Did not failed to add duplicate value to map"));
			AddErrorIfFalse(FloatMap.Contains(TEXT("TestValue0")), TEXT("Failed to add value to map"));

			FFloatAnimationAttribute Value2{ -0.13544f };
			bSuccess = FloatMap.Add(TEXT("TestValue2"), Value2);

			AddErrorIfFalse(bSuccess, TEXT("Failed to add values to map"));
			AddErrorIfFalse(FloatMap.Contains(TEXT("TestValue2")), TEXT("Failed to add value to map"));

			bSuccess = FloatMap.Remove(TEXT("TestValue2"));

			AddErrorIfFalse(bSuccess, TEXT("Failed to remove value from map"));
			AddErrorIfFalse(!FloatMap.Contains(TEXT("TestValue2")), TEXT("Failed to remove value from map"));

			// Remove value not contained

			bSuccess = FloatMap.Remove(TEXT("TestValue2"));

			AddErrorIfFalse(!bSuccess, TEXT("Did not failed to remove missing value from map"));
			AddErrorIfFalse(!FloatMap.Contains(TEXT("TestValue2")), TEXT("Failed to remove value from map"));

			// Copy
			{
				TUnboundValueMap<FFloatAnimationAttribute>& FloatMapCopy = *MakeUnboundValueMap<FFloatAnimationAttribute>(Allocator);
				FloatMap.CopyTo(FloatMapCopy);

				AddErrorIfFalse(!FloatMapCopy.IsEmpty(), TEXT("Attribute map should not be empty"));
				AddErrorIfFalse(FloatMapCopy.Num() == FloatMap.Num(), TEXT("Unexpected map size"));
				AddErrorIfFalse(FloatMapCopy.Max() == FloatMap.Num(), TEXT("Unexpected map capacity"));
				AddErrorIfFalse(FloatMapCopy.GetValueType() == FloatMap.GetValueType(), TEXT("Unexpected map value type"));

				AddErrorIfFalse(FloatMapCopy.Contains(TEXT("TestValue0")), TEXT("Failed to copy map"));
				AddErrorIfFalse(FloatMapCopy.Contains(TEXT("TestValue1")), TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(FloatMapCopy.IndexOf(TEXT("TestValue0")) != INDEX_NONE, TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(FloatMapCopy.IndexOf(TEXT("TestValue1")) != INDEX_NONE, TEXT("Failed to copy map"));
				AddErrorIfFalse(FloatMapCopy.Find(TEXT("TestValue0")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(FloatMapCopy.Find(TEXT("TestValue1")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(FloatMapCopy.GetName(FloatMapCopy.IndexOf(TEXT("TestValue0"))) == TEXT("TestValue0"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(FloatMapCopy.GetName(FloatMapCopy.IndexOf(TEXT("TestValue1"))) == TEXT("TestValue1"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(FloatMapCopy.GetValue(FloatMapCopy.IndexOf(TEXT("TestValue0"))).Value == Value0.Value, TEXT("Unexpected mapped value"));
				AddErrorIfFalse(FloatMapCopy.GetValue(FloatMapCopy.IndexOf(TEXT("TestValue1"))).Value == Value1.Value, TEXT("Unexpected mapped value"));

				ReleaseUnboundValueMap(&FloatMapCopy);
			}

			// Move
			{
				const int32 NumValues = FloatMap.Num();
				const int32 MaxValues = FloatMap.Max();

				TUnboundValueMap<FFloatAnimationAttribute>& FloatMapMoved = *MakeUnboundValueMap<FFloatAnimationAttribute>(Allocator);
				FloatMap.MoveTo(FloatMapMoved);

				AddErrorIfFalse(!FloatMapMoved.IsEmpty(), TEXT("Attribute map should not be empty"));
				AddErrorIfFalse(FloatMapMoved.Num() == NumValues, TEXT("Unexpected map size"));
				AddErrorIfFalse(FloatMapMoved.Max() == MaxValues, TEXT("Unexpected map capacity"));
				AddErrorIfFalse(FloatMapMoved.GetValueType() == FloatMap.GetValueType(), TEXT("Unexpected map value type"));

				AddErrorIfFalse(FloatMap.IsEmpty(), TEXT("Attribute map should be empty"));
				AddErrorIfFalse(FloatMap.Num() == 0, TEXT("Unexpected map size"));
				AddErrorIfFalse(FloatMap.Max() == 0, TEXT("Unexpected map size"));

				AddErrorIfFalse(FloatMapMoved.Contains(TEXT("TestValue0")), TEXT("Failed to move map"));
				AddErrorIfFalse(FloatMapMoved.Contains(TEXT("TestValue1")), TEXT("Failed to move map"));
				UE_RETURN_ON_ERROR(FloatMapMoved.IndexOf(TEXT("TestValue0")) != INDEX_NONE, TEXT("Failed to move map"));
				UE_RETURN_ON_ERROR(FloatMapMoved.IndexOf(TEXT("TestValue1")) != INDEX_NONE, TEXT("Failed to move map"));
				AddErrorIfFalse(FloatMapMoved.Find(TEXT("TestValue0")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(FloatMapMoved.Find(TEXT("TestValue1")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(FloatMapMoved.GetName(FloatMapMoved.IndexOf(TEXT("TestValue0"))) == TEXT("TestValue0"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(FloatMapMoved.GetName(FloatMapMoved.IndexOf(TEXT("TestValue1"))) == TEXT("TestValue1"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(FloatMapMoved.GetValue(FloatMapMoved.IndexOf(TEXT("TestValue0"))).Value == Value0.Value, TEXT("Unexpected mapped value"));
				AddErrorIfFalse(FloatMapMoved.GetValue(FloatMapMoved.IndexOf(TEXT("TestValue1"))).Value == Value1.Value, TEXT("Unexpected mapped value"));

				// Move it back to restore the original
				FloatMapMoved.MoveTo(FloatMap);

				AddErrorIfFalse(FloatMapMoved.IsEmpty(), TEXT("Attribute map should be empty"));
				AddErrorIfFalse(FloatMapMoved.Num() == 0, TEXT("Unexpected map size"));
				AddErrorIfFalse(FloatMapMoved.Max() == 0, TEXT("Unexpected map capacity"));

				AddErrorIfFalse(!FloatMap.IsEmpty(), TEXT("Attribute map should not be empty"));
				AddErrorIfFalse(FloatMap.Num() == NumValues, TEXT("Unexpected map size"));
				AddErrorIfFalse(FloatMap.Max() == MaxValues, TEXT("Unexpected map capacity"));

				ReleaseUnboundValueMap(&FloatMapMoved);
			}

			// Duplicate
			{
				TUnboundValueMap<FFloatAnimationAttribute>& FloatMapCopy = *reinterpret_cast<TUnboundValueMap<FFloatAnimationAttribute>*>(FloatMap.Duplicate(Allocator));

				AddErrorIfFalse(!FloatMapCopy.IsEmpty(), TEXT("Attribute map should not be empty"));
				AddErrorIfFalse(FloatMapCopy.Num() == FloatMap.Num(), TEXT("Unexpected map size"));
				AddErrorIfFalse(FloatMapCopy.Max() == FloatMap.Num(), TEXT("Unexpected map capacity"));
				AddErrorIfFalse(FloatMapCopy.GetValueType() == FloatMap.GetValueType(), TEXT("Unexpected map value type"));

				AddErrorIfFalse(FloatMapCopy.Contains(TEXT("TestValue0")), TEXT("Failed to copy map"));
				AddErrorIfFalse(FloatMapCopy.Contains(TEXT("TestValue1")), TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(FloatMapCopy.IndexOf(TEXT("TestValue0")) != INDEX_NONE, TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(FloatMapCopy.IndexOf(TEXT("TestValue1")) != INDEX_NONE, TEXT("Failed to copy map"));
				AddErrorIfFalse(FloatMapCopy.Find(TEXT("TestValue0")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(FloatMapCopy.Find(TEXT("TestValue1")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(FloatMapCopy.GetName(FloatMapCopy.IndexOf(TEXT("TestValue0"))) == TEXT("TestValue0"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(FloatMapCopy.GetName(FloatMapCopy.IndexOf(TEXT("TestValue1"))) == TEXT("TestValue1"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(FloatMapCopy.GetValue(FloatMapCopy.IndexOf(TEXT("TestValue0"))).Value == Value0.Value, TEXT("Unexpected mapped value"));
				AddErrorIfFalse(FloatMapCopy.GetValue(FloatMapCopy.IndexOf(TEXT("TestValue1"))).Value == Value1.Value, TEXT("Unexpected mapped value"));

				ReleaseUnboundValueMap(&FloatMapCopy);
			}

			// Set sorted
			{
				TUnboundValueMap<FFloatAnimationAttribute>& FloatMapCopy = *MakeUnboundValueMap<FFloatAnimationAttribute>(Allocator);

				FloatMapCopy.SetSorted(
					FloatMap.Num(),
					[&FloatMap](int32 Index) { return FloatMap.GetName(Index); },
					[&FloatMap](int32 Index) { return FloatMap.GetValue(Index); });

				AddErrorIfFalse(!FloatMapCopy.IsEmpty(), TEXT("Attribute map should not be empty"));
				AddErrorIfFalse(FloatMapCopy.Num() == FloatMap.Num(), TEXT("Unexpected map size"));
				AddErrorIfFalse(FloatMapCopy.Max() == FloatMap.Num(), TEXT("Unexpected map capacity"));
				AddErrorIfFalse(FloatMapCopy.GetValueType() == FloatMap.GetValueType(), TEXT("Unexpected map value type"));

				AddErrorIfFalse(FloatMapCopy.Contains(TEXT("TestValue0")), TEXT("Failed to copy map"));
				AddErrorIfFalse(FloatMapCopy.Contains(TEXT("TestValue1")), TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(FloatMapCopy.IndexOf(TEXT("TestValue0")) != INDEX_NONE, TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(FloatMapCopy.IndexOf(TEXT("TestValue1")) != INDEX_NONE, TEXT("Failed to copy map"));
				AddErrorIfFalse(FloatMapCopy.Find(TEXT("TestValue0")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(FloatMapCopy.Find(TEXT("TestValue1")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(FloatMapCopy.GetName(FloatMapCopy.IndexOf(TEXT("TestValue0"))) == TEXT("TestValue0"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(FloatMapCopy.GetName(FloatMapCopy.IndexOf(TEXT("TestValue1"))) == TEXT("TestValue1"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(FloatMapCopy.GetValue(FloatMapCopy.IndexOf(TEXT("TestValue0"))).Value == Value0.Value, TEXT("Unexpected mapped value"));
				AddErrorIfFalse(FloatMapCopy.GetValue(FloatMapCopy.IndexOf(TEXT("TestValue1"))).Value == Value1.Value, TEXT("Unexpected mapped value"));

				ReleaseUnboundValueMap(&FloatMapCopy);
			}

			// Set unsorted
			{
				TUnboundValueMap<FFloatAnimationAttribute>& FloatMapCopy = *MakeUnboundValueMap<FFloatAnimationAttribute>(Allocator);

				FloatMapCopy.SetUnsorted(
					FloatMap.Num(),
					[&FloatMap](int32 Index) { return FloatMap.GetName(FloatMap.Num() - Index - 1); },
					[&FloatMap](int32 Index) { return FloatMap.GetValue(FloatMap.Num() - Index - 1); });

				AddErrorIfFalse(!FloatMapCopy.IsEmpty(), TEXT("Attribute map should not be empty"));
				AddErrorIfFalse(FloatMapCopy.Num() == FloatMap.Num(), TEXT("Unexpected map size"));
				AddErrorIfFalse(FloatMapCopy.Max() == FloatMap.Num(), TEXT("Unexpected map capacity"));
				AddErrorIfFalse(FloatMapCopy.GetValueType() == FloatMap.GetValueType(), TEXT("Unexpected map value type"));

				AddErrorIfFalse(FloatMapCopy.Contains(TEXT("TestValue0")), TEXT("Failed to copy map"));
				AddErrorIfFalse(FloatMapCopy.Contains(TEXT("TestValue1")), TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(FloatMapCopy.IndexOf(TEXT("TestValue0")) != INDEX_NONE, TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(FloatMapCopy.IndexOf(TEXT("TestValue1")) != INDEX_NONE, TEXT("Failed to copy map"));
				AddErrorIfFalse(FloatMapCopy.Find(TEXT("TestValue0")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(FloatMapCopy.Find(TEXT("TestValue1")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(FloatMapCopy.GetName(FloatMapCopy.IndexOf(TEXT("TestValue0"))) == TEXT("TestValue0"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(FloatMapCopy.GetName(FloatMapCopy.IndexOf(TEXT("TestValue1"))) == TEXT("TestValue1"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(FloatMapCopy.GetValue(FloatMapCopy.IndexOf(TEXT("TestValue0"))).Value == Value0.Value, TEXT("Unexpected mapped value"));
				AddErrorIfFalse(FloatMapCopy.GetValue(FloatMapCopy.IndexOf(TEXT("TestValue1"))).Value == Value1.Value, TEXT("Unexpected mapped value"));

				ReleaseUnboundValueMap(&FloatMapCopy);
			}

			// Append
			{
				TUnboundValueMap<FFloatAnimationAttribute>& FloatMapCopy = *MakeUnboundValueMap<FFloatAnimationAttribute>(Allocator);

				for (int32 Index = 0; Index < FloatMap.Num(); ++Index)
				{
					const bool bAppendSuccess = FloatMapCopy.Append(FloatMap.GetName(Index), FloatMap.GetValue(Index));
					AddErrorIfFalse(bAppendSuccess, TEXT("Failed to append"));
				}

				AddErrorIfFalse(!FloatMapCopy.IsEmpty(), TEXT("Attribute map should not be empty"));
				AddErrorIfFalse(FloatMapCopy.Num() == FloatMap.Num(), TEXT("Unexpected map size"));
				AddErrorIfFalse(FloatMapCopy.GetValueType() == FloatMap.GetValueType(), TEXT("Unexpected map value type"));

				AddErrorIfFalse(FloatMapCopy.Contains(TEXT("TestValue0")), TEXT("Failed to copy map"));
				AddErrorIfFalse(FloatMapCopy.Contains(TEXT("TestValue1")), TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(FloatMapCopy.IndexOf(TEXT("TestValue0")) != INDEX_NONE, TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(FloatMapCopy.IndexOf(TEXT("TestValue1")) != INDEX_NONE, TEXT("Failed to copy map"));
				AddErrorIfFalse(FloatMapCopy.Find(TEXT("TestValue0")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(FloatMapCopy.Find(TEXT("TestValue1")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(FloatMapCopy.GetName(FloatMapCopy.IndexOf(TEXT("TestValue0"))) == TEXT("TestValue0"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(FloatMapCopy.GetName(FloatMapCopy.IndexOf(TEXT("TestValue1"))) == TEXT("TestValue1"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(FloatMapCopy.GetValue(FloatMapCopy.IndexOf(TEXT("TestValue0"))).Value == Value0.Value, TEXT("Unexpected mapped value"));
				AddErrorIfFalse(FloatMapCopy.GetValue(FloatMapCopy.IndexOf(TEXT("TestValue1"))).Value == Value1.Value, TEXT("Unexpected mapped value"));

				ReleaseUnboundValueMap(&FloatMapCopy);
			}

			// Iterators
			{
				const auto GetNameProjection = [](const typename TUnboundValueMap<FFloatAnimationAttribute>::FIterator& It) { return It.GetName(); };
				const auto GetNameProjectionConst = [](const typename TUnboundValueMap<FFloatAnimationAttribute>::FConstIterator& It) { return It.GetName(); };
				const auto GetValueProjection = [](const typename TUnboundValueMap<FFloatAnimationAttribute>::FIterator& It) { return *It.GetValue(); };
				const auto GetValueProjectionConst = [](const typename TUnboundValueMap<FFloatAnimationAttribute>::FConstIterator& It) { return *It.GetValue(); };
				const auto TestValuePredicate = [](const FFloatAnimationAttribute& LHS, const FFloatAnimationAttribute& RHS) { return LHS.Value == RHS.Value; };

				AddErrorIfFalse(IteratorSize(FloatMap.CreateIterator()) == 5, TEXT("Iterator did not have expected size"));
				AddErrorIfFalse(IteratorSize(FloatMap.CreateConstIterator()) == 5, TEXT("Iterator did not have expected size"));

				AddErrorIfFalse(IteratorSortedByPredicate(FloatMap.CreateIterator(), GetNameProjection, FNameFastLess()), TEXT("Iterator should be sorted"));
				AddErrorIfFalse(IteratorSortedByPredicate(FloatMap.CreateConstIterator(), GetNameProjectionConst, FNameFastLess()), TEXT("Iterator should be sorted"));

				AddErrorIfFalse(FindWithBy(FloatMap.CreateIterator(), FName(TEXT("TestValue0")), GetNameProjection), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(FloatMap.CreateIterator(), FName(TEXT("TestValue1")), GetNameProjection), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(FloatMap.CreateIterator(), FName(TEXT("TestValueA")), GetNameProjection), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(FloatMap.CreateIterator(), FName(TEXT("TestValueB")), GetNameProjection), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(FloatMap.CreateIterator(), FName(TEXT("TestValueC")), GetNameProjection), TEXT("Iterator did not contain expected name"));

				AddErrorIfFalse(FindWithByPredicate(FloatMap.CreateIterator(), Value0, GetValueProjection, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(FloatMap.CreateIterator(), Value1, GetValueProjection, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(FloatMap.CreateIterator(), ValueA, GetValueProjection, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(FloatMap.CreateIterator(), ValueB, GetValueProjection, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(FloatMap.CreateIterator(), ValueC, GetValueProjection, TestValuePredicate), TEXT("Iterator did not contain expected value"));

				AddErrorIfFalse(FindWithBy(FloatMap.CreateConstIterator(), FName(TEXT("TestValue0")), GetNameProjectionConst), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(FloatMap.CreateConstIterator(), FName(TEXT("TestValue1")), GetNameProjectionConst), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(FloatMap.CreateConstIterator(), FName(TEXT("TestValueA")), GetNameProjectionConst), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(FloatMap.CreateConstIterator(), FName(TEXT("TestValueB")), GetNameProjectionConst), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(FloatMap.CreateConstIterator(), FName(TEXT("TestValueC")), GetNameProjectionConst), TEXT("Iterator did not contain expected name"));

				AddErrorIfFalse(FindWithByPredicate(FloatMap.CreateConstIterator(), Value0, GetValueProjectionConst, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(FloatMap.CreateConstIterator(), Value1, GetValueProjectionConst, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(FloatMap.CreateConstIterator(), ValueA, GetValueProjectionConst, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(FloatMap.CreateConstIterator(), ValueB, GetValueProjectionConst, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(FloatMap.CreateConstIterator(), ValueC, GetValueProjectionConst, TestValuePredicate), TEXT("Iterator did not contain expected value"));
			}

			ReleaseUnboundValueMap(&FloatMap);
		}

		// Bone transform value
		{
			TUnboundValueMap<FBoneTransformAnimationAttribute>& BoneTransformMap = *MakeUnboundValueMap<FBoneTransformAnimationAttribute>(Allocator);

			AddErrorIfFalse(BoneTransformMap.IsEmpty(), TEXT("Attribute map should be empty"));
			AddErrorIfFalse(BoneTransformMap.Num() == 0, TEXT("Unexpected map size"));
			AddErrorIfFalse(BoneTransformMap.Max() == 0, TEXT("Unexpected map capacity"));
			AddErrorIfFalse(BoneTransformMap.GetValueType() == FBoneTransformAnimationAttribute::StaticStruct(), TEXT("Unexpected map value type"));

			AddErrorIfFalse(!BoneTransformMap.Contains(TEXT("TestValue0")), TEXT("Attribute map should be empty"));
			AddErrorIfFalse(BoneTransformMap.IndexOf(TEXT("TestValue0")) == INDEX_NONE, TEXT("Attribute map should be empty"));
			AddErrorIfFalse(BoneTransformMap.Find(TEXT("TestValue0")) == nullptr, TEXT("Attribute map should be empty"));

			bool bSuccess = true;

			FBoneTransformAnimationAttribute Value0{ FTransform::Identity };
			FBoneTransformAnimationAttribute Value1{ FTransform(FVector(1.2, 3.4, 5.6)) };
			bSuccess &= BoneTransformMap.Add(TEXT("TestValue0"), Value0);
			bSuccess &= BoneTransformMap.Add(TEXT("TestValue1"), Value1);

			// Add a few more to trigger reallocation
			FBoneTransformAnimationAttribute ValueA{ FTransform(FVector(14.5, 3.4, 5.6)) };
			FBoneTransformAnimationAttribute ValueB{ FTransform(FVector(1.2, 117.8, 5.6)) };
			FBoneTransformAnimationAttribute ValueC{ FTransform(FVector(1.2, 3.4, 0.1345)) };
			bSuccess &= BoneTransformMap.Add(TEXT("TestValueA"), ValueA);
			bSuccess &= BoneTransformMap.Add(TEXT("TestValueB"), ValueB);
			bSuccess &= BoneTransformMap.Add(TEXT("TestValueC"), ValueC);

			AddErrorIfFalse(bSuccess, TEXT("Failed to add values to map"));
			AddErrorIfFalse(BoneTransformMap.Contains(TEXT("TestValue0")), TEXT("Failed to add value to map"));
			AddErrorIfFalse(BoneTransformMap.Contains(TEXT("TestValue1")), TEXT("Failed to add value to map"));
			AddErrorIfFalse(BoneTransformMap.Contains(TEXT("TestValueA")), TEXT("Failed to add value to map"));
			AddErrorIfFalse(BoneTransformMap.Contains(TEXT("TestValueB")), TEXT("Failed to add value to map"));
			AddErrorIfFalse(BoneTransformMap.Contains(TEXT("TestValueC")), TEXT("Failed to add value to map"));
			UE_RETURN_ON_ERROR(BoneTransformMap.IndexOf(TEXT("TestValue0")) != INDEX_NONE, TEXT("Failed to add value to map"));
			UE_RETURN_ON_ERROR(BoneTransformMap.IndexOf(TEXT("TestValue1")) != INDEX_NONE, TEXT("Failed to add value to map"));
			UE_RETURN_ON_ERROR(BoneTransformMap.IndexOf(TEXT("TestValueA")) != INDEX_NONE, TEXT("Failed to add value to map"));
			UE_RETURN_ON_ERROR(BoneTransformMap.IndexOf(TEXT("TestValueB")) != INDEX_NONE, TEXT("Failed to add value to map"));
			UE_RETURN_ON_ERROR(BoneTransformMap.IndexOf(TEXT("TestValueC")) != INDEX_NONE, TEXT("Failed to add value to map"));
			AddErrorIfFalse(BoneTransformMap.Find(TEXT("TestValue0")) != nullptr, TEXT("Failed to add value to map"));
			AddErrorIfFalse(BoneTransformMap.Find(TEXT("TestValue1")) != nullptr, TEXT("Failed to add value to map"));
			AddErrorIfFalse(BoneTransformMap.Find(TEXT("TestValueA")) != nullptr, TEXT("Failed to add value to map"));
			AddErrorIfFalse(BoneTransformMap.Find(TEXT("TestValueB")) != nullptr, TEXT("Failed to add value to map"));
			AddErrorIfFalse(BoneTransformMap.Find(TEXT("TestValueC")) != nullptr, TEXT("Failed to add value to map"));
			AddErrorIfFalse(BoneTransformMap.GetName(BoneTransformMap.IndexOf(TEXT("TestValue0"))) == TEXT("TestValue0"), TEXT("Unexpected mapped value"));
			AddErrorIfFalse(BoneTransformMap.GetName(BoneTransformMap.IndexOf(TEXT("TestValue1"))) == TEXT("TestValue1"), TEXT("Unexpected mapped value"));
			AddErrorIfFalse(BoneTransformMap.GetName(BoneTransformMap.IndexOf(TEXT("TestValueA"))) == TEXT("TestValueA"), TEXT("Unexpected mapped value"));
			AddErrorIfFalse(BoneTransformMap.GetName(BoneTransformMap.IndexOf(TEXT("TestValueB"))) == TEXT("TestValueB"), TEXT("Unexpected mapped value"));
			AddErrorIfFalse(BoneTransformMap.GetName(BoneTransformMap.IndexOf(TEXT("TestValueC"))) == TEXT("TestValueC"), TEXT("Unexpected mapped value"));
			AddErrorIfFalse(BoneTransformMap.GetValue(BoneTransformMap.IndexOf(TEXT("TestValue0"))).Value.Equals(Value0.Value), TEXT("Unexpected mapped value"));
			AddErrorIfFalse(BoneTransformMap.GetValue(BoneTransformMap.IndexOf(TEXT("TestValue1"))).Value.Equals(Value1.Value), TEXT("Unexpected mapped value"));
			AddErrorIfFalse(BoneTransformMap.GetValue(BoneTransformMap.IndexOf(TEXT("TestValueA"))).Value.Equals(ValueA.Value), TEXT("Unexpected mapped value"));
			AddErrorIfFalse(BoneTransformMap.GetValue(BoneTransformMap.IndexOf(TEXT("TestValueB"))).Value.Equals(ValueB.Value), TEXT("Unexpected mapped value"));
			AddErrorIfFalse(BoneTransformMap.GetValue(BoneTransformMap.IndexOf(TEXT("TestValueC"))).Value.Equals(ValueC.Value), TEXT("Unexpected mapped value"));

			// Add duplicate
			bSuccess = BoneTransformMap.Add(TEXT("TestValue0"), Value0);

			AddErrorIfFalse(!bSuccess, TEXT("Did not failed to add duplicate value to map"));
			AddErrorIfFalse(BoneTransformMap.Contains(TEXT("TestValue0")), TEXT("Failed to add value to map"));

			FBoneTransformAnimationAttribute Value2{ FTransform(FVector(0.233, 331.54, 571.146)) };
			bSuccess = BoneTransformMap.Add(TEXT("TestValue2"), Value2);

			AddErrorIfFalse(bSuccess, TEXT("Failed to add values to map"));
			AddErrorIfFalse(BoneTransformMap.Contains(TEXT("TestValue2")), TEXT("Failed to add value to map"));

			bSuccess = BoneTransformMap.Remove(TEXT("TestValue2"));

			AddErrorIfFalse(bSuccess, TEXT("Failed to remove value from map"));
			AddErrorIfFalse(!BoneTransformMap.Contains(TEXT("TestValue2")), TEXT("Failed to remove value from map"));

			// Remove value not contained

			bSuccess = BoneTransformMap.Remove(TEXT("TestValue2"));

			AddErrorIfFalse(!bSuccess, TEXT("Did not failed to remove missing value from map"));
			AddErrorIfFalse(!BoneTransformMap.Contains(TEXT("TestValue2")), TEXT("Failed to remove value from map"));

			// Copy
			{
				TUnboundValueMap<FBoneTransformAnimationAttribute>& BoneTransformMapCopy = *MakeUnboundValueMap<FBoneTransformAnimationAttribute>(Allocator);
				BoneTransformMap.CopyTo(BoneTransformMapCopy);

				AddErrorIfFalse(!BoneTransformMapCopy.IsEmpty(), TEXT("Attribute map should not be empty"));
				AddErrorIfFalse(BoneTransformMapCopy.Num() == BoneTransformMap.Num(), TEXT("Unexpected map size"));
				AddErrorIfFalse(BoneTransformMapCopy.Max() == BoneTransformMap.Num(), TEXT("Unexpected map capacity"));
				AddErrorIfFalse(BoneTransformMapCopy.GetValueType() == BoneTransformMap.GetValueType(), TEXT("Unexpected map value type"));

				AddErrorIfFalse(BoneTransformMapCopy.Contains(TEXT("TestValue0")), TEXT("Failed to copy map"));
				AddErrorIfFalse(BoneTransformMapCopy.Contains(TEXT("TestValue1")), TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(BoneTransformMapCopy.IndexOf(TEXT("TestValue0")) != INDEX_NONE, TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(BoneTransformMapCopy.IndexOf(TEXT("TestValue1")) != INDEX_NONE, TEXT("Failed to copy map"));
				AddErrorIfFalse(BoneTransformMapCopy.Find(TEXT("TestValue0")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(BoneTransformMapCopy.Find(TEXT("TestValue1")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(BoneTransformMapCopy.GetName(BoneTransformMapCopy.IndexOf(TEXT("TestValue0"))) == TEXT("TestValue0"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(BoneTransformMapCopy.GetName(BoneTransformMapCopy.IndexOf(TEXT("TestValue1"))) == TEXT("TestValue1"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(BoneTransformMapCopy.GetValue(BoneTransformMapCopy.IndexOf(TEXT("TestValue0"))).Value.Equals(Value0.Value), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(BoneTransformMapCopy.GetValue(BoneTransformMapCopy.IndexOf(TEXT("TestValue1"))).Value.Equals(Value1.Value), TEXT("Unexpected mapped value"));

				ReleaseUnboundValueMap(&BoneTransformMapCopy);
			}

			// Move
			{
				const int32 NumValues = BoneTransformMap.Num();
				const int32 MaxValues = BoneTransformMap.Max();

				TUnboundValueMap<FBoneTransformAnimationAttribute>& BoneTransformMapMoved = *MakeUnboundValueMap<FBoneTransformAnimationAttribute>(Allocator);
				BoneTransformMap.MoveTo(BoneTransformMapMoved);

				AddErrorIfFalse(!BoneTransformMapMoved.IsEmpty(), TEXT("Attribute map should not be empty"));
				AddErrorIfFalse(BoneTransformMapMoved.Num() == NumValues, TEXT("Unexpected map size"));
				AddErrorIfFalse(BoneTransformMapMoved.Max() == MaxValues, TEXT("Unexpected map capacity"));
				AddErrorIfFalse(BoneTransformMapMoved.GetValueType() == BoneTransformMap.GetValueType(), TEXT("Unexpected map value type"));

				AddErrorIfFalse(BoneTransformMap.IsEmpty(), TEXT("Attribute map should be empty"));
				AddErrorIfFalse(BoneTransformMap.Num() == 0, TEXT("Unexpected map size"));
				AddErrorIfFalse(BoneTransformMap.Max() == 0, TEXT("Unexpected map size"));

				AddErrorIfFalse(BoneTransformMapMoved.Contains(TEXT("TestValue0")), TEXT("Failed to move map"));
				AddErrorIfFalse(BoneTransformMapMoved.Contains(TEXT("TestValue1")), TEXT("Failed to move map"));
				UE_RETURN_ON_ERROR(BoneTransformMapMoved.IndexOf(TEXT("TestValue0")) != INDEX_NONE, TEXT("Failed to move map"));
				UE_RETURN_ON_ERROR(BoneTransformMapMoved.IndexOf(TEXT("TestValue1")) != INDEX_NONE, TEXT("Failed to move map"));
				AddErrorIfFalse(BoneTransformMapMoved.Find(TEXT("TestValue0")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(BoneTransformMapMoved.Find(TEXT("TestValue1")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(BoneTransformMapMoved.GetName(BoneTransformMapMoved.IndexOf(TEXT("TestValue0"))) == TEXT("TestValue0"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(BoneTransformMapMoved.GetName(BoneTransformMapMoved.IndexOf(TEXT("TestValue1"))) == TEXT("TestValue1"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(BoneTransformMapMoved.GetValue(BoneTransformMapMoved.IndexOf(TEXT("TestValue0"))).Value.Equals(Value0.Value), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(BoneTransformMapMoved.GetValue(BoneTransformMapMoved.IndexOf(TEXT("TestValue1"))).Value.Equals(Value1.Value), TEXT("Unexpected mapped value"));

				// Move it back to restore the original
				BoneTransformMapMoved.MoveTo(BoneTransformMap);

				AddErrorIfFalse(BoneTransformMapMoved.IsEmpty(), TEXT("Attribute map should be empty"));
				AddErrorIfFalse(BoneTransformMapMoved.Num() == 0, TEXT("Unexpected map size"));
				AddErrorIfFalse(BoneTransformMapMoved.Max() == 0, TEXT("Unexpected map capacity"));

				AddErrorIfFalse(!BoneTransformMap.IsEmpty(), TEXT("Attribute map should not be empty"));
				AddErrorIfFalse(BoneTransformMap.Num() == NumValues, TEXT("Unexpected map size"));
				AddErrorIfFalse(BoneTransformMap.Max() == MaxValues, TEXT("Unexpected map capacity"));

				ReleaseUnboundValueMap(&BoneTransformMapMoved);
			}

			// Duplicate
			{
				TUnboundValueMap<FBoneTransformAnimationAttribute>& BoneTransformMapCopy = *reinterpret_cast<TUnboundValueMap<FBoneTransformAnimationAttribute>*>(BoneTransformMap.Duplicate(Allocator));

				AddErrorIfFalse(!BoneTransformMapCopy.IsEmpty(), TEXT("Attribute map should not be empty"));
				AddErrorIfFalse(BoneTransformMapCopy.Num() == BoneTransformMap.Num(), TEXT("Unexpected map size"));
				AddErrorIfFalse(BoneTransformMapCopy.Max() == BoneTransformMap.Num(), TEXT("Unexpected map capacity"));
				AddErrorIfFalse(BoneTransformMapCopy.GetValueType() == BoneTransformMap.GetValueType(), TEXT("Unexpected map value type"));

				AddErrorIfFalse(BoneTransformMapCopy.Contains(TEXT("TestValue0")), TEXT("Failed to copy map"));
				AddErrorIfFalse(BoneTransformMapCopy.Contains(TEXT("TestValue1")), TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(BoneTransformMapCopy.IndexOf(TEXT("TestValue0")) != INDEX_NONE, TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(BoneTransformMapCopy.IndexOf(TEXT("TestValue1")) != INDEX_NONE, TEXT("Failed to copy map"));
				AddErrorIfFalse(BoneTransformMapCopy.Find(TEXT("TestValue0")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(BoneTransformMapCopy.Find(TEXT("TestValue1")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(BoneTransformMapCopy.GetName(BoneTransformMapCopy.IndexOf(TEXT("TestValue0"))) == TEXT("TestValue0"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(BoneTransformMapCopy.GetName(BoneTransformMapCopy.IndexOf(TEXT("TestValue1"))) == TEXT("TestValue1"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(BoneTransformMapCopy.GetValue(BoneTransformMapCopy.IndexOf(TEXT("TestValue0"))).Value.Equals(Value0.Value), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(BoneTransformMapCopy.GetValue(BoneTransformMapCopy.IndexOf(TEXT("TestValue1"))).Value.Equals(Value1.Value), TEXT("Unexpected mapped value"));

				ReleaseUnboundValueMap(&BoneTransformMapCopy);
			}

			// Set sorted
			{
				TUnboundValueMap<FBoneTransformAnimationAttribute>& BoneTransformMapCopy = *MakeUnboundValueMap<FBoneTransformAnimationAttribute>(Allocator);

				BoneTransformMapCopy.SetSorted(
					BoneTransformMap.Num(),
					[&BoneTransformMap](int32 Index) { return BoneTransformMap.GetName(Index); },
					[&BoneTransformMap](int32 Index) { return BoneTransformMap.GetValue(Index); });

				AddErrorIfFalse(!BoneTransformMapCopy.IsEmpty(), TEXT("Attribute map should not be empty"));
				AddErrorIfFalse(BoneTransformMapCopy.Num() == BoneTransformMap.Num(), TEXT("Unexpected map size"));
				AddErrorIfFalse(BoneTransformMapCopy.Max() == BoneTransformMap.Num(), TEXT("Unexpected map capacity"));
				AddErrorIfFalse(BoneTransformMapCopy.GetValueType() == BoneTransformMap.GetValueType(), TEXT("Unexpected map value type"));

				AddErrorIfFalse(BoneTransformMapCopy.Contains(TEXT("TestValue0")), TEXT("Failed to copy map"));
				AddErrorIfFalse(BoneTransformMapCopy.Contains(TEXT("TestValue1")), TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(BoneTransformMapCopy.IndexOf(TEXT("TestValue0")) != INDEX_NONE, TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(BoneTransformMapCopy.IndexOf(TEXT("TestValue1")) != INDEX_NONE, TEXT("Failed to copy map"));
				AddErrorIfFalse(BoneTransformMapCopy.Find(TEXT("TestValue0")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(BoneTransformMapCopy.Find(TEXT("TestValue1")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(BoneTransformMapCopy.GetName(BoneTransformMapCopy.IndexOf(TEXT("TestValue0"))) == TEXT("TestValue0"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(BoneTransformMapCopy.GetName(BoneTransformMapCopy.IndexOf(TEXT("TestValue1"))) == TEXT("TestValue1"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(BoneTransformMapCopy.GetValue(BoneTransformMapCopy.IndexOf(TEXT("TestValue0"))).Value.Equals(Value0.Value), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(BoneTransformMapCopy.GetValue(BoneTransformMapCopy.IndexOf(TEXT("TestValue1"))).Value.Equals(Value1.Value), TEXT("Unexpected mapped value"));

				ReleaseUnboundValueMap(&BoneTransformMapCopy);
			}

			// Set unsorted
			{
				TUnboundValueMap<FBoneTransformAnimationAttribute>& BoneTransformMapCopy = *MakeUnboundValueMap<FBoneTransformAnimationAttribute>(Allocator);

				BoneTransformMapCopy.SetUnsorted(
					BoneTransformMap.Num(),
					[&BoneTransformMap](int32 Index) { return BoneTransformMap.GetName(BoneTransformMap.Num() - Index - 1); },
					[&BoneTransformMap](int32 Index) { return BoneTransformMap.GetValue(BoneTransformMap.Num() - Index - 1); });

				AddErrorIfFalse(!BoneTransformMapCopy.IsEmpty(), TEXT("Attribute map should not be empty"));
				AddErrorIfFalse(BoneTransformMapCopy.Num() == BoneTransformMap.Num(), TEXT("Unexpected map size"));
				AddErrorIfFalse(BoneTransformMapCopy.Max() == BoneTransformMap.Num(), TEXT("Unexpected map capacity"));
				AddErrorIfFalse(BoneTransformMapCopy.GetValueType() == BoneTransformMap.GetValueType(), TEXT("Unexpected map value type"));

				AddErrorIfFalse(BoneTransformMapCopy.Contains(TEXT("TestValue0")), TEXT("Failed to copy map"));
				AddErrorIfFalse(BoneTransformMapCopy.Contains(TEXT("TestValue1")), TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(BoneTransformMapCopy.IndexOf(TEXT("TestValue0")) != INDEX_NONE, TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(BoneTransformMapCopy.IndexOf(TEXT("TestValue1")) != INDEX_NONE, TEXT("Failed to copy map"));
				AddErrorIfFalse(BoneTransformMapCopy.Find(TEXT("TestValue0")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(BoneTransformMapCopy.Find(TEXT("TestValue1")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(BoneTransformMapCopy.GetName(BoneTransformMapCopy.IndexOf(TEXT("TestValue0"))) == TEXT("TestValue0"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(BoneTransformMapCopy.GetName(BoneTransformMapCopy.IndexOf(TEXT("TestValue1"))) == TEXT("TestValue1"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(BoneTransformMapCopy.GetValue(BoneTransformMapCopy.IndexOf(TEXT("TestValue0"))).Value.Equals(Value0.Value), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(BoneTransformMapCopy.GetValue(BoneTransformMapCopy.IndexOf(TEXT("TestValue1"))).Value.Equals(Value1.Value), TEXT("Unexpected mapped value"));

				ReleaseUnboundValueMap(&BoneTransformMapCopy);
			}

			// Append
			{
				TUnboundValueMap<FBoneTransformAnimationAttribute>& BoneTransformMapCopy = *MakeUnboundValueMap<FBoneTransformAnimationAttribute>(Allocator);

				for (int32 Index = 0; Index < BoneTransformMap.Num(); ++Index)
				{
					const bool bAppendSuccess = BoneTransformMapCopy.Append(BoneTransformMap.GetName(Index), BoneTransformMap.GetValue(Index));
					AddErrorIfFalse(bAppendSuccess, TEXT("Failed to append"));
				}

				AddErrorIfFalse(!BoneTransformMapCopy.IsEmpty(), TEXT("Attribute map should not be empty"));
				AddErrorIfFalse(BoneTransformMapCopy.Num() == BoneTransformMap.Num(), TEXT("Unexpected map size"));
				AddErrorIfFalse(BoneTransformMapCopy.GetValueType() == BoneTransformMap.GetValueType(), TEXT("Unexpected map value type"));

				AddErrorIfFalse(BoneTransformMapCopy.Contains(TEXT("TestValue0")), TEXT("Failed to copy map"));
				AddErrorIfFalse(BoneTransformMapCopy.Contains(TEXT("TestValue1")), TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(BoneTransformMapCopy.IndexOf(TEXT("TestValue0")) != INDEX_NONE, TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(BoneTransformMapCopy.IndexOf(TEXT("TestValue1")) != INDEX_NONE, TEXT("Failed to copy map"));
				AddErrorIfFalse(BoneTransformMapCopy.Find(TEXT("TestValue0")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(BoneTransformMapCopy.Find(TEXT("TestValue1")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(BoneTransformMapCopy.GetName(BoneTransformMapCopy.IndexOf(TEXT("TestValue0"))) == TEXT("TestValue0"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(BoneTransformMapCopy.GetName(BoneTransformMapCopy.IndexOf(TEXT("TestValue1"))) == TEXT("TestValue1"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(BoneTransformMapCopy.GetValue(BoneTransformMapCopy.IndexOf(TEXT("TestValue0"))).Value.Equals(Value0.Value), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(BoneTransformMapCopy.GetValue(BoneTransformMapCopy.IndexOf(TEXT("TestValue1"))).Value.Equals(Value1.Value), TEXT("Unexpected mapped value"));

				ReleaseUnboundValueMap(&BoneTransformMapCopy);
			}

			// Iterators
			{
				const auto GetNameProjection = [](const typename TUnboundValueMap<FBoneTransformAnimationAttribute>::FIterator& It) { return It.GetName(); };
				const auto GetNameProjectionConst = [](const typename TUnboundValueMap<FBoneTransformAnimationAttribute>::FConstIterator& It) { return It.GetName(); };
				const auto GetValueProjection = [](const typename TUnboundValueMap<FBoneTransformAnimationAttribute>::FIterator& It) { return *It.GetValue(); };
				const auto GetValueProjectionConst = [](const typename TUnboundValueMap<FBoneTransformAnimationAttribute>::FConstIterator& It) { return *It.GetValue(); };
				const auto TestValuePredicate = [](const FBoneTransformAnimationAttribute& LHS, const FBoneTransformAnimationAttribute& RHS) { return LHS.Value.Equals(RHS.Value); };

				AddErrorIfFalse(IteratorSize(BoneTransformMap.CreateIterator()) == 5, TEXT("Iterator did not have expected size"));
				AddErrorIfFalse(IteratorSize(BoneTransformMap.CreateConstIterator()) == 5, TEXT("Iterator did not have expected size"));

				AddErrorIfFalse(IteratorSortedByPredicate(BoneTransformMap.CreateIterator(), GetNameProjection, FNameFastLess()), TEXT("Iterator should be sorted"));
				AddErrorIfFalse(IteratorSortedByPredicate(BoneTransformMap.CreateConstIterator(), GetNameProjectionConst, FNameFastLess()), TEXT("Iterator should be sorted"));

				AddErrorIfFalse(FindWithBy(BoneTransformMap.CreateIterator(), FName(TEXT("TestValue0")), GetNameProjection), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(BoneTransformMap.CreateIterator(), FName(TEXT("TestValue1")), GetNameProjection), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(BoneTransformMap.CreateIterator(), FName(TEXT("TestValueA")), GetNameProjection), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(BoneTransformMap.CreateIterator(), FName(TEXT("TestValueB")), GetNameProjection), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(BoneTransformMap.CreateIterator(), FName(TEXT("TestValueC")), GetNameProjection), TEXT("Iterator did not contain expected name"));

				AddErrorIfFalse(FindWithByPredicate(BoneTransformMap.CreateIterator(), Value0, GetValueProjection, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(BoneTransformMap.CreateIterator(), Value1, GetValueProjection, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(BoneTransformMap.CreateIterator(), ValueA, GetValueProjection, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(BoneTransformMap.CreateIterator(), ValueB, GetValueProjection, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(BoneTransformMap.CreateIterator(), ValueC, GetValueProjection, TestValuePredicate), TEXT("Iterator did not contain expected value"));

				AddErrorIfFalse(FindWithBy(BoneTransformMap.CreateConstIterator(), FName(TEXT("TestValue0")), GetNameProjectionConst), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(BoneTransformMap.CreateConstIterator(), FName(TEXT("TestValue1")), GetNameProjectionConst), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(BoneTransformMap.CreateConstIterator(), FName(TEXT("TestValueA")), GetNameProjectionConst), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(BoneTransformMap.CreateConstIterator(), FName(TEXT("TestValueB")), GetNameProjectionConst), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(BoneTransformMap.CreateConstIterator(), FName(TEXT("TestValueC")), GetNameProjectionConst), TEXT("Iterator did not contain expected name"));

				AddErrorIfFalse(FindWithByPredicate(BoneTransformMap.CreateConstIterator(), Value0, GetValueProjectionConst, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(BoneTransformMap.CreateConstIterator(), Value1, GetValueProjectionConst, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(BoneTransformMap.CreateConstIterator(), ValueA, GetValueProjectionConst, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(BoneTransformMap.CreateConstIterator(), ValueB, GetValueProjectionConst, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(BoneTransformMap.CreateConstIterator(), ValueC, GetValueProjectionConst, TestValuePredicate), TEXT("Iterator did not contain expected value"));
			}

			ReleaseUnboundValueMap(&BoneTransformMap);
		}

		// String value
		{
			TUnboundValueMap<FStringAnimationAttribute>& StringMap = *MakeUnboundValueMap<FStringAnimationAttribute>(Allocator);

			AddErrorIfFalse(StringMap.IsEmpty(), TEXT("Attribute map should be empty"));
			AddErrorIfFalse(StringMap.Num() == 0, TEXT("Unexpected map size"));
			AddErrorIfFalse(StringMap.Max() == 0, TEXT("Unexpected map capacity"));
			AddErrorIfFalse(StringMap.GetValueType() == FStringAnimationAttribute::StaticStruct(), TEXT("Unexpected map value type"));

			AddErrorIfFalse(!StringMap.Contains(TEXT("TestValue0")), TEXT("Attribute map should be empty"));
			AddErrorIfFalse(StringMap.IndexOf(TEXT("TestValue0")) == INDEX_NONE, TEXT("Attribute map should be empty"));
			AddErrorIfFalse(StringMap.Find(TEXT("TestValue0")) == nullptr, TEXT("Attribute map should be empty"));

			bool bSuccess = true;

			FStringAnimationAttribute Value0{ TEXT("FooBar") };
			FStringAnimationAttribute Value1{ TEXT("Sweet123") };
			bSuccess &= StringMap.Add(TEXT("TestValue0"), Value0);
			bSuccess &= StringMap.Add(TEXT("TestValue1"), Value1);

			// Add a few more to trigger reallocation
			FStringAnimationAttribute ValueA{ TEXT("someValue154") };
			FStringAnimationAttribute ValueB{ TEXT("moreValj190") };
			FStringAnimationAttribute ValueC{ TEXT("test914") };
			bSuccess &= StringMap.Add(TEXT("TestValueA"), ValueA);
			bSuccess &= StringMap.Add(TEXT("TestValueB"), ValueB);
			bSuccess &= StringMap.Add(TEXT("TestValueC"), ValueC);

			AddErrorIfFalse(bSuccess, TEXT("Failed to add values to map"));
			AddErrorIfFalse(StringMap.Contains(TEXT("TestValue0")), TEXT("Failed to add value to map"));
			AddErrorIfFalse(StringMap.Contains(TEXT("TestValue1")), TEXT("Failed to add value to map"));
			AddErrorIfFalse(StringMap.Contains(TEXT("TestValueA")), TEXT("Failed to add value to map"));
			AddErrorIfFalse(StringMap.Contains(TEXT("TestValueB")), TEXT("Failed to add value to map"));
			AddErrorIfFalse(StringMap.Contains(TEXT("TestValueC")), TEXT("Failed to add value to map"));
			UE_RETURN_ON_ERROR(StringMap.IndexOf(TEXT("TestValue0")) != INDEX_NONE, TEXT("Failed to add value to map"));
			UE_RETURN_ON_ERROR(StringMap.IndexOf(TEXT("TestValue1")) != INDEX_NONE, TEXT("Failed to add value to map"));
			UE_RETURN_ON_ERROR(StringMap.IndexOf(TEXT("TestValueA")) != INDEX_NONE, TEXT("Failed to add value to map"));
			UE_RETURN_ON_ERROR(StringMap.IndexOf(TEXT("TestValueB")) != INDEX_NONE, TEXT("Failed to add value to map"));
			UE_RETURN_ON_ERROR(StringMap.IndexOf(TEXT("TestValueC")) != INDEX_NONE, TEXT("Failed to add value to map"));
			AddErrorIfFalse(StringMap.Find(TEXT("TestValue0")) != nullptr, TEXT("Failed to add value to map"));
			AddErrorIfFalse(StringMap.Find(TEXT("TestValue1")) != nullptr, TEXT("Failed to add value to map"));
			AddErrorIfFalse(StringMap.Find(TEXT("TestValueA")) != nullptr, TEXT("Failed to add value to map"));
			AddErrorIfFalse(StringMap.Find(TEXT("TestValueB")) != nullptr, TEXT("Failed to add value to map"));
			AddErrorIfFalse(StringMap.Find(TEXT("TestValueC")) != nullptr, TEXT("Failed to add value to map"));
			AddErrorIfFalse(StringMap.GetName(StringMap.IndexOf(TEXT("TestValue0"))) == TEXT("TestValue0"), TEXT("Unexpected mapped value"));
			AddErrorIfFalse(StringMap.GetName(StringMap.IndexOf(TEXT("TestValue1"))) == TEXT("TestValue1"), TEXT("Unexpected mapped value"));
			AddErrorIfFalse(StringMap.GetName(StringMap.IndexOf(TEXT("TestValueA"))) == TEXT("TestValueA"), TEXT("Unexpected mapped value"));
			AddErrorIfFalse(StringMap.GetName(StringMap.IndexOf(TEXT("TestValueB"))) == TEXT("TestValueB"), TEXT("Unexpected mapped value"));
			AddErrorIfFalse(StringMap.GetName(StringMap.IndexOf(TEXT("TestValueC"))) == TEXT("TestValueC"), TEXT("Unexpected mapped value"));
			AddErrorIfFalse(StringMap.GetValue(StringMap.IndexOf(TEXT("TestValue0"))).Value == Value0.Value, TEXT("Unexpected mapped value"));
			AddErrorIfFalse(StringMap.GetValue(StringMap.IndexOf(TEXT("TestValue1"))).Value == Value1.Value, TEXT("Unexpected mapped value"));
			AddErrorIfFalse(StringMap.GetValue(StringMap.IndexOf(TEXT("TestValueA"))).Value == ValueA.Value, TEXT("Unexpected mapped value"));
			AddErrorIfFalse(StringMap.GetValue(StringMap.IndexOf(TEXT("TestValueB"))).Value == ValueB.Value, TEXT("Unexpected mapped value"));
			AddErrorIfFalse(StringMap.GetValue(StringMap.IndexOf(TEXT("TestValueC"))).Value == ValueC.Value, TEXT("Unexpected mapped value"));

			// Add duplicate
			bSuccess = StringMap.Add(TEXT("TestValue0"), Value0);

			AddErrorIfFalse(!bSuccess, TEXT("Did not failed to add duplicate value to map"));
			AddErrorIfFalse(StringMap.Contains(TEXT("TestValue0")), TEXT("Failed to add value to map"));

			FStringAnimationAttribute Value2{ TEXT("01Wow") };
			bSuccess = StringMap.Add(TEXT("TestValue2"), Value2);

			AddErrorIfFalse(bSuccess, TEXT("Failed to add values to map"));
			AddErrorIfFalse(StringMap.Contains(TEXT("TestValue2")), TEXT("Failed to add value to map"));

			bSuccess = StringMap.Remove(TEXT("TestValue2"));

			AddErrorIfFalse(bSuccess, TEXT("Failed to remove value from map"));
			AddErrorIfFalse(!StringMap.Contains(TEXT("TestValue2")), TEXT("Failed to remove value from map"));

			// Remove value not contained

			bSuccess = StringMap.Remove(TEXT("TestValue2"));

			AddErrorIfFalse(!bSuccess, TEXT("Did not failed to remove missing value from map"));
			AddErrorIfFalse(!StringMap.Contains(TEXT("TestValue2")), TEXT("Failed to remove value from map"));

			// Copy
			{
				TUnboundValueMap<FStringAnimationAttribute>& StringMapCopy = *MakeUnboundValueMap<FStringAnimationAttribute>(Allocator);
				StringMap.CopyTo(StringMapCopy);

				AddErrorIfFalse(!StringMapCopy.IsEmpty(), TEXT("Attribute map should not be empty"));
				AddErrorIfFalse(StringMapCopy.Num() == StringMap.Num(), TEXT("Unexpected map size"));
				AddErrorIfFalse(StringMapCopy.Max() == StringMap.Num(), TEXT("Unexpected map capacity"));
				AddErrorIfFalse(StringMapCopy.GetValueType() == StringMap.GetValueType(), TEXT("Unexpected map value type"));

				AddErrorIfFalse(StringMapCopy.Contains(TEXT("TestValue0")), TEXT("Failed to copy map"));
				AddErrorIfFalse(StringMapCopy.Contains(TEXT("TestValue1")), TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(StringMapCopy.IndexOf(TEXT("TestValue0")) != INDEX_NONE, TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(StringMapCopy.IndexOf(TEXT("TestValue1")) != INDEX_NONE, TEXT("Failed to copy map"));
				AddErrorIfFalse(StringMapCopy.Find(TEXT("TestValue0")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(StringMapCopy.Find(TEXT("TestValue1")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(StringMapCopy.GetName(StringMapCopy.IndexOf(TEXT("TestValue0"))) == TEXT("TestValue0"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(StringMapCopy.GetName(StringMapCopy.IndexOf(TEXT("TestValue1"))) == TEXT("TestValue1"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(StringMapCopy.GetValue(StringMapCopy.IndexOf(TEXT("TestValue0"))).Value == Value0.Value, TEXT("Unexpected mapped value"));
				AddErrorIfFalse(StringMapCopy.GetValue(StringMapCopy.IndexOf(TEXT("TestValue1"))).Value == Value1.Value, TEXT("Unexpected mapped value"));

				ReleaseUnboundValueMap(&StringMapCopy);
			}


			// Move
			{
				const int32 NumValues = StringMap.Num();
				const int32 MaxValues = StringMap.Max();

				TUnboundValueMap<FStringAnimationAttribute>& StringMapMoved = *MakeUnboundValueMap<FStringAnimationAttribute>(Allocator);
				StringMap.MoveTo(StringMapMoved);

				AddErrorIfFalse(!StringMapMoved.IsEmpty(), TEXT("Attribute map should not be empty"));
				AddErrorIfFalse(StringMapMoved.Num() == NumValues, TEXT("Unexpected map size"));
				AddErrorIfFalse(StringMapMoved.Max() == MaxValues, TEXT("Unexpected map capacity"));
				AddErrorIfFalse(StringMapMoved.GetValueType() == StringMap.GetValueType(), TEXT("Unexpected map value type"));

				AddErrorIfFalse(StringMap.IsEmpty(), TEXT("Attribute map should be empty"));
				AddErrorIfFalse(StringMap.Num() == 0, TEXT("Unexpected map size"));
				AddErrorIfFalse(StringMap.Max() == 0, TEXT("Unexpected map size"));

				AddErrorIfFalse(StringMapMoved.Contains(TEXT("TestValue0")), TEXT("Failed to move map"));
				AddErrorIfFalse(StringMapMoved.Contains(TEXT("TestValue1")), TEXT("Failed to move map"));
				UE_RETURN_ON_ERROR(StringMapMoved.IndexOf(TEXT("TestValue0")) != INDEX_NONE, TEXT("Failed to move map"));
				UE_RETURN_ON_ERROR(StringMapMoved.IndexOf(TEXT("TestValue1")) != INDEX_NONE, TEXT("Failed to move map"));
				AddErrorIfFalse(StringMapMoved.Find(TEXT("TestValue0")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(StringMapMoved.Find(TEXT("TestValue1")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(StringMapMoved.GetName(StringMapMoved.IndexOf(TEXT("TestValue0"))) == TEXT("TestValue0"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(StringMapMoved.GetName(StringMapMoved.IndexOf(TEXT("TestValue1"))) == TEXT("TestValue1"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(StringMapMoved.GetValue(StringMapMoved.IndexOf(TEXT("TestValue0"))).Value == Value0.Value, TEXT("Unexpected mapped value"));
				AddErrorIfFalse(StringMapMoved.GetValue(StringMapMoved.IndexOf(TEXT("TestValue1"))).Value == Value1.Value, TEXT("Unexpected mapped value"));

				// Move it back to restore the original
				StringMapMoved.MoveTo(StringMap);

				AddErrorIfFalse(StringMapMoved.IsEmpty(), TEXT("Attribute map should be empty"));
				AddErrorIfFalse(StringMapMoved.Num() == 0, TEXT("Unexpected map size"));
				AddErrorIfFalse(StringMapMoved.Max() == 0, TEXT("Unexpected map capacity"));

				AddErrorIfFalse(!StringMap.IsEmpty(), TEXT("Attribute map should not be empty"));
				AddErrorIfFalse(StringMap.Num() == NumValues, TEXT("Unexpected map size"));
				AddErrorIfFalse(StringMap.Max() == MaxValues, TEXT("Unexpected map capacity"));

				ReleaseUnboundValueMap(&StringMapMoved);
			}

			// Duplicate
			{
				TUnboundValueMap<FStringAnimationAttribute>& StringMapCopy = *reinterpret_cast<TUnboundValueMap<FStringAnimationAttribute>*>(StringMap.Duplicate(Allocator));

				AddErrorIfFalse(!StringMapCopy.IsEmpty(), TEXT("Attribute map should not be empty"));
				AddErrorIfFalse(StringMapCopy.Num() == StringMap.Num(), TEXT("Unexpected map size"));
				AddErrorIfFalse(StringMapCopy.Max() == StringMap.Num(), TEXT("Unexpected map capacity"));
				AddErrorIfFalse(StringMapCopy.GetValueType() == StringMap.GetValueType(), TEXT("Unexpected map value type"));

				AddErrorIfFalse(StringMapCopy.Contains(TEXT("TestValue0")), TEXT("Failed to copy map"));
				AddErrorIfFalse(StringMapCopy.Contains(TEXT("TestValue1")), TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(StringMapCopy.IndexOf(TEXT("TestValue0")) != INDEX_NONE, TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(StringMapCopy.IndexOf(TEXT("TestValue1")) != INDEX_NONE, TEXT("Failed to copy map"));
				AddErrorIfFalse(StringMapCopy.Find(TEXT("TestValue0")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(StringMapCopy.Find(TEXT("TestValue1")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(StringMapCopy.GetName(StringMapCopy.IndexOf(TEXT("TestValue0"))) == TEXT("TestValue0"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(StringMapCopy.GetName(StringMapCopy.IndexOf(TEXT("TestValue1"))) == TEXT("TestValue1"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(StringMapCopy.GetValue(StringMapCopy.IndexOf(TEXT("TestValue0"))).Value == Value0.Value, TEXT("Unexpected mapped value"));
				AddErrorIfFalse(StringMapCopy.GetValue(StringMapCopy.IndexOf(TEXT("TestValue1"))).Value == Value1.Value, TEXT("Unexpected mapped value"));

				ReleaseUnboundValueMap(&StringMapCopy);
			}

			// Set sorted
			{
				TUnboundValueMap<FStringAnimationAttribute>& StringMapCopy = *MakeUnboundValueMap<FStringAnimationAttribute>(Allocator);

				StringMapCopy.SetSorted(
					StringMap.Num(),
					[&StringMap](int32 Index) { return StringMap.GetName(Index); },
					[&StringMap](int32 Index) { return StringMap.GetValue(Index); });

				AddErrorIfFalse(!StringMapCopy.IsEmpty(), TEXT("Attribute map should not be empty"));
				AddErrorIfFalse(StringMapCopy.Num() == StringMap.Num(), TEXT("Unexpected map size"));
				AddErrorIfFalse(StringMapCopy.Max() == StringMap.Num(), TEXT("Unexpected map capacity"));
				AddErrorIfFalse(StringMapCopy.GetValueType() == StringMap.GetValueType(), TEXT("Unexpected map value type"));

				AddErrorIfFalse(StringMapCopy.Contains(TEXT("TestValue0")), TEXT("Failed to copy map"));
				AddErrorIfFalse(StringMapCopy.Contains(TEXT("TestValue1")), TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(StringMapCopy.IndexOf(TEXT("TestValue0")) != INDEX_NONE, TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(StringMapCopy.IndexOf(TEXT("TestValue1")) != INDEX_NONE, TEXT("Failed to copy map"));
				AddErrorIfFalse(StringMapCopy.Find(TEXT("TestValue0")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(StringMapCopy.Find(TEXT("TestValue1")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(StringMapCopy.GetName(StringMapCopy.IndexOf(TEXT("TestValue0"))) == TEXT("TestValue0"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(StringMapCopy.GetName(StringMapCopy.IndexOf(TEXT("TestValue1"))) == TEXT("TestValue1"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(StringMapCopy.GetValue(StringMapCopy.IndexOf(TEXT("TestValue0"))).Value == Value0.Value, TEXT("Unexpected mapped value"));
				AddErrorIfFalse(StringMapCopy.GetValue(StringMapCopy.IndexOf(TEXT("TestValue1"))).Value == Value1.Value, TEXT("Unexpected mapped value"));

				ReleaseUnboundValueMap(&StringMapCopy);
			}

			// Set unsorted
			{
				TUnboundValueMap<FStringAnimationAttribute>& StringMapCopy = *MakeUnboundValueMap<FStringAnimationAttribute>(Allocator);

				StringMapCopy.SetUnsorted(
					StringMap.Num(),
					[&StringMap](int32 Index) { return StringMap.GetName(StringMap.Num() - Index - 1); },
					[&StringMap](int32 Index) { return StringMap.GetValue(StringMap.Num() - Index - 1); });

				AddErrorIfFalse(!StringMapCopy.IsEmpty(), TEXT("Attribute map should not be empty"));
				AddErrorIfFalse(StringMapCopy.Num() == StringMap.Num(), TEXT("Unexpected map size"));
				AddErrorIfFalse(StringMapCopy.Max() == StringMap.Num(), TEXT("Unexpected map capacity"));
				AddErrorIfFalse(StringMapCopy.GetValueType() == StringMap.GetValueType(), TEXT("Unexpected map value type"));

				AddErrorIfFalse(StringMapCopy.Contains(TEXT("TestValue0")), TEXT("Failed to copy map"));
				AddErrorIfFalse(StringMapCopy.Contains(TEXT("TestValue1")), TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(StringMapCopy.IndexOf(TEXT("TestValue0")) != INDEX_NONE, TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(StringMapCopy.IndexOf(TEXT("TestValue1")) != INDEX_NONE, TEXT("Failed to copy map"));
				AddErrorIfFalse(StringMapCopy.Find(TEXT("TestValue0")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(StringMapCopy.Find(TEXT("TestValue1")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(StringMapCopy.GetName(StringMapCopy.IndexOf(TEXT("TestValue0"))) == TEXT("TestValue0"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(StringMapCopy.GetName(StringMapCopy.IndexOf(TEXT("TestValue1"))) == TEXT("TestValue1"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(StringMapCopy.GetValue(StringMapCopy.IndexOf(TEXT("TestValue0"))).Value == Value0.Value, TEXT("Unexpected mapped value"));
				AddErrorIfFalse(StringMapCopy.GetValue(StringMapCopy.IndexOf(TEXT("TestValue1"))).Value == Value1.Value, TEXT("Unexpected mapped value"));

				ReleaseUnboundValueMap(&StringMapCopy);
			}

			// Append
			{
				TUnboundValueMap<FStringAnimationAttribute>& StringMapCopy = *MakeUnboundValueMap<FStringAnimationAttribute>(Allocator);

				for (int32 Index = 0; Index < StringMap.Num(); ++Index)
				{
					const bool bAppendSuccess = StringMapCopy.Append(StringMap.GetName(Index), StringMap.GetValue(Index));
					AddErrorIfFalse(bAppendSuccess, TEXT("Failed to append"));
				}

				AddErrorIfFalse(!StringMapCopy.IsEmpty(), TEXT("Attribute map should not be empty"));
				AddErrorIfFalse(StringMapCopy.Num() == StringMap.Num(), TEXT("Unexpected map size"));
				AddErrorIfFalse(StringMapCopy.GetValueType() == StringMap.GetValueType(), TEXT("Unexpected map value type"));

				AddErrorIfFalse(StringMapCopy.Contains(TEXT("TestValue0")), TEXT("Failed to copy map"));
				AddErrorIfFalse(StringMapCopy.Contains(TEXT("TestValue1")), TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(StringMapCopy.IndexOf(TEXT("TestValue0")) != INDEX_NONE, TEXT("Failed to copy map"));
				UE_RETURN_ON_ERROR(StringMapCopy.IndexOf(TEXT("TestValue1")) != INDEX_NONE, TEXT("Failed to copy map"));
				AddErrorIfFalse(StringMapCopy.Find(TEXT("TestValue0")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(StringMapCopy.Find(TEXT("TestValue1")) != nullptr, TEXT("Failed to add value to map"));
				AddErrorIfFalse(StringMapCopy.GetName(StringMapCopy.IndexOf(TEXT("TestValue0"))) == TEXT("TestValue0"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(StringMapCopy.GetName(StringMapCopy.IndexOf(TEXT("TestValue1"))) == TEXT("TestValue1"), TEXT("Unexpected mapped value"));
				AddErrorIfFalse(StringMapCopy.GetValue(StringMapCopy.IndexOf(TEXT("TestValue0"))).Value == Value0.Value, TEXT("Unexpected mapped value"));
				AddErrorIfFalse(StringMapCopy.GetValue(StringMapCopy.IndexOf(TEXT("TestValue1"))).Value == Value1.Value, TEXT("Unexpected mapped value"));

				ReleaseUnboundValueMap(&StringMapCopy);
			}

			// Iterators
			{
				const auto GetNameProjection = [](const typename TUnboundValueMap<FStringAnimationAttribute>::FIterator& It) { return It.GetName(); };
				const auto GetNameProjectionConst = [](const typename TUnboundValueMap<FStringAnimationAttribute>::FConstIterator& It) { return It.GetName(); };
				const auto GetValueProjection = [](const typename TUnboundValueMap<FStringAnimationAttribute>::FIterator& It) { return *It.GetValue(); };
				const auto GetValueProjectionConst = [](const typename TUnboundValueMap<FStringAnimationAttribute>::FConstIterator& It) { return *It.GetValue(); };
				const auto TestValuePredicate = [](const FStringAnimationAttribute& LHS, const FStringAnimationAttribute& RHS) { return LHS.Value == RHS.Value; };

				AddErrorIfFalse(IteratorSize(StringMap.CreateIterator()) == 5, TEXT("Iterator did not have expected size"));
				AddErrorIfFalse(IteratorSize(StringMap.CreateConstIterator()) == 5, TEXT("Iterator did not have expected size"));

				AddErrorIfFalse(IteratorSortedByPredicate(StringMap.CreateIterator(), GetNameProjection, FNameFastLess()), TEXT("Iterator should be sorted"));
				AddErrorIfFalse(IteratorSortedByPredicate(StringMap.CreateConstIterator(), GetNameProjectionConst, FNameFastLess()), TEXT("Iterator should be sorted"));

				AddErrorIfFalse(FindWithBy(StringMap.CreateIterator(), FName(TEXT("TestValue0")), GetNameProjection), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(StringMap.CreateIterator(), FName(TEXT("TestValue1")), GetNameProjection), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(StringMap.CreateIterator(), FName(TEXT("TestValueA")), GetNameProjection), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(StringMap.CreateIterator(), FName(TEXT("TestValueB")), GetNameProjection), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(StringMap.CreateIterator(), FName(TEXT("TestValueC")), GetNameProjection), TEXT("Iterator did not contain expected name"));

				AddErrorIfFalse(FindWithByPredicate(StringMap.CreateIterator(), Value0, GetValueProjection, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(StringMap.CreateIterator(), Value1, GetValueProjection, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(StringMap.CreateIterator(), ValueA, GetValueProjection, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(StringMap.CreateIterator(), ValueB, GetValueProjection, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(StringMap.CreateIterator(), ValueC, GetValueProjection, TestValuePredicate), TEXT("Iterator did not contain expected value"));

				AddErrorIfFalse(FindWithBy(StringMap.CreateConstIterator(), FName(TEXT("TestValue0")), GetNameProjectionConst), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(StringMap.CreateConstIterator(), FName(TEXT("TestValue1")), GetNameProjectionConst), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(StringMap.CreateConstIterator(), FName(TEXT("TestValueA")), GetNameProjectionConst), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(StringMap.CreateConstIterator(), FName(TEXT("TestValueB")), GetNameProjectionConst), TEXT("Iterator did not contain expected name"));
				AddErrorIfFalse(FindWithBy(StringMap.CreateConstIterator(), FName(TEXT("TestValueC")), GetNameProjectionConst), TEXT("Iterator did not contain expected name"));

				AddErrorIfFalse(FindWithByPredicate(StringMap.CreateConstIterator(), Value0, GetValueProjectionConst, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(StringMap.CreateConstIterator(), Value1, GetValueProjectionConst, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(StringMap.CreateConstIterator(), ValueA, GetValueProjectionConst, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(StringMap.CreateConstIterator(), ValueB, GetValueProjectionConst, TestValuePredicate), TEXT("Iterator did not contain expected value"));
				AddErrorIfFalse(FindWithByPredicate(StringMap.CreateConstIterator(), ValueC, GetValueProjectionConst, TestValuePredicate), TEXT("Iterator did not contain expected value"));
			}

			ReleaseUnboundValueMap(&StringMap);
		}

		return true;
	}
}

#endif	// WITH_DEV_AUTOMATION_TESTS
