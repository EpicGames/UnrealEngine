// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextTest.generated.h"

#define UE_API UAFTESTSUITE_API

//// --- Raw type ---
USTRUCT()
struct FAnimNextTestData
{
	GENERATED_BODY()

	float A = 0.f;
	float B = 0.f;
};

namespace UE::UAF::Tests
{

struct FUtils final
{
	// Clean up after tests. Clears transaction buffer, collects garbage
	static UE_API void CleanupAfterTests();
	static UE_API class UWorld* GetWorld();
};

}

#undef UE_API
