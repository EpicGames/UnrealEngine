// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class UAFTestSuite : ModuleRules
	{
		public UAFTestSuite(ReadOnlyTargetRules Target) : base(Target)
		{
			
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"UAF",
					"RigVM",
					"UniversalObjectLocator",
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"UnrealEd",
						"UAFUncookedOnly",
						"UAFEditor",
						"PythonScriptPlugin",
						"RigVMDeveloper",
					}
				);
			}
		}
	}
}