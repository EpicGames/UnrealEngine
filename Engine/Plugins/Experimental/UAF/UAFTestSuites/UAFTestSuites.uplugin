{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "UAF Tests",
	"Description": "UAF Automated Tests",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "UAFCQTestSuite",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name" : "UAFTestSuite",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "UAFAnimGraphTestSuite",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "UAFAnimNodeTestData",
			"Type" : "DeveloperTool",
			"LoadingPhase" : "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Workspace",
			"Enabled": true
		},
		{
			"Name": "UAFAnimNode",
			"Enabled": true
		},
		{
			"Name": "UAF",
			"Enabled": true
		},
		{
			"Name": "UAFAnimGraph",
			"Enabled": true		
		},		
		{
			"Name": "UAFStateTree",
			"Enabled": true		
		},
		{
			"Name": "RigVM",
			"Enabled": true		
		},
		{
			"Name": "PythonAutomationTest",
			"Enabled": true
		}
	]
}
